{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Cloth",
	"Description": "Adds Chaos Cloth modules.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosCloth",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "ChaosClothEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ChaosCaching",
			"Enabled": true
		},
		{
			"Name": "Buoyancy",
			"Enabled": true
		},
		{
			"Name": "Water",
			"Enabled": true
		}
	]
}
