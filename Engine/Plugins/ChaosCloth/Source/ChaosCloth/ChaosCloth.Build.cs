// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosCloth : ModuleRules
{
    public ChaosCloth(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "ClothingSystemRuntimeCommon",
                "ClothingSystemRuntimeInterface",
                "Engine",
                "Chaos",
                "ChaosCaching",
				"Buoyancy",
				"Water"
            }
		);

        SetupModulePhysicsSupport(Target);
		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

		bDisableAutoRTFMInstrumentation = true;
	}
}
