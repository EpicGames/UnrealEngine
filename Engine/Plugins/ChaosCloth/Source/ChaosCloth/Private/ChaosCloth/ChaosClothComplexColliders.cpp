// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothComplexColliders.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h"
#include "Chaos/PBDSoftBodyCollisionConstraint.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/SoftsEvolution.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "HAL/IConsoleManager.h"

namespace Chaos {

	FClothComplexColliders::FClothComplexColliders(
		Softs::FEvolution* InEvolution,
		int32 InCollisionRangeId)
	: Evolution(InEvolution)
	, CollisionRangeId(InCollisionRangeId)
	{
	}

	void FClothComplexColliders::Reset()
	{
		CollisionSubBones.Reset();
		SkinnedLevelSets.Reset();
		MLLevelSets.Reset();
		SkinnedTriangleMeshes.Reset();
	}

	void FClothComplexColliders::AddSubBoneIndices(const TArray<int32>& InSubBoneIndices)
	{
		const int32 Offset = CollisionSubBones.Size();
		CollisionSubBones.AddSubBones(InSubBoneIndices.Num());

		for (int32 Index = 0; Index < InSubBoneIndices.Num(); ++Index)
		{
			CollisionSubBones.BoneIndices[Index + Offset] = InSubBoneIndices[Index];
			CollisionSubBones.BaseTransforms[Index + Offset] = Softs::FSolverRigidTransform3::Identity;
		}
	}

	void FClothComplexColliders::AddSkinnedLevelSet(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& SkinnedLevelSet)
	{
		FSkinnedLevelSetData& NewData = SkinnedLevelSets.AddDefaulted_GetRef();
		NewData.Index = Index;
		NewData.MappedSubBones = MappedSubBones;
		NewData.SkinnedLevelSet = SkinnedLevelSet;
	}

	void FClothComplexColliders::AddMLLevelSet(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& MLLevelSet)
	{
		FMLLevelSetData& NewData = MLLevelSets.AddDefaulted_GetRef();
		NewData.Index = Index;
		NewData.MappedSubBones = MappedSubBones;
		NewData.MLLevelSet = MLLevelSet;
	}

	void FClothComplexColliders::AddSkinnedTriangleMesh(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& SkinnedTriangleMesh)
	{
		FSkinnedTriangleMeshData& NewData = SkinnedTriangleMeshes.AddDefaulted_GetRef();
		NewData.Index = Index;
		NewData.MappedSubBones = MappedSubBones;
		NewData.SkinnedTriangleMesh = SkinnedTriangleMesh;
		const FSkinnedTriangleMesh& TriMesh = SkinnedTriangleMesh->GetObjectChecked<FSkinnedTriangleMesh>();
		NewData.SkinnedPositions.SetNum(TriMesh.GetBoneData().Num());
	}

	void FClothComplexColliders::Update(const Softs::FSolverTransform3& ComponentToLocalSpace, const TArray<FTransform>& BoneTransforms, const TConstArrayView<Softs::FSolverRigidTransform3>& CollisionRangeTransforms)
	{
		for (uint32 Index = 0; Index < CollisionSubBones.Size(); ++Index)
		{
			const int32 BoneIndex = CollisionSubBones.BoneIndices[Index];
			CollisionSubBones.Transforms[Index] = BoneTransforms.IsValidIndex(BoneIndex) ?
				CollisionSubBones.BaseTransforms[Index] * Softs::FSolverTransform3(BoneTransforms[BoneIndex]) * ComponentToLocalSpace :
				CollisionSubBones.BaseTransforms[Index] * ComponentToLocalSpace;
		}

		for (FSkinnedTriangleMeshData& Data : SkinnedTriangleMeshes)
		{
			const FTransform RootTransformInv = FTransform(CollisionRangeTransforms[Data.Index].Inverse());
			TArray<FTransform> SubBoneTransforms;
			SubBoneTransforms.Reserve(Data.MappedSubBones.Num());
			for (int32 SubBoneIndex : Data.MappedSubBones)
			{
				SubBoneTransforms.Emplace(TRigidTransform<FReal, 3>(CollisionSubBones.Transforms[SubBoneIndex]) * RootTransformInv);
			}
			FSkinnedTriangleMesh& SkinnedTriangleMesh = Data.SkinnedTriangleMesh->GetObjectChecked<FSkinnedTriangleMesh>();
			SkinnedTriangleMesh.SkinPositions(SubBoneTransforms, TArrayView<Softs::FSolverVec3>(Data.SkinnedPositions.Positions));
		}
	}

	void FClothComplexColliders::ResetStartPose()
	{
		for (uint32 Index = 0; Index < CollisionSubBones.Size(); ++Index)
		{
			CollisionSubBones.OldTransforms[Index] = CollisionSubBones.Transforms[Index];
			CollisionSubBones.X[Index] = CollisionSubBones.Transforms[Index].GetTranslation();
			CollisionSubBones.R[Index] = CollisionSubBones.Transforms[Index].GetRotation();
			CollisionSubBones.V[Index] = CollisionSubBones.W[Index] = Softs::FSolverVec3(0.f);
		}
		for (FSkinnedTriangleMeshData& Data : SkinnedTriangleMeshes)
		{
			for (int32 VtxIndex = 0; VtxIndex < Data.SkinnedPositions.Positions.Num(); ++VtxIndex)
			{
				Data.SkinnedPositions.OldPositions[VtxIndex] = Data.SkinnedPositions.Positions[VtxIndex];
				Data.SkinnedPositions.SolverSpaceVelocities[VtxIndex] = Softs::FSolverVec3(0.f);
			}
		}
	}

	void FClothComplexColliders::SwapBuffersForFrameFlip()
	{
		Swap(CollisionSubBones.OldTransforms, CollisionSubBones.Transforms);
		for (FSkinnedTriangleMeshData& TriMeshData : SkinnedTriangleMeshes)
		{
			Swap(TriMeshData.SkinnedPositions.OldPositions, TriMeshData.SkinnedPositions.Positions);
		}
	}

	void FClothComplexColliders::KinematicUpdate(const Softs::FSolverCollisionParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::FSolverReal Alpha)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothComplexColliders_KinematicUpdate);
		check(Particles.GetRangeId() == CollisionRangeId);
		// Update subbones
		const Softs::FSolverReal OneMinusAlpha = (Softs::FSolverReal)1.f - Alpha;
		for (uint32 Index = 0; Index < CollisionSubBones.Size(); ++Index)
		{
			const Softs::FSolverVec3 NewX = Alpha * CollisionSubBones.Transforms[Index].GetTranslation() + OneMinusAlpha * CollisionSubBones.OldTransforms[Index].GetTranslation();
			CollisionSubBones.V[Index] = (NewX - CollisionSubBones.X[Index]) / Dt;
			CollisionSubBones.X[Index] = NewX;
			const Softs::FSolverRotation3 NewR = Softs::FSolverRotation3::Slerp(CollisionSubBones.OldTransforms[Index].GetRotation(), CollisionSubBones.Transforms[Index].GetRotation(), Alpha);
			const Softs::FSolverRotation3 Delta = NewR * CollisionSubBones.R[Index].Inverse();
			const Softs::FSolverReal Angle = Delta.GetAngle();
			const Softs::FSolverVec3 Axis = Delta.GetRotationAxis();
			CollisionSubBones.W[Index] = Axis * Angle / Dt;
			CollisionSubBones.R[Index] = NewR;
		}

		// Update SkinnedLevelSets
		for (FSkinnedLevelSetData& Data : SkinnedLevelSets)
		{
			check(Particles.IsValidIndex(Data.Index));
			check(Particles.GetGeometry(Data.Index) == Data.SkinnedLevelSet);
			TWeightedLatticeImplicitObject<FLevelSet>& SkinnedLevelSet = Data.SkinnedLevelSet->GetObjectChecked<TWeightedLatticeImplicitObject<FLevelSet>>();

			const FTransform RootTransformInv = TRigidTransform<FReal, 3>(Particles.X(Data.Index), Particles.R(Data.Index)).Inverse();
			TArray<FTransform> SubBoneTransforms;
			SubBoneTransforms.Reserve(Data.MappedSubBones.Num());
			for (int32 SubBoneIndex : Data.MappedSubBones)
			{
				SubBoneTransforms.Emplace(TRigidTransform<FReal, 3>(CollisionSubBones.X[SubBoneIndex], CollisionSubBones.R[SubBoneIndex]) * RootTransformInv);
			}
			SkinnedLevelSet.DeformPoints(SubBoneTransforms);
			SkinnedLevelSet.UpdateSpatialHierarchy(MinLODSize);
		}

		// Update MLLevelSets
		for (FMLLevelSetData& Data : MLLevelSets)
		{
			check(Particles.IsValidIndex(Data.Index));
			check(Particles.GetGeometry(Data.Index) == Data.MLLevelSet);
			FMLLevelSet& MLLevelSet = Data.MLLevelSet->GetObjectChecked<FMLLevelSet>();

			const FTransform RootTransformInv = TRigidTransform<FReal, 3>(Particles.X(Data.Index), Particles.R(Data.Index)).Inverse();
			TArray<FTransform> SubBoneTransforms;
			SubBoneTransforms.Reserve(Data.MappedSubBones.Num());
			for (int32 SubBoneIndex : Data.MappedSubBones)
			{
				SubBoneTransforms.Emplace(TRigidTransform<FReal, 3>(CollisionSubBones.X[SubBoneIndex], CollisionSubBones.R[SubBoneIndex]) * RootTransformInv);
			}
			MLLevelSet.UpdateActiveBonesRelativeTransforms(SubBoneTransforms);
		}

		// Update SkinnedTriangleMeshes
		if (!bSkipSkinnedTriangleMeshKinematicUpdate)
		{
			for (FSkinnedTriangleMeshData& Data : SkinnedTriangleMeshes)
			{
				check(Particles.IsValidIndex(Data.Index));
				check(Particles.GetGeometry(Data.Index) == Data.SkinnedTriangleMesh);
				FSkinnedTriangleMesh& SkinnedTriangleMesh = Data.SkinnedTriangleMesh->GetObjectChecked<FSkinnedTriangleMesh>();
				TArrayView<Softs::FSolverVec3> InterpolatedPositions = SkinnedTriangleMesh.GetLocalPositions();
				const Softs::FSolverRigidTransform3 Frame(Particles.X(Data.Index), Particles.R(Data.Index));
				for (int32 VtxIndex = 0; VtxIndex < InterpolatedPositions.Num(); ++VtxIndex)
				{
					// Set interpolated positions in Solver space, not collider space.
					InterpolatedPositions[VtxIndex] = Frame.TransformPositionNoScale(Alpha * Data.SkinnedPositions.Positions[VtxIndex] + OneMinusAlpha * Data.SkinnedPositions.OldPositions[VtxIndex]);
				}
				SkinnedTriangleMesh.UpdateLocalBoundingBox();
				SkinnedTriangleMesh.UpdateSpatialHierarchy(MinLODSize);
			}
		}
	}

	void FClothComplexColliders::ApplyPreSimulationTransforms(const Softs::FSolverRigidTransform3& PreSimulationTransform, const Softs::FSolverVec3& DeltaLocalSpaceLocation,
		const TConstArrayView<Softs::FSolverRigidTransform3>& OldParticleTransforms, const TConstArrayView<Softs::FSolverRigidTransform3>& ParticleTransforms, const Softs::FSolverReal Dt)
	{
		for (uint32 Index = 0; Index < CollisionSubBones.Size(); ++Index)
		{
			// Update initial state for collisions
			CollisionSubBones.OldTransforms[Index] = CollisionSubBones.OldTransforms[Index] * PreSimulationTransform;
			CollisionSubBones.OldTransforms[Index].AddToTranslation(-DeltaLocalSpaceLocation);
			CollisionSubBones.X[Index] = CollisionSubBones.OldTransforms[Index].GetTranslation();
			CollisionSubBones.R[Index] = CollisionSubBones.OldTransforms[Index].GetRotation();
		}
		for (FSkinnedTriangleMeshData& Data : SkinnedTriangleMeshes)
		{
			// Calculate velocities here
			const Softs::FSolverRigidTransform3& OldFrame = OldParticleTransforms[Data.Index];
			const Softs::FSolverRigidTransform3& Frame = ParticleTransforms[Data.Index];
			for (int32 VtxIndex = 0; VtxIndex < Data.SkinnedPositions.Positions.Num(); ++VtxIndex)
			{
				const Softs::FSolverVec3 OldPosition = OldFrame.TransformPositionNoScale(Data.SkinnedPositions.OldPositions[VtxIndex]);
				const Softs::FSolverVec3 Position = Frame.TransformPositionNoScale(Data.SkinnedPositions.Positions[VtxIndex]);
				Data.SkinnedPositions.SolverSpaceVelocities[VtxIndex] = (Position - OldPosition) / Dt;
			}
		}
	}


	void FClothComplexColliders::ExtractComplexColliderBoneData(TMap<Softs::FParticleRangeIndex, Softs::FPBDComplexColliderBoneData>& BoneData) const
	{
		for (const FSkinnedLevelSetData& Data : SkinnedLevelSets)
		{
			Softs::FPBDComplexColliderBoneData NewBoneData;
			NewBoneData.MappedBoneIndices = TConstArrayView<int32>(Data.MappedSubBones);
			NewBoneData.X = TConstArrayView<Softs::FSolverVec3>(CollisionSubBones.X);
			NewBoneData.V = TConstArrayView<Softs::FSolverVec3>(CollisionSubBones.V);
			NewBoneData.R = TConstArrayView<Softs::FSolverRotation3>(CollisionSubBones.R);
			NewBoneData.W = TConstArrayView<Softs::FSolverVec3>(CollisionSubBones.W);

			BoneData.Emplace(Softs::FParticleRangeIndex(CollisionRangeId, Data.Index), MoveTemp(NewBoneData));
		}
	}
}  // End namespace Chaos
