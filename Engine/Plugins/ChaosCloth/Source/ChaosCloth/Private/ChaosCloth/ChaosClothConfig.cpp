// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothConfig.h"
#include "ClothConfig_Legacy.h"
#include "ChaosClothConfigCustomVersion.h"
#include "ChaosClothSharedConfigCustomVersion.h"
#include "ClothingSimulationInteractor.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosClothConfig)


UChaosClothConfig::UChaosClothConfig()
{}

UChaosClothConfig::~UChaosClothConfig()
{}

void UChaosClothConfig::MigrateFrom(const FClothConfig_Legacy& ClothConfig)
{
#if WITH_EDITORONLY_DATA
	// Legacy PhysX parameters never migrated to Chaos Cloth Config parameters:
	//  VerticalConstraintConfig.CompressionLimit
	//  VerticalConstraintConfig.StretchLimit
	//  HorizontalConstraintConfig.CompressionLimit
	//  HorizontalConstraintConfig.StretchLimit
	//  BendConstraintConfig.CompressionLimit
	//  BendConstraintConfig.StretchLimit
	//  ShearConstraintConfig.CompressionLimit
	//  ShearConstraintConfig.StretchLimit
	//  SelfCollisionStiffness
	//  SelfCollisionCullScale
	//  LinearDrag
	//  AngularDrag
	//  StiffnessFrequency

	const float VerticalStiffness =
		ClothConfig.VerticalConstraintConfig.Stiffness;
	const float HorizontalStiffness =
		ClothConfig.HorizontalConstraintConfig.Stiffness;
	EdgeStiffnessWeighted.Low = EdgeStiffnessWeighted.High = FMath::Clamp((VerticalStiffness + HorizontalStiffness) * 0.5f + 0.5f, 0.f, 1.f);

	BendingStiffnessWeighted.Low = BendingStiffnessWeighted.High = FMath::Clamp(
		ClothConfig.BendConstraintConfig.Stiffness + 0.5f, 0.f, 1.f);

	AreaStiffnessWeighted.Low = AreaStiffnessWeighted.High = FMath::Clamp(
		ClothConfig.ShearConstraintConfig.Stiffness + 0.5f, 0.f, 1.f);

	AnimDriveStiffness.Low = 0.f;
	AnimDriveStiffness.High = FMath::Clamp(ClothConfig.AnimDriveSpringStiffness + 0.5f, 0.f, 1.f);

	AnimDriveDamping.Low = 0.f;
	AnimDriveDamping.High = FMath::Clamp(ClothConfig.AnimDriveDamperStiffness, 0.f, 1.f);

	FrictionCoefficient = FMath::Clamp(ClothConfig.Friction, 0.f, 10.f);

	bUseBendingElements = false;
	bUseSelfCollisions = (ClothConfig.SelfCollisionRadius > 0.f && ClothConfig.SelfCollisionStiffness > 0.f);

	TetherStiffness.Low = FMath::Clamp(ClothConfig.TetherStiffness + 0.5f, 0.f, 1.f);
	TetherStiffness.High = 1.f;
	TetherScale.Low = TetherScale.High = FMath::Clamp(ClothConfig.TetherLimit, 0.01f, 10.f);
	ShapeTargetStiffness = 0.f;

	bUsePointBasedWindModel = (ClothConfig.WindMethod == EClothingWindMethod_Legacy::Legacy);
	Drag.Low = Drag.High = bUsePointBasedWindModel ? 0.07f  : ClothConfig.WindDragCoefficient;  // Only Accurate wind uses the WindDragCoefficient
	Lift.Low = Lift.High = bUsePointBasedWindModel ? 0.035f : ClothConfig.WindLiftCoefficient;  // Only Accurate wind uses the WindLiftCoefficient

	// Apply legacy damping calculations, see FClothingSimulationNv::ApplyClothConfig()
	constexpr float DampStiffnesssFreq = 10.0f;
	constexpr float PrecalcLog2 = 0.69314718f;
	const float Damping = (ClothConfig.Damping.X + ClothConfig.Damping.Y + ClothConfig.Damping.Z) / 3.f;
	const float DampStiffFreqRatio = DampStiffnesssFreq / ClothConfig.StiffnessFrequency;
	const float ExpDamp = DampStiffFreqRatio * FMath::Log2(1.0f - Damping);
	const float AdjustedDamping = 1.0f - FMath::Exp(ExpDamp * PrecalcLog2);
	DampingCoefficient = FMath::Clamp(AdjustedDamping, 0.f, 1.f);

	CollisionThickness = FMath::Clamp(ClothConfig.CollisionThickness, 0.f, 1000.f);
	SelfCollisionThickness = FMath::Clamp(ClothConfig.SelfCollisionRadius, 0.f, 1000.f);

	LinearVelocityScale = ClothConfig.LinearInertiaScale * 0.75f;
	const FVector AngularInertiaScale = ClothConfig.AngularInertiaScale * ClothConfig.CentrifugalInertiaScale * 0.75f;
	AngularVelocityScale = (AngularInertiaScale.X + AngularInertiaScale.Y + AngularInertiaScale.Z) / 3.f;

	bUseGravityOverride = ClothConfig.bUseGravityOverride;
	GravityScale = ClothConfig.GravityScale;
	Gravity = ClothConfig.GravityOverride;

	bUseLegacyBackstop = true;
#endif  // #if WITH_EDITORONLY_DATA
}

void UChaosClothConfig::MigrateFrom(const UClothSharedConfigCommon* ClothSharedConfig)
{
#if WITH_EDITORONLY_DATA
	if (const UChaosClothSharedSimConfig* const ChaosClothSharedSimConfig = Cast<UChaosClothSharedSimConfig>(ClothSharedConfig))
	{
		const int32 ChaosClothConfigCustomVersion = GetLinkerCustomVersion(FChaosClothConfigCustomVersion::GUID);

		if (ChaosClothConfigCustomVersion < FChaosClothConfigCustomVersion::AddDampingThicknessMigration)
		{
			if (ChaosClothSharedSimConfig->bUseDampingOverride_DEPRECATED)
			{
				DampingCoefficient = ChaosClothSharedSimConfig->Damping_DEPRECATED;
			}
			CollisionThickness = ChaosClothSharedSimConfig->CollisionThickness_DEPRECATED;
		}
		if (ChaosClothConfigCustomVersion < FChaosClothConfigCustomVersion::AddGravitySelfCollisionMigration)
		{
			SelfCollisionThickness = ChaosClothSharedSimConfig->SelfCollisionThickness_DEPRECATED;
			bUseGravityOverride = ChaosClothSharedSimConfig->bUseGravityOverride_DEPRECATED;
			GravityScale = ChaosClothSharedSimConfig->GravityScale_DEPRECATED;
			Gravity = ChaosClothSharedSimConfig->Gravity_DEPRECATED;
		}
		const int32 UE5ReleaseStreamObjectVersion = GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		if (UE5ReleaseStreamObjectVersion < FUE5ReleaseStreamObjectVersion::ChaosClothFictitiousAngularVelocitySubframeFix)
		{
			FictitiousAngularScale = FMath::Clamp(FictitiousAngularScale * (float)ChaosClothSharedSimConfig->SubdivisionCount, 0.f, 2.f);
		}
	}

#endif  // #if WITH_EDITORONLY_DATA
}

void UChaosClothConfig::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FChaosClothConfigCustomVersion::GUID);
	Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

void UChaosClothConfig::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const int32 ChaosClothConfigCustomVersion = GetLinkerCustomVersion(FChaosClothConfigCustomVersion::GUID);
	const int32 PhysicsObjectVersion = GetLinkerCustomVersion(FPhysicsObjectVersion::GUID);
	const int32 FortniteMainBranchObjectVersion = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	const int32 UE5ReleaseStreamObjectVersion = GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	const int32 FortniteReleaseBranchCustomObjectVersion = GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	if (ChaosClothConfigCustomVersion < FChaosClothConfigCustomVersion::UpdateDragDefault)
	{
		DragCoefficient_DEPRECATED = 0.07f;  // Reset to a more appropriate default for chaos cloth assets saved before this custom version that had too high drag
	}

	if (ChaosClothConfigCustomVersion < FChaosClothConfigCustomVersion::RemoveInternalConfigParameters)
	{
		MinPerParticleMass = 0.0001f;  // Override these values in case they might have been accidentally
	}

	if (ChaosClothConfigCustomVersion < FChaosClothConfigCustomVersion::AddLegacyBackstopParameter)
	{
		bUseLegacyBackstop = true;
	}

	if (PhysicsObjectVersion < FPhysicsObjectVersion::ChaosClothAddWeightedValue &&
		FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAddWeightedValue)
	{
		AnimDriveStiffness.Low = 0.f;
		AnimDriveStiffness.High = FMath::Clamp(FMath::Loge(AnimDriveSpringStiffness_DEPRECATED) / FMath::Loge(1.e3f) + 1.f, 0.f, 1.f);
	}

	if (TetherMode_DEPRECATED != EChaosClothTetherMode::MaxChaosClothTetherMode)
	{
		// Note: MaxChaosClothTetherMode is used here to detect that the TetherMode parameter isn't set to its default value and therefore needs to be migrated.
		bUseGeodesicDistance = (TetherMode_DEPRECATED != EChaosClothTetherMode::FastTetherFastLength);
		TetherMode_DEPRECATED = EChaosClothTetherMode::MaxChaosClothTetherMode;
	}

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAddfictitiousforces)
	{
		FictitiousAngularScale = 0.f;  // Maintain early behavior with no fictitious forces
	}

	if (PhysicsObjectVersion < FPhysicsObjectVersion::ChaosClothAddTetherStiffnessWeightMap &&
		FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAddTetherStiffnessWeightMap)
	{
		// Note: Unlike AnimDriveStiffness, Low is updated here, because there was no existing weight map before this version
		TetherStiffness.Low = FMath::Clamp(FMath::Loge(StrainLimitingStiffness_DEPRECATED) / FMath::Loge(1.e3f) + 1.f, 0.f, 1.f);
		TetherStiffness.High = 1.f;
	}

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAddTetherScaleAndDragLiftWeightMaps)
	{
		TetherScale.Low = TetherScale.High = LimitScale_DEPRECATED;
		Drag.Low = Drag.High = DragCoefficient_DEPRECATED;
		Lift.Low = Lift.High = LiftCoefficient_DEPRECATED;
	}

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAddMaterialWeightMaps)
	{
		EdgeStiffnessWeighted.Low = EdgeStiffnessWeighted.High = EdgeStiffness_DEPRECATED;
		BendingStiffnessWeighted.Low = BendingStiffnessWeighted.High = BendingStiffness_DEPRECATED;
		AreaStiffnessWeighted.Low = AreaStiffnessWeighted.High = AreaStiffness_DEPRECATED;
	}

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothFasterDamping &&
		UE5ReleaseStreamObjectVersion < FUE5ReleaseStreamObjectVersion::ChaosClothFasterDamping)
	{
		// Note: The previous damping has been renamed LocalDamping to make space for a faster but more primitive global point damping.
		LocalDampingCoefficient = DampingCoefficient;
		DampingCoefficient = 0.f;
	}

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothAllowZeroBucklingStiffness)
	{
		// Before this version, there was a bug where BucklingStiffness.Low == BucklingStiffness.High == 0 would result in the parameter being set to default (1.f).
		if (BucklingStiffnessWeighted.Low <= 0 && BucklingStiffnessWeighted.High <= 0)
		{
			BucklingStiffnessWeighted.Low = BucklingStiffnessWeighted.High = 1.f;
		}
	}

	if (FortniteReleaseBranchCustomObjectVersion < FFortniteReleaseBranchCustomObjectVersion::ChaosClothAddNewFictitiousForcesModes)
	{
		FictitiousForcesModel = EChaosClothingSimulationSolverFictitiousForcesModel::Legacy;
	}
#endif  // #if WITH_EDITORONLY_DATA
}

float UChaosClothConfig::GetMassValue() const
{
	switch (MassMode)
	{
	default:
	case EClothMassMode::Density: return Density;
	case EClothMassMode::TotalMass: return TotalMass;
	case EClothMassMode::UniformMass: return UniformMass;
	}
}

UChaosClothSharedSimConfig::UChaosClothSharedSimConfig()
{}

UChaosClothSharedSimConfig::~UChaosClothSharedSimConfig()
{}

void UChaosClothSharedSimConfig::MigrateFrom(const FClothConfig_Legacy& ClothConfig)
{
#if WITH_EDITORONLY_DATA
	IterationCount = FMath::Clamp(int32(ClothConfig.SolverFrequency / 120.f), 1, 100);
	MaxIterationCount = FMath::Clamp(int32(ClothConfig.SolverFrequency / 30.f), 1, 100);

	bUseDampingOverride_DEPRECATED = false;  // Damping is migrated to per cloth configs
#endif
}

void UChaosClothSharedSimConfig::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FChaosClothSharedConfigCustomVersion::GUID);
}

void UChaosClothSharedSimConfig::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const int32 ChaosClothSharedConfigCustomVersion = GetLinkerCustomVersion(FChaosClothSharedConfigCustomVersion::GUID);

	if (ChaosClothSharedConfigCustomVersion < FChaosClothSharedConfigCustomVersion::AddGravityOverride)
	{
		bUseGravityOverride_DEPRECATED = true;  // Default gravity override would otherwise disable the currently set gravity on older versions
	}
#endif
}

#if WITH_EDITOR
void UChaosClothSharedSimConfig::PostEditChangeChainProperty(FPropertyChangedChainEvent& ChainEvent)
{
	Super::PostEditChangeChainProperty(ChainEvent);

	if (ChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothSharedSimConfig, MaxIterationCount))
	{
		IterationCount = FMath::Min(IterationCount, MaxIterationCount);
	}
	else if (ChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UChaosClothSharedSimConfig, IterationCount))
	{
		MaxIterationCount = FMath::Max(MaxIterationCount, IterationCount);
	}
}
#endif  // #if WITH_EDITOR

