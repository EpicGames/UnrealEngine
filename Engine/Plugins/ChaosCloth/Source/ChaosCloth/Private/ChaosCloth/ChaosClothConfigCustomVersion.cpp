// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothConfigCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FChaosClothConfigCustomVersion::GUID(0x76A52329, 0x092345B5, 0x98AED841, 0xCF2F6AD8);
// Register the custom version with core
FCustomVersionRegistration GRegisterChaosClothConfigCustomVersion(FChaosClothConfigCustomVersion::GUID, FChaosClothConfigCustomVersion::LatestVersion, TEXT("ChaosClothConfigVer"));
