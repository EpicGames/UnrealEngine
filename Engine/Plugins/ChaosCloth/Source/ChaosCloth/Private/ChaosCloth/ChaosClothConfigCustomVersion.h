// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for cloth config
struct FChaosClothConfigCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Update drag default to better preserve legacy behavior
		UpdateDragDefault = 1,
		// Added damping and collision thickness per cloth
		AddDampingThicknessMigration = 2,
		// Added gravity and self collision thickness per cloth
		AddGravitySelfCollisionMigration = 3,
		// Remove internal config parameters from UI
		RemoveInternalConfigParameters = 4,
		// Add a parameter to fix the backstop behavior without breaking PhysX assets backward compatibility
		AddLegacyBackstopParameter = 5,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FChaosClothConfigCustomVersion() {}
};
