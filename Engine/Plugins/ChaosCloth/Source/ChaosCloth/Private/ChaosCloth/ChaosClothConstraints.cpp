// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothConstraints.h"
#include "ChaosCloth/ChaosClothingAccessoryMeshData.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosWeightMapTarget.h"
#include "ChaosCloth/ChaosClothingPatternData.h"
#include "ChaosCloth/ChaosClothComplexColliders.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/XPBDStretchBiasElementConstraints.h"
#include "Chaos/XPBDAnisotropicSpringConstraints.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/XPBDBendingConstraints.h"
#include "Chaos/XPBDAnisotropicBendingConstraints.h"
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/XPBDAxialSpringConstraints.h"
#include "Chaos/PBDVolumeConstraint.h"
#include "Chaos/XPBDLongRangeConstraints.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDAnimDriveConstraint.h"
#include "Chaos/PBDShapeConstraints.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDSelfCollisionSphereConstraints.h"
#include "Chaos/PBDKinematicTriangleMeshCollisions.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/PBDTriangleMeshIntersections.h"
#include "Chaos/PBDUnilateralTetConstraints.h"
#include "Chaos/PBDEvolution.h"
#include "Chaos/PBDSoftBodyCollisionConstraint.h"
#include "Chaos/SoftsExternalForces.h"
#include "Chaos/SoftsEvolution.h"
#include "Chaos/VelocityField.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/Deformable/GaussSeidelMainConstraint.h"
#include "Chaos/Deformable/GaussSeidelCorotatedCodimensionalConstraints.h"
#include "Chaos/XPBDEmbeddedSpringConstraints.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "Utils/ClothingMeshUtils.h"
#include "HAL/IConsoleManager.h"
#include "BuoyancyField.h"

namespace Chaos {

namespace ClothingSimulationClothConsoleVariables
{
// These are defined in ChaosClothingSimulationCloth.cpp
extern TAutoConsoleVariable<bool> CVarLegacyDisablesAccurateWind;
extern TAutoConsoleVariable<float> CVarGravityMultiplier;
}

namespace ClothConstraints::Private
{
	static bool bEnableBuoyancy = false;  // #todo(dmp): once we decide what we want to do for enabling buoyancy by default or controlling it some way, we can enable it for shipping
	static bool bSwapBackstopAnimDriveApply = false;  // TODO: Move this to a property
}

#if !UE_BUILD_SHIPPING
	
	static FAutoConsoleVariableRef CVarClothEnableBuoyancy(TEXT("p.Chaos.Cloth.EnableBuoyancy"), ClothConstraints::Private::bEnableBuoyancy, TEXT("Query water bodies for drag and buoyancy effects with cloth [def: false]"));

	static FAutoConsoleVariableRef CVarClothSwapBackstopAnimDriveApply(TEXT("p.Chaos.Cloth.SwapBackstopAnimDriveApply"), ClothConstraints::Private::bSwapBackstopAnimDriveApply, TEXT("Swap the order of the Backstop and AnimDrive Constraints so that the Backstop is processed last [def: false]"));
#endif

class FClothConstraints::FRuleCreator
{
public:
	explicit FRuleCreator(FClothConstraints* InConstraints)
		: Constraints(InConstraints)
	{
		check(Constraints);
		Constraints->Evolution->AllocatePreSubstepParallelInitRange(Constraints->ParticleRangeId, Constraints->NumPreSubstepInits);
		Constraints->Evolution->AllocatePBDExternalForceRulesRange(Constraints->ParticleRangeId, Constraints->NumExternalForceRules);
		Constraints->Evolution->AllocatePostInitialGuessParallelInitRange(Constraints->ParticleRangeId, Constraints->NumConstraintInits);
		Constraints->Evolution->AllocatePreSubstepConstraintRulesRange(Constraints->ParticleRangeId, Constraints->NumPreSubstepConstraintRules);
		Constraints->Evolution->AllocatePerIterationPBDConstraintRulesRange(Constraints->ParticleRangeId, Constraints->NumConstraintRules);
		Constraints->Evolution->AllocatePerIterationCollisionPBDConstraintRulesRange(Constraints->ParticleRangeId, Constraints->NumCollisionConstraintRules);
		Constraints->Evolution->AllocatePerIterationPostCollisionsPBDConstraintRulesRange(Constraints->ParticleRangeId, Constraints->NumPostCollisionConstraintRules);
		Constraints->Evolution->AllocateUpdateLinearSystemRulesRange(Constraints->ParticleRangeId, Constraints->NumUpdateLinearSystemRules);
		Constraints->Evolution->AllocateUpdateLinearSystemCollisionsRulesRange(Constraints->ParticleRangeId, Constraints->NumUpdateLinearSystemCollisionsRules);
		Constraints->Evolution->AllocatePostSubstepConstraintRulesRange(Constraints->ParticleRangeId, Constraints->NumPostprocessingConstraintRules);

		PreSubstepParallelInits = Constraints->Evolution->GetPreSubstepParallelInitRange(Constraints->ParticleRangeId);
		ExternalForceRules = Constraints->Evolution->GetPBDExternalForceRulesRange(Constraints->ParticleRangeId);
		PostInitialGuessParallelInits = Constraints->Evolution->GetPostInitialGuessParallelInitRange(Constraints->ParticleRangeId);
		PreSubstepConstraintRules = Constraints->Evolution->GetPreSubstepConstraintRulesRange(Constraints->ParticleRangeId);
		PerIterationConstraintRules = Constraints->Evolution->GetPerIterationPBDConstraintRulesRange(Constraints->ParticleRangeId);
		PerIterationCollisionConstraintRules = Constraints->Evolution->GetPerIterationCollisionPBDConstraintRulesRange(Constraints->ParticleRangeId);
		PerIterationPostCollisionsConstraintRules = Constraints->Evolution->GetPerIterationPostCollisionsPBDConstraintRulesRange(Constraints->ParticleRangeId);
		UpdateLinearSystemRules = Constraints->Evolution->GetUpdateLinearSystemRulesRange(Constraints->ParticleRangeId);
		UpdateLinearSystemCollisionsRules = Constraints->Evolution->GetUpdateLinearSystemCollisionsRulesRange(Constraints->ParticleRangeId);
		PostSubstepConstraintRules = Constraints->Evolution->GetPostSubstepConstraintRulesRange(Constraints->ParticleRangeId);
	}

	~FRuleCreator()
	{
		if (Constraints)
		{
			check(PreSubstepInitsIndex == Constraints->NumPreSubstepInits);
			check(ExternalForceRulesIndex == Constraints->NumExternalForceRules);
			check(PostInitialGuessInitsIndex == Constraints->NumConstraintInits);
			check(PreSubstepConstraintRulesIndex == Constraints->NumPreSubstepConstraintRules);
			check(ConstraintRuleIndex == Constraints->NumConstraintRules);
			check(CollisionConstraintRulesIndex == Constraints->NumCollisionConstraintRules);
			check(PostCollisionConstraintRulesIndex == Constraints->NumPostCollisionConstraintRules);
			check(UpdateLinearSystemRulesIndex == Constraints->NumUpdateLinearSystemRules);
			check(UpdateLinearSystemCollisionsRulesIndex == Constraints->NumUpdateLinearSystemCollisionsRules);
			check(PostSubstepConstraintRulesIndex == Constraints->NumPostprocessingConstraintRules);
		}
	}

	void PreSubstepParallelInitRule(Softs::FEvolution::ParallelInitFunc&& Rule)
	{
		PreSubstepParallelInits[PreSubstepInitsIndex++] = MoveTemp(Rule);
	}

	void AddExternalForceRule(Softs::FEvolution::PBDConstraintRuleFunc&& Rule)
	{
		ExternalForceRules[ExternalForceRulesIndex++] = MoveTemp(Rule);
	}

	void AddPostInitialGuessParallelInitRule(Softs::FEvolution::ParallelInitFunc&& Rule)
	{
		PostInitialGuessParallelInits[PostInitialGuessInitsIndex++] = MoveTemp(Rule);
	}

	void AddPreSubstepConstraintRule(Softs::FEvolution::ConstraintRuleFunc&& Rule)
	{
		PreSubstepConstraintRules[PreSubstepConstraintRulesIndex++] = MoveTemp(Rule);
	}

	void AddPerIterationPBDConstraintRule(Softs::FEvolution::PBDConstraintRuleFunc&& Rule)
	{
		PerIterationConstraintRules[ConstraintRuleIndex++] = MoveTemp(Rule);
	}

	void AddPerIterationCollisionPBDConstraintRule(Softs::FEvolution::PBDCollisionConstraintRuleFunc&& Rule)
	{
		PerIterationCollisionConstraintRules[CollisionConstraintRulesIndex++] = MoveTemp(Rule);
	}

	void AddPerIterationPostCollisionsPBDConstraintRule(Softs::FEvolution::PBDConstraintRuleFunc&& Rule)
	{
		PerIterationPostCollisionsConstraintRules[PostCollisionConstraintRulesIndex++] = MoveTemp(Rule);
	}

	void AddUpdateLinearSystemRule(Softs::FEvolution::UpdateLinearSystemFunc&& Rule)
	{
		UpdateLinearSystemRules[UpdateLinearSystemRulesIndex++] = MoveTemp(Rule);
	}

	void AddUpdateLinearSystemCollisionsRule(Softs::FEvolution::UpdateLinearSystemCollisionsFunc&& Rule)
	{
		UpdateLinearSystemCollisionsRules[UpdateLinearSystemCollisionsRulesIndex++] = MoveTemp(Rule);
	}

	void AddPostSubstepConstraintRule(Softs::FEvolution::ConstraintRuleFunc&& Rule)
	{
		PostSubstepConstraintRules[PostSubstepConstraintRulesIndex++] = MoveTemp(Rule);
	}

	template<typename ConstraintType>
	void AddExternalForceRule_Apply(ConstraintType* const Constraint)
	{
		AddExternalForceRule(
			[Constraint](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt)
		{
			Constraint->Apply(Particles, Dt);
		});
	}

	template<typename ConstraintType>
	void AddUpdateLinearSystemRule_UpdateLinearSystem(ConstraintType* const Constraint)
	{
		AddUpdateLinearSystemRule([Constraint](const Softs::FSolverParticlesRange& Particles, const FSolverReal Dt, Softs::FEvolutionLinearSystem& LinearSystem)
		{
			Constraint->UpdateLinearSystem(Particles, Dt, LinearSystem);
		});
	}

	template<bool bInitParticles = false, typename ConstraintType>
	void AddXPBDParallelInitRule(ConstraintType* const Constraint)
	{
		if constexpr (bInitParticles)
		{
			AddPostInitialGuessParallelInitRule(
				[Constraint, Evolution = Constraints->Evolution](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
			{
				Constraint->ApplyProperties(Dt, Evolution->GetIterations());
				if ((SolverMode & Softs::ESolverMode::PBD) != Softs::ESolverMode::None)\
				{
					Constraint->Init(Particles);
				}
			});
		}
		else
		{
			AddPostInitialGuessParallelInitRule(
				[Constraint, Evolution = Constraints->Evolution](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
			{
				Constraint->ApplyProperties(Dt, Evolution->GetIterations());
				if ((SolverMode & Softs::ESolverMode::PBD) != Softs::ESolverMode::None)\
				{
					Constraint->Init();
				}
			});
		}
	}

	template<typename ConstraintType>
	void AddPBDParallelInitRule(ConstraintType* const Constraint)
	{
		AddPostInitialGuessParallelInitRule(
			[Constraint, Evolution = Constraints->Evolution](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if ((SolverMode & Softs::ESolverMode::PBD) != Softs::ESolverMode::None)
			{
				Constraint->ApplyProperties(Dt, Evolution->GetIterations()); 
			}
		});
	}

	template<bool bPostCollisions, typename ConstraintType>
	void AddPerIterationPBDConstraintRule_Apply(ConstraintType* const Constraint)
	{
		if constexpr (bPostCollisions)
		{
			AddPerIterationPostCollisionsPBDConstraintRule(
				[Constraint](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt)
			{
				Constraint->Apply(Particles, Dt);
			});
		}
		else
		{
			AddPerIterationPBDConstraintRule(
				[Constraint](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt)
			{
				Constraint->Apply(Particles, Dt);
			});
		}
	}

	template<typename ConstraintType>
	void AddPerIterationCollisionPBDConstraintRule_Apply(ConstraintType* const Constraint)
	{
		AddPerIterationCollisionPBDConstraintRule(
			[Constraint](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const TArray<Softs::FSolverCollisionParticlesRange>& CollisionParticles)
		{
			Constraint->Apply(Particles, Dt, CollisionParticles);
		}
		);
	}

	template<typename ConstraintType>
	void AddUpdateLinearSystemCollisionsRule_UpdateLinearSystem(ConstraintType* const Constraint)
	{
		AddUpdateLinearSystemCollisionsRule(
			[Constraint](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const TArray<Softs::FSolverCollisionParticlesRange>& CollisionParticles, Softs::FEvolutionLinearSystem& LinearSystem)
		{
			Constraint->UpdateLinearSystem(Particles, Dt, CollisionParticles, LinearSystem);
		}
		);
	}

private:

	FClothConstraints* const Constraints = nullptr;
	TArrayView<Softs::FEvolution::ParallelInitFunc> PreSubstepParallelInits;
	TArrayView<Softs::FEvolution::PBDConstraintRuleFunc> ExternalForceRules;
	TArrayView<Softs::FEvolution::ParallelInitFunc> PostInitialGuessParallelInits;
	TArrayView<Softs::FEvolution::ConstraintRuleFunc> PreSubstepConstraintRules;
	TArrayView<Softs::FEvolution::PBDConstraintRuleFunc> PerIterationConstraintRules;
	TArrayView<Softs::FEvolution::PBDCollisionConstraintRuleFunc> PerIterationCollisionConstraintRules;
	TArrayView<Softs::FEvolution::PBDConstraintRuleFunc> PerIterationPostCollisionsConstraintRules;
	TArrayView<Softs::FEvolution::UpdateLinearSystemFunc> UpdateLinearSystemRules;
	TArrayView<Softs::FEvolution::UpdateLinearSystemCollisionsFunc> UpdateLinearSystemCollisionsRules;
	TArrayView<Softs::FEvolution::ConstraintRuleFunc> PostSubstepConstraintRules;

	int32 PreSubstepInitsIndex = 0;
	int32 ExternalForceRulesIndex = 0;
	int32 PostInitialGuessInitsIndex = 0;
	int32 PreSubstepConstraintRulesIndex = 0;
	int32 ConstraintRuleIndex = 0;
	int32 CollisionConstraintRulesIndex = 0;
	int32 PostCollisionConstraintRulesIndex = 0;
	int32 UpdateLinearSystemRulesIndex = 0;
	int32 UpdateLinearSystemCollisionsRulesIndex = 0;
	int32 PostSubstepConstraintRulesIndex = 0;
};

FClothConstraints::FClothConstraints()
	: Evolution(nullptr), PBDEvolution(nullptr)
	, AnimationPositions(nullptr)
	, AnimationNormals(nullptr)
	, AnimationVelocities(nullptr)
	, ParticleOffset(0)
	, ParticleRangeId(0)
	, NumParticles(0)
	, NumConstraintInits(0)
	, NumConstraintRules(0)
	, NumPostCollisionConstraintRules(0)
	, NumPostprocessingConstraintRules(0)

	, PerSolverField(nullptr)
	, Normals(nullptr)
	, LastSubframeCollisionTransformsCCD(nullptr)
	, CollisionParticleCollided(nullptr)
	, CollisionContacts(nullptr)
	, CollisionNormals(nullptr)
	, CollisionPhis(nullptr)
	, NumPreSubstepInits(0)
	, NumExternalForceRules(0)
	, NumPreSubstepConstraintRules(0)
	, NumCollisionConstraintRules(0)
	, NumUpdateLinearSystemRules(0)
	, NumUpdateLinearSystemCollisionsRules(0)

	, ConstraintInitOffset(INDEX_NONE)
	, ConstraintRuleOffset(INDEX_NONE)
	, PostCollisionConstraintRuleOffset(INDEX_NONE)
	, PostprocessingConstraintRuleOffset(INDEX_NONE)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FClothConstraints::~FClothConstraints()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FClothConstraints::Initialize(
	Softs::FEvolution* InEvolution,
	FPerSolverFieldSystem* InPerSolverField,
	const TArray<Softs::FSolverVec3>& InInterpolatedAnimationPositions,
	const TArray<Softs::FSolverVec3>& InInterpolatedAnimationNormals,
	const TArray<Softs::FSolverVec3>& InAnimationVelocities,
	const TArray<Softs::FSolverVec3>& InNormals,
	const TArray<Softs::FSolverRigidTransform3>& InLastSubframeCollisionTransformsCCD,
	TArray<bool>& InCollisionParticleCollided,
	TArray<Softs::FSolverVec3>& InCollisionContacts,
	TArray<Softs::FSolverVec3>& InCollisionNormals,
	TArray<Softs::FSolverReal>& InCollisionPhis,
	int32 InParticleRangeId)
{
	PBDEvolution = nullptr;
	Evolution = InEvolution;
	PerSolverField = InPerSolverField;
	AnimationPositions = &InInterpolatedAnimationPositions;
	AnimationNormals = &InInterpolatedAnimationNormals;
	AnimationVelocities = &InAnimationVelocities;
	Normals = &InNormals;
	LastSubframeCollisionTransformsCCD = &InLastSubframeCollisionTransformsCCD;
	CollisionParticleCollided = &InCollisionParticleCollided;
	CollisionContacts = &InCollisionContacts;
	CollisionNormals = &InCollisionNormals;
	CollisionPhis = &InCollisionPhis;
	ParticleOffset = 0;
	ParticleRangeId = InParticleRangeId;
	NumParticles = Evolution->GetSoftBodyParticleNum(ParticleRangeId);
}

void FClothConstraints::Initialize(
	Softs::FPBDEvolution* InEvolution,
	const TArray<Softs::FSolverVec3>& InInterpolatedAnimationPositions,
	const TArray<Softs::FSolverVec3>& /*InOldAnimationPositions*/, // deprecated
	const TArray<Softs::FSolverVec3>& InInterpolatedAnimationNormals,
	const TArray<Softs::FSolverVec3>& InAnimationVelocities,
	int32 InParticleOffset,
	int32 InNumParticles)
{
	Evolution = nullptr;
	PBDEvolution = InEvolution;
	AnimationPositions = &InInterpolatedAnimationPositions;
	AnimationNormals = &InInterpolatedAnimationNormals;
	AnimationVelocities = &InAnimationVelocities;
	ParticleOffset = InParticleOffset;
	ParticleRangeId = InParticleOffset;
	NumParticles = InNumParticles;
}

void FClothConstraints::Enable(bool bEnable)
{
	if (ConstraintInitOffset != INDEX_NONE)
	{
		check(PBDEvolution);
		PBDEvolution->ActivateConstraintInitRange(ConstraintInitOffset, bEnable);
	}
	if (ConstraintRuleOffset != INDEX_NONE)
	{
		check(PBDEvolution);
		PBDEvolution->ActivateConstraintRuleRange(ConstraintRuleOffset, bEnable);
	}
	if (PostCollisionConstraintRuleOffset != INDEX_NONE)
	{
		check(PBDEvolution);
		PBDEvolution->ActivatePostCollisionConstraintRuleRange(PostCollisionConstraintRuleOffset, bEnable);
	}
	if (PostprocessingConstraintRuleOffset != INDEX_NONE)
	{
		check(PBDEvolution);
		PBDEvolution->ActivateConstraintPostprocessingsRange(PostprocessingConstraintRuleOffset, bEnable);
	}
}

void FClothConstraints::OnCollisionRangeRemoved(int32 CollisionRangeId)
{
	if (CollisionConstraint)
	{
		CollisionConstraint->OnCollisionRangeRemoved(CollisionRangeId);
	}
	if (SkinnedTriangleCollisionsConstraint)
	{
		SkinnedTriangleCollisionsConstraint->OnCollisionRangeRemoved(CollisionRangeId);
	}
}

void FClothConstraints::AddRules(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const FTriangleMesh& TriangleMesh,
	const FClothingPatternData* PatternData,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TMap<FString, const TSet<int32>*>& VertexSets,
	const TMap<FString, const TSet<int32>*>& FaceSets,
	const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
	const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& Tethers,
	Softs::FSolverReal MeshScale, bool bEnabled,
	const TArray<const FClothComplexColliders*>& ComplexColliders,
	const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection,
	const TMap<FName, FClothingAccessoryMeshData>* AccessoryMeshes)
{
	// Self collisions
	CreateSelfCollisionConstraints(ConfigProperties, WeightMaps, VertexSets, FaceSets, FaceIntMaps, TriangleMesh);

	// Edge constraints
	CreateStretchConstraints(ConfigProperties, WeightMaps, TriangleMesh, PatternData, ManagedArrayCollection, MeshScale);

	// Bending constraints
	CreateBendingConstraints(ConfigProperties, WeightMaps, TriangleMesh, PatternData);

	// Extreme deformation constraints
	CreateExtremeDeformationConstraints(ConfigProperties, WeightMaps, VertexSets, TriangleMesh, PatternData);

	// Area constraints
	CreateAreaConstraints(ConfigProperties, WeightMaps, TriangleMesh, PatternData);

	// Long range constraints
	CreateLongRangeConstraints(ConfigProperties, WeightMaps, Tethers, MeshScale);

	// Max distances
	CreateMaxDistanceConstraints(ConfigProperties, WeightMaps, MeshScale);

	// Backstop Constraints
	CreateBackstopConstraints(ConfigProperties, WeightMaps, MeshScale, AccessoryMeshes);

	// Animation Drive Constraints
	CreateAnimDriveConstraints(ConfigProperties, WeightMaps);

	if (Evolution)
	{
		// External Forces
		CreateExternalForces(ConfigProperties, WeightMaps);

		if (!ClothConstraints::Private::bEnableBuoyancy)
		{
			// Velocity Field
			CreateVelocityAndPressureField(ConfigProperties, WeightMaps, TriangleMesh);
		}
		else
		{		
			// Buoyancy Field
			CreateVelocityPressureAndBuoyancyField(ConfigProperties, WeightMaps, TriangleMesh);
		}

		// PerSolverField
		if (PerSolverField)
		{
			++NumExternalForceRules;
		}

		// Body collisions
		CreateCollisionConstraint(ConfigProperties, MeshScale, WeightMaps, ComplexColliders);

		// ClothCloth Springs
		CreateClothClothConstraints(ConfigProperties, ManagedArrayCollection);
	}

	// Commit rules to solver
	if (Evolution)
	{
		CreateForceBasedRules();
	}
	else
	{
		CreatePBDRules();
	}

	if (PBDEvolution)
	{
		// Enable or disable constraints as requested
		Enable(bEnabled);
	}
}

void FClothConstraints::CreateSelfCollisionConstraints(const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TMap<FString, const TSet<int32>*>& VertexSets,
	const TMap<FString, const TSet<int32>*>& FaceSets,
	const TMap<FString, TConstArrayView<int32>>& FaceIntMaps, const FTriangleMesh& TriangleMesh)
{
	if (Softs::FPBDCollisionSpringConstraints::IsEnabled(ConfigProperties))
	{
		SelfCollisionInit = MakeShared<Softs::FPBDTriangleMeshCollisions>(
			ParticleOffset,
			NumParticles,
			FaceSets,
			TriangleMesh,
			ConfigProperties);
		++NumConstraintInits;

		SelfCollisionConstraints = MakeShared<Softs::FPBDCollisionSpringConstraints>(
			ParticleOffset,
			NumParticles,
			TriangleMesh,
			AnimationPositions,
			WeightMaps,
			FaceIntMaps,
			ConfigProperties);
		++NumPostCollisionConstraintRules;

		SelfIntersectionConstraints = MakeShared<Softs::FPBDTriangleMeshIntersections>(
			ParticleOffset,
			NumParticles,
			TriangleMesh);
		++NumPostprocessingConstraintRules;

		if (Evolution)
		{
			++NumPreSubstepConstraintRules; // Contour minimization
			++NumUpdateLinearSystemRules;
		}
		else
		{
			++NumConstraintInits; // Contour minimization
		}
	}
	else if (Softs::FPBDSelfCollisionSphereConstraints::IsEnabled(ConfigProperties))
	{
		SelfCollisionSphereConstraints = MakeShared<Softs::FPBDSelfCollisionSphereConstraints>(
			ParticleOffset,
			NumParticles,
			VertexSets,
			ConfigProperties);
		++NumConstraintInits;
		++NumPostCollisionConstraintRules;
	}
}

void FClothConstraints::CreateStretchConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FTriangleMesh& TriangleMesh,
	const FClothingPatternData* PatternData,
	const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection,
	Softs::FSolverReal MeshScale)
{
	if (PatternData && PatternData->PatternPositions.Num() && Softs::FXPBDStretchBiasElementConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			XStretchBiasConstraints = MakeShared<Softs::FXPBDStretchBiasElementConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		else
		{
			XStretchBiasConstraints = MakeShared<Softs::FXPBDStretchBiasElementConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
	else if (Softs::FXPBDEdgeSpringConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			XEdgeConstraints = MakeShared<Softs::FXPBDEdgeSpringConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties);
			++NumUpdateLinearSystemRules;
		}
		else
		{
			XEdgeConstraints = MakeShared<Softs::FXPBDEdgeSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties);
		}

		++NumConstraintInits;
		++NumConstraintRules;
	}
	else if (Softs::FPBDEdgeSpringConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			if (PatternData && PatternData->PatternPositions.Num())
			{
				EdgeConstraints = MakeShared<Softs::FPBDEdgeSpringConstraints>(
					Evolution->GetSoftBodyParticles(ParticleRangeId),
					TriangleMesh,
					PatternData->WeldedFaceVertexPatternPositions,
					WeightMaps,
					ConfigProperties,
					/*bTrimKinematicConstraints =*/ true);

			}
			else
			{
				EdgeConstraints = MakeShared<Softs::FPBDEdgeSpringConstraints>(
					Evolution->GetSoftBodyParticles(ParticleRangeId),
					TriangleMesh.GetSurfaceElements(),
					WeightMaps,
					ConfigProperties,
					/*bTrimKinematicConstraints =*/ true);
			}
		}
		else
		{
			EdgeConstraints = MakeShared<Softs::FPBDEdgeSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
	if (PatternData && PatternData->PatternPositions.Num() && Softs::FXPBDAnisotropicSpringConstraints::IsEnabled(ConfigProperties))
	{
		TArray<FVector3f> PreResizedPositions;
		if (ManagedArrayCollection)
		{
			if (const TManagedArray<FVector3f>* PreResizedAttr =
				ManagedArrayCollection->FindAttributeTyped<FVector3f>(
					TEXT("PreResizedSimPosition3D"),
					TEXT("SimVertices3D")))
			{
				PreResizedPositions = PreResizedAttr->GetConstArray();
			}
		}
		if (Evolution)
		{
			XAnisoSpringConstraints = MakeShared<Softs::FXPBDAnisotropicSpringConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties,
				PreResizedPositions,
				MeshScale);
			++NumUpdateLinearSystemRules;
		}
		else
		{
			XAnisoSpringConstraints = MakeShared<Softs::FXPBDAnisotropicSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties,
				PreResizedPositions,
				MeshScale);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
		++NumConstraintRules; // 2X because edge and axial spring applies are split
	}
}

void FClothConstraints::CreateBendingConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FTriangleMesh& TriangleMesh,
	const FClothingPatternData* PatternData)
{
	if (PatternData && PatternData->PatternPositions.Num() && Softs::FXPBDAnisotropicBendingConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			XAnisoBendingElementConstraints = MakeShared<Softs::FXPBDAnisotropicBendingConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties);
		}
		else
		{
			XAnisoBendingElementConstraints = MakeShared<Softs::FXPBDAnisotropicBendingConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset, NumParticles,
				TriangleMesh,
				PatternData->WeldedFaceVertexPatternPositions,
				WeightMaps,
				ConfigProperties);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
	else if (Softs::FXPBDBendingConstraints::IsEnabled(ConfigProperties))
	{
		TArray<Chaos::TVec4<int32>> BendingElements = TriangleMesh.GetUniqueAdjacentElements();

		if (Evolution)
		{
			XBendingElementConstraints = MakeShared<Softs::FXPBDBendingConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				MoveTemp(BendingElements),
				WeightMaps,
				ConfigProperties);
		}
		else
		{
			XBendingElementConstraints = MakeShared<Softs::FXPBDBendingConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset, NumParticles,
				MoveTemp(BendingElements),
				WeightMaps,
				ConfigProperties);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
	else if (Softs::FPBDBendingConstraints::IsEnabled(ConfigProperties))
	{
		TArray<Chaos::TVec4<int32>> BendingElements = TriangleMesh.GetUniqueAdjacentElements();

		if (Evolution)
		{
			BendingElementConstraints = MakeShared<Softs::FPBDBendingConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				MoveTemp(BendingElements),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		else
		{
			BendingElementConstraints = MakeShared<Softs::FPBDBendingConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset, NumParticles,
				MoveTemp(BendingElements),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
	else if (Softs::FXPBDBendingSpringConstraints::IsEnabled(ConfigProperties))
	{
		const TArray<Chaos::TVec2<int32>> CrossEdges = TriangleMesh.GetUniqueAdjacentPoints();

		if (Evolution)
		{
			XBendingConstraints = MakeShared<Softs::FXPBDBendingSpringConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				CrossEdges,
				WeightMaps,
				ConfigProperties);
			++NumUpdateLinearSystemRules;
		}
		else
		{
			XBendingConstraints = MakeShared<Softs::FXPBDBendingSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset, NumParticles,
				CrossEdges,
				WeightMaps,
				ConfigProperties);
		}

		++NumConstraintInits;
		++NumConstraintRules;
	}
	else if (Softs::FPBDBendingSpringConstraints::IsEnabled(ConfigProperties))
	{
		const TArray<Chaos::TVec2<int32>> CrossEdges = TriangleMesh.GetUniqueAdjacentPoints();

		if (Evolution)
		{
			BendingConstraints = MakeShared<Softs::FPBDBendingSpringConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				CrossEdges,
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		else
		{
			BendingConstraints = MakeShared<Softs::FPBDBendingSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				CrossEdges,
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
}

void FClothConstraints::CreateExtremeDeformationConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TMap<FString, const TSet<int32>*>& VertexSets,
	const FTriangleMesh& TriangleMesh,
	const FClothingPatternData* PatternData)
{
	if (Softs::FPBDExtremeDeformationConstraints::IsEnabled(ConfigProperties))
	{
		const TArray<Chaos::TVec2<int32>> CrossEdges = TriangleMesh.GetUniqueAdjacentPoints();

		if (Evolution)
		{
			ExtremeDeformationConstraints = MakeShared<Softs::FPBDExtremeDeformationConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				CrossEdges,
				WeightMaps,
				VertexSets,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		else
		{
			ExtremeDeformationConstraints = MakeShared<Softs::FPBDExtremeDeformationConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				CrossEdges,
				WeightMaps,
				VertexSets,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		//No ConstraintInits or ConstraintRules
	}
}

void FClothConstraints::CreateAreaConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FTriangleMesh& TriangleMesh,
	const FClothingPatternData* PatternData)
{
	if (Softs::FXPBDAreaSpringConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			XAreaConstraints = MakeShared<Softs::FXPBDAreaSpringConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}
		else
		{
			XAreaConstraints = MakeShared<Softs::FXPBDAreaSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;
		++NumConstraintRules;
	}
	else if (Softs::FPBDAreaSpringConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			if (PatternData && PatternData->PatternPositions.Num())
			{
				AreaConstraints = MakeShared<Softs::FPBDAreaSpringConstraints>(
					Evolution->GetSoftBodyParticles(ParticleRangeId),
					TriangleMesh,
					PatternData->WeldedFaceVertexPatternPositions,
					WeightMaps,
					ConfigProperties,
					/*bTrimKinematicConstraints =*/ true);

			}
			else
			{
				AreaConstraints = MakeShared<Softs::FPBDAreaSpringConstraints>(
					Evolution->GetSoftBodyParticles(ParticleRangeId),
					TriangleMesh.GetSurfaceElements(),
					WeightMaps,
					ConfigProperties,
					/*bTrimKinematicConstraints =*/ true);
			}
		}
		else
		{
			AreaConstraints = MakeShared<Softs::FPBDAreaSpringConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				TriangleMesh.GetSurfaceElements(),
				WeightMaps,
				ConfigProperties,
				/*bTrimKinematicConstraints =*/ true);
		}

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
}

void FClothConstraints::CreateLongRangeConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& Tethers,
	Softs::FSolverReal MeshScale)
{
	if (Softs::FPBDLongRangeConstraints::IsEnabled(ConfigProperties))
	{
		if (Evolution)
		{
			//  Now that we're only doing a single iteration of Long range constraints, and they're more of a fake constraint to jump start our initial guess, it's not clear that using XPBD makes sense here.
			LongRangeConstraints = MakeShared<Softs::FPBDLongRangeConstraints>(
				Evolution->GetSoftBodyParticles(ParticleRangeId),
				Tethers,
				WeightMaps,
				ConfigProperties,
				MeshScale);
			++NumPreSubstepConstraintRules;
		}
		else
		{
			LongRangeConstraints = MakeShared<Softs::FPBDLongRangeConstraints>(
				PBDEvolution->Particles(),
				ParticleOffset,
				NumParticles,
				Tethers,
				WeightMaps,
				ConfigProperties,
				MeshScale);
			++NumConstraintInits;
		}
	}
}

void FClothConstraints::CreateMaxDistanceConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	Softs::FSolverReal MeshScale)
{
	if (ConfigProperties.GetValue(FClothingSimulationConfig::UseLegacyConfigName, false))
	{
		FString MaxDistanceString = TEXT("MaxDistance");
		MaxDistanceString = Softs::FPBDSphericalConstraint::GetMaxDistanceString(ConfigProperties, MaxDistanceString);  // Uses the same string for both the default weight map name and the property name
		const TConstArrayView<FRealSingle> MaxDistances = WeightMaps.FindRef(MaxDistanceString);
		if (MaxDistances.Num() != NumParticles)
		{
			return;  // Legacy configs disable the constraint when the weight map is missing
		}
	}

	if (Softs::FPBDSphericalConstraint::IsEnabled(ConfigProperties))
	{
		MaximumDistanceConstraints = MakeShared<Softs::FPBDSphericalConstraint>(
			ParticleOffset,
			NumParticles,
			*AnimationPositions,
			WeightMaps,
			ConfigProperties,
			MeshScale);

		++NumConstraintRules;
	}
}

void FClothConstraints::CreateBackstopConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	Softs::FSolverReal MeshScale,
	const TMap<FName, FClothingAccessoryMeshData>* AccessoryMeshes)
{
	if (ConfigProperties.GetValue(FClothingSimulationConfig::UseLegacyConfigName, false))
	{
		FString BackstopRadiusString = TEXT("BackstopRadius");
		FString BackstopDistanceString = TEXT("BackstopDistance");
		BackstopRadiusString = Softs::FPBDSphericalBackstopConstraint::GetBackstopRadiusString(ConfigProperties, BackstopRadiusString);        // Uses the same string for both the default weight map name and the property name
		BackstopDistanceString = Softs::FPBDSphericalBackstopConstraint::GetBackstopDistanceString(ConfigProperties, BackstopDistanceString);  //
		const TConstArrayView<FRealSingle> BackstopRadiuses = WeightMaps.FindRef(BackstopRadiusString);
		const TConstArrayView<FRealSingle> BackstopDistances = WeightMaps.FindRef(BackstopDistanceString);

		if (BackstopRadiuses.Num() != NumParticles || BackstopDistances.Num() != NumParticles)
		{
			return;  // Legacy configs disable the constraint when the weight maps are missing
		}
	}

	if (Softs::FPBDSphericalBackstopConstraint::IsEnabled(ConfigProperties))
	{
		// Get name of backstop mesh
		const TArray<Softs::FSolverVec3>* BackstopPositions = AnimationPositions;
		const TArray<Softs::FSolverVec3>* BackstopNormals = AnimationNormals;
		bool bUseGlobalIndexation = true;

		if (AccessoryMeshes)
		{
			const FName BackstopMeshName = FName(Softs::FPBDSphericalBackstopConstraint::GetBackstopMeshNameString(ConfigProperties, TEXT("")));
			if (const FClothingAccessoryMeshData* const BackstopMeshData = AccessoryMeshes->Find(BackstopMeshName))
			{
				if (BackstopMeshData->GetInterpolatedAnimationPositions().Num() == NumParticles && BackstopMeshData->GetInterpolatedAnimationNormals().Num() == NumParticles)
				{
					BackstopPositions = &BackstopMeshData->GetInterpolatedAnimationPositions();
					BackstopNormals = &BackstopMeshData->GetInterpolatedAnimationNormals();
					bUseGlobalIndexation = false;
				}
			}
		}


		BackstopConstraints = MakeShared<Softs::FPBDSphericalBackstopConstraint>(
			ParticleOffset,
			NumParticles,
			*BackstopPositions,
			*BackstopNormals,
			WeightMaps,
			ConfigProperties,
			MeshScale,
			bUseGlobalIndexation);

		++NumConstraintRules;
	}
}

void FClothConstraints::CreateAnimDriveConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (Softs::FPBDAnimDriveConstraint::IsEnabled(ConfigProperties))
	{
		check(AnimationVelocities); // Legacy code didn't use to have AnimationVelocities

		AnimDriveConstraints = MakeShared<Softs::FPBDAnimDriveConstraint>(
			ParticleOffset,
			NumParticles,
			*AnimationPositions,
			*AnimationVelocities,
			WeightMaps,
			ConfigProperties);

		++NumConstraintInits;  // Uses init to update the property tables
		++NumConstraintRules;
	}
}

void FClothConstraints::CreateVelocityAndPressureField(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FTriangleMesh& TriangleMesh)
{
	if (Evolution)
	{
		// Always create velocity field--we allow turning it on via blueprints
		constexpr Softs::FSolverReal WorldScale = (Softs::FSolverReal)100.;
		VelocityAndPressureField = MakeShared<Softs::FVelocityAndPressureField>(
			Evolution->GetSoftBodyParticles(ParticleRangeId),
			&TriangleMesh,
			ConfigProperties,
			WeightMaps,
			WorldScale
			);
		++NumExternalForceRules;
	}
}

void FClothConstraints::CreateVelocityPressureAndBuoyancyField(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FTriangleMesh& TriangleMesh)
{
	if (Evolution)
	{
		// Always create velocity, pressure and buoyancy field--we allow turning it on via blueprints
		constexpr Softs::FSolverReal WorldScale = (Softs::FSolverReal)100.;
		VelocityPressureAndBuoyancyField = MakeShared<Buoyancy::FBuoyancyField>(
			Evolution->GetSoftBodyParticles(ParticleRangeId),
			&TriangleMesh,
			ConfigProperties,
			WeightMaps,
			WorldScale,
			ExternalForces
			);
		++NumExternalForceRules;
	}
}

void FClothConstraints::CreateExternalForces(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (Evolution)
	{
		// Always create external forces
		check(Normals);
		ExternalForces = MakeShared<Softs::FExternalForces>(
			Evolution->GetSoftBodyParticles(ParticleRangeId),
			*Normals,
			WeightMaps,
			ConfigProperties
			);

		++NumExternalForceRules;
		++NumUpdateLinearSystemRules;
	}
}

void FClothConstraints::CreateCollisionConstraint(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	Softs::FSolverReal MeshScale,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TArray<const FClothComplexColliders*>& ComplexColliders)
{
	if (Evolution)
	{
		// Always create collision constraint
		check(LastSubframeCollisionTransformsCCD);

		TMap<Softs::FParticleRangeIndex, Softs::FPBDComplexColliderBoneData> ComplexBoneData;
		for (const FClothComplexColliders* Colliders : ComplexColliders)
		{
			if (Colliders)
			{
				Colliders->ExtractComplexColliderBoneData(ComplexBoneData);
			}
		}

		CollisionConstraint = MakeShared<Softs::FPBDSoftBodyCollisionConstraint>(
			*LastSubframeCollisionTransformsCCD,
			ConfigProperties,
			MeshScale,
			CollisionParticleCollided,
			CollisionContacts,
			CollisionNormals,
			CollisionPhis,
			ComplexBoneData,
			WeightMaps
			);
		++NumCollisionConstraintRules;
		++NumUpdateLinearSystemCollisionsRules;

		SkinnedTriangleCollisionsConstraint = MakeShared<Softs::FPBDSkinnedTriangleMeshCollisionConstraints>(
			NumParticles,
			WeightMaps,
			ConfigProperties
		);

		for (const FClothComplexColliders* Colliders : ComplexColliders)
		{
			if (Colliders)
			{
				const int32 NumMeshes = Colliders->GetNumSkinnedTriangleMeshes();
				for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
				{
					SkinnedTriangleCollisionsConstraint->AddSkinnedTriangleMesh(
						Softs::FParticleRangeIndex(Colliders->GetCollisionRangeId(), Colliders->GetSkinnedTriangleMeshIndex(MeshIndex)),
						FSkinnedTriangleMeshPtr(Colliders->GetSkinnedTriangleMesh(MeshIndex)),
						Colliders->GetSkinnedTriangleMeshVelocities(MeshIndex));
				}
			}
		}
		++NumConstraintInits;
		++NumPostCollisionConstraintRules;
	}
}

void FClothConstraints::CreateClothClothConstraints(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection)
{
	if (ManagedArrayCollection)
	{
		const Softs::FEmbeddedSpringFacade SpringFacade(*ManagedArrayCollection, FName(TEXT("SimVertices3D")));
		for (int32 ConstraintIndex = 0; ConstraintIndex < SpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
		{
			const Softs::FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.GetSpringConstraintConst(ConstraintIndex);
			if (SpringConstraintFacade.IsValid())
			{
				const FUintVector2 EndPoints = SpringConstraintFacade.GetConstraintEndPointNumIndices();
				const FString& ConstraintName = SpringConstraintFacade.GetConstraintName();
				if (EndPoints == FUintVector2(1, 1))
				{
					if (ConstraintName == TEXT("VertexSpringConstraint"))
					{
						if (Softs::FXPBDVertexConstraints::IsEnabled(ConfigProperties))
						{
							if (ensure(!ClothVertexSpringConstraints))
							{

								ClothVertexSpringConstraints = MakeShared<Softs::FXPBDVertexConstraints>(
									Evolution->GetSoftBodyParticles(ParticleRangeId),
									ConfigProperties,
									SpringConstraintFacade
								);
								++NumConstraintInits;
								++NumConstraintRules;
							}
						}
					}
				}
				else if (EndPoints == FUintVector2(1, 3))
				{
					if (ConstraintName == TEXT("VertexFaceSpringConstraint"))
					{
						if (Softs::FXPBDVertexFaceConstraints::IsEnabled(ConfigProperties))
						{
							if (ensure(!ClothVertexFaceSpringConstraints))
							{

								ClothVertexFaceSpringConstraints = MakeShared<Softs::FXPBDVertexFaceConstraints>(
									Evolution->GetSoftBodyParticles(ParticleRangeId),
									ConfigProperties,
									SpringConstraintFacade
								);
								++NumConstraintInits;
								++NumConstraintRules;
							}
						}
					}
					else if (ConstraintName == TEXT("VertexFaceRepulsionConstraint"))
					{
						if (Softs::FPBDVertexFaceRepulsionConstraints::IsEnabled(ConfigProperties))
						{
							if (ensure(!RepulsionConstraints))
							{
								RepulsionConstraints = MakeShared<Softs::FPBDVertexFaceRepulsionConstraints>(
									Evolution->GetSoftBodyParticles(ParticleRangeId),
									ConfigProperties,
									SpringConstraintFacade
								);
								++NumPostCollisionConstraintRules;
							}
						}
					}
				}
				else if (EndPoints == FUintVector2(3, 3))
				{
					if (ConstraintName == TEXT("FaceSpringConstraint"))
					{
						if (Softs::FXPBDFaceConstraints::IsEnabled(ConfigProperties))
						{
							if (ensure(!ClothFaceSpringConstraints))
							{

								ClothFaceSpringConstraints = MakeShared<Softs::FXPBDFaceConstraints>(
									Evolution->GetSoftBodyParticles(ParticleRangeId),
									ConfigProperties,
									SpringConstraintFacade
								);
								++NumConstraintInits;
								++NumConstraintRules;
							}
						}
					}
				}
			}
		}
	}
}

void FClothConstraints::CreateForceBasedRules()
{
	FRuleCreator RuleCreator(this);

	if (ExternalForces)
	{
		RuleCreator.AddExternalForceRule_Apply(ExternalForces.Get());
		RuleCreator.AddUpdateLinearSystemRule_UpdateLinearSystem(ExternalForces.Get());
	}
	if (VelocityAndPressureField)
	{
		RuleCreator.AddExternalForceRule_Apply(VelocityAndPressureField.Get());

		// TODO Linear System
	}

	if (VelocityPressureAndBuoyancyField)
	{
		RuleCreator.AddExternalForceRule_Apply(VelocityPressureAndBuoyancyField.Get());		
	}

	if (PerSolverField)
	{
		RuleCreator.AddExternalForceRule(
			[this](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PerSolverField);
			const TArray<FVector>& LinearVelocities = PerSolverField->GetOutputResults(EFieldCommandOutputType::LinearVelocity);
			const TArray<FVector>& LinearForces = PerSolverField->GetOutputResults(EFieldCommandOutputType::LinearForce);
			const FVector* const LinearVelocitiesView = LinearVelocities.IsEmpty() ? nullptr :
				Particles.GetConstArrayView(LinearVelocities).GetData();
			const FVector* const LinearForcesView = LinearForces.IsEmpty() ? nullptr :
				Particles.GetConstArrayView(LinearForces).GetData();
			if (!LinearVelocitiesView && !LinearForcesView)
			{
				return;
			}

			Softs::FSolverVec3* const Acceleration = Particles.GetAcceleration().GetData();
			const Softs::FSolverReal* const InvM = Particles.GetInvM().GetData();
			for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
			{
				if (InvM[Index] != (Softs::FSolverReal)0.)
				{
					if (LinearForcesView)
					{
						Acceleration[Index] += Softs::FSolverVec3(LinearForcesView[Index]) * InvM[Index];
					}
					if (LinearVelocitiesView)
					{
						Acceleration[Index] += Softs::FSolverVec3(LinearVelocitiesView[Index]) / Dt;
					}
				}
			}
		});

		// TODO: Linear System
	}

	if (XStretchBiasConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XStretchBiasConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XStretchBiasConstraints.Get());
	}
	if (XEdgeConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XEdgeConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XEdgeConstraints.Get());
		RuleCreator.AddUpdateLinearSystemRule_UpdateLinearSystem(XEdgeConstraints.Get());
	}
	if (EdgeConstraints)
	{
		RuleCreator.AddPBDParallelInitRule(EdgeConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(EdgeConstraints.Get());
	}
	if (XAnisoSpringConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XAnisoSpringConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(&XAnisoSpringConstraints->GetEdgeConstraints());
		RuleCreator.AddUpdateLinearSystemRule(
			[this](const Softs::FSolverParticlesRange& Particles, const FSolverReal Dt, Softs::FEvolutionLinearSystem& LinearSystem)
		{
			XAnisoSpringConstraints->GetEdgeConstraints().UpdateLinearSystem(Particles, Dt, LinearSystem);
			XAnisoSpringConstraints->GetAxialConstraints().UpdateLinearSystem(Particles, Dt, LinearSystem);
		});
	}

	if (XBendingConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XBendingConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XBendingConstraints.Get());
		RuleCreator.AddUpdateLinearSystemRule_UpdateLinearSystem(XBendingConstraints.Get());
	}
	if (BendingConstraints)
	{
		RuleCreator.AddPBDParallelInitRule(BendingConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(BendingConstraints.Get());
	}
	if (BendingElementConstraints)
	{
		// These are PBD, not XPBD constraints, but the difference is that XPBD constraints have an Init method and PBD (usually) do not.
		// BendingElementConstraints do have an Init function, so this will add a call to that.
		// TODO: clean up the names of these methods to be more explicit.
		constexpr bool bInitParticles = true;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(BendingElementConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(BendingElementConstraints.Get());
	}
	if (XBendingElementConstraints)
	{
		constexpr bool bInitParticles = true;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XBendingElementConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XBendingElementConstraints.Get());
	}
	if (XAnisoBendingElementConstraints)
	{
		constexpr bool bInitParticles = true;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XAnisoBendingElementConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XAnisoBendingElementConstraints.Get());
	}
	if (XAreaConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(XAreaConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(XAreaConstraints.Get());
	}
	if (AreaConstraints)
	{
		RuleCreator.AddPBDParallelInitRule(AreaConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(AreaConstraints.Get());
	}
	if (XAnisoSpringConstraints)
	{
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(&XAnisoSpringConstraints->GetAxialConstraints());
	}
	if (MaximumDistanceConstraints)
	{
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(MaximumDistanceConstraints.Get());
	}
	if (BackstopConstraints && !ClothConstraints::Private::bSwapBackstopAnimDriveApply)
	{
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(BackstopConstraints.Get());
	}
	if (AnimDriveConstraints)
	{
		RuleCreator.AddPBDParallelInitRule(AnimDriveConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(AnimDriveConstraints.Get());
	}
	if (BackstopConstraints && ClothConstraints::Private::bSwapBackstopAnimDriveApply)
	{
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(BackstopConstraints.Get());
	}
	if (ClothVertexSpringConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(ClothVertexSpringConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(ClothVertexSpringConstraints.Get());
	}
	if (ClothVertexFaceSpringConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(ClothVertexFaceSpringConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(ClothVertexFaceSpringConstraints.Get());
	}
	if (ClothFaceSpringConstraints)
	{
		constexpr bool bInitParticles = false;
		RuleCreator.AddXPBDParallelInitRule<bInitParticles>(ClothFaceSpringConstraints.Get());
		constexpr bool bPostCollisions = false;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(ClothFaceSpringConstraints.Get());
	}

	if (CollisionConstraint)
	{
		RuleCreator.AddPerIterationCollisionPBDConstraintRule(
			[this](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const TArray<Softs::FSolverCollisionParticlesRange>& CollisionParticles)
			{
				const Softs::FEvolutionGroupContext& Context = Evolution->GetGroupContextForSoftBody(Particles.GetRangeId());

				if (Context.NumPBDIterations == 1)
				{
					// When only doing one PBDIteration, there's no need for planar constraints.
					CollisionConstraint->Apply(Particles, Dt, CollisionParticles);
				}
				else
				{
					CollisionConstraint->ApplyWithPlanarConstraints(Particles, Dt, CollisionParticles, Context.CurrentPBDIteration == 0);
				}
			});
		RuleCreator.AddUpdateLinearSystemCollisionsRule_UpdateLinearSystem(CollisionConstraint.Get());
	}

	if (SkinnedTriangleCollisionsConstraint)
	{
		RuleCreator.AddPostInitialGuessParallelInitRule(
			[this](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
			{
				if (bSkipSelfCollisionInit && SkinnedTriangleCollisionsConstraint->GetUseSelfCollisionSubstepsForSkinnedTriangleMeshes())
				{
					return;
				}
				SkinnedTriangleCollisionsConstraint->Init(Particles, Dt);
			});
		constexpr bool bPostCollisions = true;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(SkinnedTriangleCollisionsConstraint.Get());
	}

	if (SelfCollisionInit && SelfCollisionConstraints)
	{
		RuleCreator.AddPostInitialGuessParallelInitRule(
		[this](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if (bSkipSelfCollisionInit)
			{
				return;
			}
			SelfCollisionInit->Init(Particles, SelfCollisionConstraints->GetThicknessWeighted());
			SelfCollisionConstraints->Init(Particles, Dt, SelfCollisionInit->GetCollidableSubMesh(), SelfCollisionInit->GetDynamicSpatialHash(), SelfCollisionInit->GetKinematicColliderSpatialHash(), SelfCollisionInit->GetVertexGIAColors(), SelfCollisionInit->GetTriangleGIAColors());
		});

		constexpr bool bPostCollisions = true;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(SelfCollisionConstraints.Get());
		RuleCreator.AddUpdateLinearSystemRule_UpdateLinearSystem(SelfCollisionConstraints.Get());
	}

	if (RepulsionConstraints)
	{
		constexpr bool bPostCollisions = true;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(RepulsionConstraints.Get());
	}

	if (SelfCollisionSphereConstraints)
	{
		RuleCreator.AddPostInitialGuessParallelInitRule(
			[this](const Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if (bSkipSelfCollisionInit)
			{
				return;
			}
			SelfCollisionSphereConstraints->Init(Particles);
		});

		constexpr bool bPostCollisions = true;
		RuleCreator.AddPerIterationPBDConstraintRule_Apply<bPostCollisions>(SelfCollisionSphereConstraints.Get());
	}

	if (SelfCollisionInit && SelfIntersectionConstraints)
	{
		RuleCreator.AddPreSubstepConstraintRule(
			[this](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if (bSkipSelfCollisionInit)
			{
				return;
			}
			SelfIntersectionConstraints->Apply(Particles, SelfCollisionInit->GetContourMinimizationIntersections(), Dt);
		});

		RuleCreator.AddPostSubstepConstraintRule(
			[this](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if (bSkipSelfCollisionInit)
			{
				return;
			}
			const int32 NumContourIterations = SelfCollisionInit->GetNumContourMinimizationPostSteps();
			for (int32 Iter = 0; Iter < NumContourIterations; ++Iter)
			{
				SelfCollisionInit->PostStepInit(Particles);
				SelfIntersectionConstraints->Apply(Particles, SelfCollisionInit->GetPostStepContourMinimizationIntersections(), Dt);
			}
		});
	}
	if (LongRangeConstraints)
	{
		RuleCreator.AddPreSubstepConstraintRule(
			[this](Softs::FSolverParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::ESolverMode SolverMode)
		{
			if ((SolverMode & Softs::ESolverMode::PBD) != Softs::ESolverMode::None)
			{
				// Only doing one iteration.
				constexpr int32 NumLRAIterations = 1;
				LongRangeConstraints->ApplyProperties(Dt, NumLRAIterations);
				LongRangeConstraints->Apply(Particles, Dt);  // Run the LRA constraint only once per timestep
			}
		});
	}
}


void FClothConstraints::CreatePBDRules()
{
	check(PBDEvolution);
	check(ConstraintInitOffset == INDEX_NONE)
	if (NumConstraintInits)
	{
		ConstraintInitOffset = PBDEvolution->AddConstraintInitRange(NumConstraintInits, false);
	}
	check(ConstraintRuleOffset == INDEX_NONE)

	if (NumConstraintRules)
	{
		ConstraintRuleOffset = PBDEvolution->AddConstraintRuleRange(NumConstraintRules, false);
	}
	check(PostCollisionConstraintRuleOffset == INDEX_NONE);
	if (NumPostCollisionConstraintRules)
	{
		PostCollisionConstraintRuleOffset = PBDEvolution->AddPostCollisionConstraintRuleRange(NumPostCollisionConstraintRules, false);
	}
	check(PostprocessingConstraintRuleOffset == INDEX_NONE);
	if (NumPostprocessingConstraintRules)
	{
		PostprocessingConstraintRuleOffset = PBDEvolution->AddConstraintPostprocessingsRange(NumPostprocessingConstraintRules, false);
	}

	TFunction<void(Softs::FSolverParticles&, const Softs::FSolverReal)>* const ConstraintInits = PBDEvolution->ConstraintInits().GetData() + ConstraintInitOffset;
	TFunction<void(Softs::FSolverParticles&, const Softs::FSolverReal)>* const ConstraintRules = PBDEvolution->ConstraintRules().GetData() + ConstraintRuleOffset;
	TFunction<void(Softs::FSolverParticles&, const Softs::FSolverReal)>* const PostCollisionConstraintRules = PBDEvolution->PostCollisionConstraintRules().GetData() + PostCollisionConstraintRuleOffset;
	TFunction<void(Softs::FSolverParticles&, const Softs::FSolverReal)>* const PostprocessingConstraintRules = PBDEvolution->ConstraintPostprocessings().GetData() + PostprocessingConstraintRuleOffset;

	int32 ConstraintInitIndex = 0;
	int32 ConstraintRuleIndex = 0;
	int32 PostCollisionConstraintRuleIndex = 0;
	int32 PostprocessingConstraintRuleIndex = 0;

	if (XStretchBiasConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
		{
			XStretchBiasConstraints->Init();
			XStretchBiasConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
		};

		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XStretchBiasConstraints->Apply(Particles, Dt);
		};
	}
	if (XEdgeConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				XEdgeConstraints->Init();
				XEdgeConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};

		ConstraintRules[ConstraintRuleIndex++] = 
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				XEdgeConstraints->Apply(Particles, Dt);
			};
	}
	if (EdgeConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				EdgeConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				EdgeConstraints->Apply(Particles, Dt);
			};
	}
	if (XAnisoSpringConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
		{
			XAnisoSpringConstraints->Init();
			XAnisoSpringConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
		};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XAnisoSpringConstraints->GetEdgeConstraints().Apply(Particles, Dt);
		};
	}
	if (XBendingConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				XBendingConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
				XBendingConstraints->Init();
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				XBendingConstraints->Apply(Particles, Dt);
			};
	}
	if (BendingConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				BendingConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				BendingConstraints->Apply(Particles, Dt);
			};
	}
	if (BendingElementConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				BendingElementConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
				BendingElementConstraints->Init(Particles);
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				BendingElementConstraints->Apply(Particles, Dt);
			};
	}
	if (XBendingElementConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XBendingElementConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			XBendingElementConstraints->Init(Particles);
		};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XBendingElementConstraints->Apply(Particles, Dt);
		};
	}
	if (XAnisoBendingElementConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XAnisoBendingElementConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			XAnisoBendingElementConstraints->Init(Particles);
		};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XAnisoBendingElementConstraints->Apply(Particles, Dt);
		};
	}
	if (XAreaConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				XAreaConstraints->Init();
				XAreaConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				XAreaConstraints->Apply(Particles, Dt);
			};
	}
	if (AreaConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				AreaConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				AreaConstraints->Apply(Particles, Dt);
			};
	}
	if (XAnisoSpringConstraints)
	{
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			XAnisoSpringConstraints->GetAxialConstraints().Apply(Particles, Dt);
		};
	}
	if (MaximumDistanceConstraints)
	{
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				MaximumDistanceConstraints->Apply(Particles, Dt);
			};
	}
	if (BackstopConstraints)
	{
		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				BackstopConstraints->Apply(Particles, Dt);
			};
	}
	if (AnimDriveConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& /*Particles*/, const Softs::FSolverReal Dt)
			{
				AnimDriveConstraints->ApplyProperties(Dt, PBDEvolution->GetIterations());
			};

		ConstraintRules[ConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				AnimDriveConstraints->Apply(Particles, Dt);
			};
	}

	if (SelfCollisionInit && SelfCollisionConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				if (bSkipSelfCollisionInit)
				{
					return;
				}
				// Thickness * 2 to account for collision radius for both particles
				SelfCollisionInit->Init(Particles, SelfCollisionConstraints->GetThicknessWeighted());
				SelfCollisionConstraints->Init(Particles, Dt, SelfCollisionInit->GetCollidableSubMesh(), SelfCollisionInit->GetDynamicSpatialHash(), SelfCollisionInit->GetKinematicColliderSpatialHash(), SelfCollisionInit->GetVertexGIAColors(), SelfCollisionInit->GetTriangleGIAColors());
			};

		PostCollisionConstraintRules[PostCollisionConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				SelfCollisionConstraints->Apply(Particles, Dt);
			};

	}

	if (SelfCollisionSphereConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal /*Dt*/)
			{
				if (bSkipSelfCollisionInit)
				{
					return;
				}
				SelfCollisionSphereConstraints->Init(Particles);
			};

		PostCollisionConstraintRules[PostCollisionConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				SelfCollisionSphereConstraints->Apply(Particles, Dt);
			};
	}

	// The following constraints only run once per subframe, so we do their Apply as part of the Init() which modifies P
	// To avoid possible dependency order issues, add them last
	if (SelfCollisionInit && SelfIntersectionConstraints)
	{
		// The following constraints only run once per subframe, so we do their Apply as part of the Init() which modifies P
		// To avoid possible dependency order issues, add them last
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{
				if (bSkipSelfCollisionInit)
				{
					return;
				}
				SelfIntersectionConstraints->Apply(Particles, SelfCollisionInit->GetContourMinimizationIntersections(), Dt);
			};

		PostprocessingConstraintRules[PostprocessingConstraintRuleIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
		{
			if (bSkipSelfCollisionInit)
			{
				return;
			}
			const int32 NumContourIterations = SelfCollisionInit->GetNumContourMinimizationPostSteps();
			for (int32 Iter = 0; Iter < NumContourIterations; ++Iter)
			{
				SelfCollisionInit->PostStepInit(Particles);
				SelfIntersectionConstraints->Apply(Particles, SelfCollisionInit->GetPostStepContourMinimizationIntersections(), Dt);
			}
		};
	}

	// Long range constraints modify particle P as part of Init. To avoid possible dependency order issues,
	// add them last
	if (LongRangeConstraints)
	{
		ConstraintInits[ConstraintInitIndex++] =
			[this](Softs::FSolverParticles& Particles, const Softs::FSolverReal Dt)
			{	
				// Only doing one iteration.
				constexpr int32 NumLRAIterations = 1;
				LongRangeConstraints->ApplyProperties(Dt, NumLRAIterations);
				LongRangeConstraints->Apply(Particles, Dt);  // Run the LRA constraint only once per timestep
			};
	}
	check(ConstraintInitIndex == NumConstraintInits);
	check(ConstraintRuleIndex == NumConstraintRules);
	check(PostCollisionConstraintRuleIndex == NumPostCollisionConstraintRules);
}

void FClothConstraints::UpdateFromSolver(const FSolverVec3& SolverGravity, bool bPerClothGravityOverrideEnabled,
	const FSolverVec3& FictitiousAngularVelocity, const FSolverVec3& ReferenceSpaceLocation,
	const FSolverVec3& InSolverWindVelocity, const FSolverReal LegacyWindAdaptation)
{
	if (ExternalForces)
	{
		ExternalForces->SetWorldGravityMultiplier((FSolverReal)ClothingSimulationClothConsoleVariables::CVarGravityMultiplier.GetValueOnAnyThread());
		ExternalForces->SetSolverGravityProperties(SolverGravity, bPerClothGravityOverrideEnabled);
		ExternalForces->SetFictitiousForcesData(FictitiousAngularVelocity, ReferenceSpaceLocation);
		ExternalForces->SetSolverWind(InSolverWindVelocity, LegacyWindAdaptation);
	}
	SolverWindVelocity = InSolverWindVelocity;
}

void FClothConstraints::Update(
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const TMap<FString, const TSet<int32>*>& VertexSets,
	const TMap<FString, const TSet<int32>*>& FaceSets,
	const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
	Softs::FSolverReal MeshScale,
	Softs::FSolverReal MaxDistancesScale,
	const FRotation3& LocalSpaceRotation,
	const FRotation3& ReferenceSpaceRotation,
	const FReal LocalSpaceScale,
	const FSolverVec3 LocalSpaceLocation)
{
	if (EdgeConstraints)
	{
		EdgeConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XEdgeConstraints)
	{
		XEdgeConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XStretchBiasConstraints)
	{
		XStretchBiasConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XAnisoSpringConstraints)
	{
		XAnisoSpringConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (BendingConstraints)
	{
		BendingConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XBendingConstraints)
	{
		XBendingConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (BendingElementConstraints)
	{
		BendingElementConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XBendingElementConstraints)
	{
		XBendingElementConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XAnisoBendingElementConstraints)
	{
		XAnisoBendingElementConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (ExtremeDeformationConstraints)
	{
		ExtremeDeformationConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (AreaConstraints)
	{
		AreaConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (XAreaConstraints)
	{
		XAreaConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (LongRangeConstraints)
	{
		LongRangeConstraints->SetProperties(ConfigProperties, WeightMaps, MeshScale);
	}
	if (MaximumDistanceConstraints)
	{
		MaximumDistanceConstraints->SetProperties(ConfigProperties, WeightMaps, MeshScale * MaxDistancesScale);
	}
	if (BackstopConstraints)
	{
		BackstopConstraints->SetProperties(ConfigProperties, WeightMaps, MeshScale);
	}
	if (AnimDriveConstraints)
	{
		AnimDriveConstraints->SetProperties(ConfigProperties, WeightMaps);
	}
	if (ClothVertexSpringConstraints)
	{
		ClothVertexSpringConstraints->SetProperties(ConfigProperties);
	}
	if (ClothVertexFaceSpringConstraints)
	{
		ClothVertexFaceSpringConstraints->SetProperties(ConfigProperties);
	}
	if (ClothFaceSpringConstraints)
	{
		ClothFaceSpringConstraints->SetProperties(ConfigProperties);
	}
	bool bSelfCollisionInitCollidableSubMeshWillRebuild = false;
	if (SelfCollisionInit)
	{
		SelfCollisionInit->SetProperties(ConfigProperties, FaceSets);
		bSelfCollisionInitCollidableSubMeshWillRebuild = SelfCollisionInit->GetCollidableSubMeshDirty();
	}
	if (SelfCollisionConstraints)
	{
		SelfCollisionConstraints->SetProperties(ConfigProperties, WeightMaps, FaceIntMaps);
		if (bSelfCollisionInitCollidableSubMeshWillRebuild)
		{
			// Kinematic collider indices will change on rebuild. Reset any existing constraints.
			SelfCollisionConstraints->ResetKinematicCollider();
		}
	}
	if (SelfCollisionSphereConstraints)
	{
		SelfCollisionSphereConstraints->SetProperties(ConfigProperties, VertexSets);
	}
	if (RepulsionConstraints)
	{
		RepulsionConstraints->SetProperties(ConfigProperties);
	}

	bool bUsePointBasedWindModel = false;
	if (ExternalForces)
	{
		ExternalForces->SetProperties(ConfigProperties, WeightMaps);
		bUsePointBasedWindModel = ExternalForces->UsePointBasedWindModel();
	}
	if (VelocityAndPressureField)
	{
		constexpr FSolverReal WorldScale = 100.f;
		const bool bPointBasedWindDisablesAccurateWind = ClothingSimulationClothConsoleVariables::CVarLegacyDisablesAccurateWind.GetValueOnAnyThread();
		const bool bEnableAerodynamics = !(bUsePointBasedWindModel && bPointBasedWindDisablesAccurateWind);
		VelocityAndPressureField->SetPropertiesAndWind(
			ConfigProperties,
			WeightMaps,
			WorldScale,
			bEnableAerodynamics,
			SolverWindVelocity,
			LocalSpaceRotation,
			ReferenceSpaceRotation
		);
	}
	if (VelocityPressureAndBuoyancyField)
	{
		constexpr FSolverReal WorldScale = 100.f;
		const bool bPointBasedWindDisablesAccurateWind = ClothingSimulationClothConsoleVariables::CVarLegacyDisablesAccurateWind.GetValueOnAnyThread();
		const bool bEnableAerodynamics = !(bUsePointBasedWindModel && bPointBasedWindDisablesAccurateWind);
		VelocityPressureAndBuoyancyField->SetPropertiesAndBuoyancy(
			ConfigProperties,
			WeightMaps,
			WorldScale,
			bEnableAerodynamics,
			SolverWindVelocity,
			LocalSpaceRotation,
			LocalSpaceScale,
			LocalSpaceLocation,
			ReferenceSpaceRotation
		);
	}
	if (CollisionConstraint)
	{
		static IConsoleVariable* const WriteCCDContacts = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.PBDEvolution.WriteCCDContacts"));
		const bool bWriteCCDContacts = WriteCCDContacts ? WriteCCDContacts->GetBool() : false;
		CollisionConstraint->SetProperties(ConfigProperties, WeightMaps);
		CollisionConstraint->SetWriteDebugContacts(bWriteCCDContacts);
	}
	if (SkinnedTriangleCollisionsConstraint)
	{
		SkinnedTriangleCollisionsConstraint->SetProperties(ConfigProperties, WeightMaps);
	}
}

void FClothConstraints::ResetRestLengths(
	const TConstArrayView<Softs::FSolverVec3>& NewRestLengthPositions,
	const Softs::FCollectionPropertyConstFacade& ConfigProperties,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps
)
{
	if (XStretchBiasConstraints)
	{ 
		// TODO. There is no way to create these from the cloth UI
	}
	if (XEdgeConstraints)
	{
		XEdgeConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (EdgeConstraints)
	{
		EdgeConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (XAnisoSpringConstraints)
	{
		XAnisoSpringConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (XBendingConstraints)
	{
		XBendingConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (BendingConstraints)
	{
		BendingConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (BendingElementConstraints)
	{
		BendingElementConstraints->ResetRestLengths(NewRestLengthPositions, ConfigProperties, WeightMaps);
	}
	if (XBendingElementConstraints)
	{
		XBendingElementConstraints->ResetRestLengths(NewRestLengthPositions, ConfigProperties, WeightMaps);
	}
	if (XAnisoBendingElementConstraints)
	{
		XAnisoBendingElementConstraints->ResetRestLengths(NewRestLengthPositions, ConfigProperties, WeightMaps);
	}
	if (XAreaConstraints)
	{
		XAreaConstraints->ResetRestLengths(NewRestLengthPositions);
	}
	if (AreaConstraints)
	{
		AreaConstraints->ResetRestLengths(NewRestLengthPositions);
	}
}
}  // End namespace Chaos
