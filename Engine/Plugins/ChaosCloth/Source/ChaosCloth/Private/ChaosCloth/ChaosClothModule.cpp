// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothModule.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "ChaosCloth/ChaosClothingSimulationFactory.h"
#include "ChaosCloth/SkeletalMeshComponentCacheAdapter.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

//////////////////////////////////////////////////////////////////////////
// FChaosClothModule

class FChaosClothModule : public IChaosClothModuleInterface, public IClothingSimulationFactoryClassProvider
{
  public:
    virtual void StartupModule() override
    {
        check(GConfig);
		IModularFeatures::Get().RegisterModularFeature(IClothingSimulationFactoryClassProvider::FeatureName, this);

    	SkeletalMeshAdapter = MakeUnique<Chaos::FSkeletalMeshCacheAdapter>();
    	Chaos::RegisterAdapter(SkeletalMeshAdapter.Get());
    }

    virtual void ShutdownModule() override
    {
		IModularFeatures::Get().UnregisterModularFeature(IClothingSimulationFactoryClassProvider::FeatureName, this);

    	UnregisterAdapter(SkeletalMeshAdapter.Get());
    	SkeletalMeshAdapter = nullptr;
    }

	TSubclassOf<UClothingSimulationFactory> GetClothingSimulationFactoryClass() const override
	{
		return UChaosClothingSimulationFactory::StaticClass();
	}
private:
	TUniquePtr<Chaos::FSkeletalMeshCacheAdapter> SkeletalMeshAdapter;
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FChaosClothModule, ChaosCloth);
DEFINE_LOG_CATEGORY(LogChaosCloth);
