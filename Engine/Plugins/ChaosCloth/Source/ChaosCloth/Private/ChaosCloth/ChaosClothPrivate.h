// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosCloth, Log, All);
DECLARE_STATS_GROUP(TEXT("ChaosCloth"), STATGROUP_ChaosCloth, STATCAT_Advanced);
