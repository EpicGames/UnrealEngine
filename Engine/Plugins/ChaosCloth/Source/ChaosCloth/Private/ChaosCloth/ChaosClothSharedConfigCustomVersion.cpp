// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothSharedConfigCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FChaosClothSharedConfigCustomVersion::GUID(0x5FBC6907, 0x55C840AE, 0x8E67F184, 0x5EFFF13F);

// Register the custom version with core
FCustomVersionRegistration GRegisterChaosClothSharedConfigCustomVersion(FChaosClothSharedConfigCustomVersion::GUID, FChaosClothSharedConfigCustomVersion::LatestVersion, TEXT("ChaosClothSharedConfigVer"));
