// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothVisualization.h"
#if CHAOS_DEBUG_DRAW
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosWeightMapTarget.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ExtrudedTaperedCapsule.h"
#include "Chaos/Levelset.h"
#include "Chaos/Sphere.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/TaperedCylinder.h"
#include "Chaos/Triangle.h"
#include "Chaos/VelocityField.h"
#include "Chaos/PBDAnimDriveConstraint.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/PBDSelfCollisionSphereConstraints.h"
#include "Chaos/PBDSoftBodyCollisionConstraint.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/PBDUnilateralTetConstraints.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/SoftsExternalForces.h"
#include "Chaos/XPBDBendingConstraints.h"
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/XPBDAnisotropicBendingConstraints.h"
#include "Chaos/XPBDAnisotropicSpringConstraints.h"
#include "Chaos/XPBDEmbeddedSpringConstraints.h"
#include "Chaos/XPBDStretchBiasElementConstraints.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "DynamicMeshBuilder.h"
#include "Engine/EngineTypes.h"
#include "Misc/LazySingleton.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Chaos/MLLevelset.h"
#if WITH_EDITOR
#include "CanvasItem.h"     //
#include "Engine/Canvas.h"  // For draw text
#include "Engine/Engine.h"  //
#include "Materials/Material.h"
#include "UObject/ICookInfo.h"
#endif  // #if WITH_EDITOR

#define LOCTEXT_NAMESPACE "ChaosClothVisualization"

namespace Chaos
{
	namespace Private
	{
		static int DrawSkinnedLattice = 0;
		static FAutoConsoleVariableRef CVarClothVizDrawSkinnedLattice(TEXT("p.ChaosClothVisualization.DrawSkinnedLattice"), DrawSkinnedLattice, TEXT("Draw skinned lattice, 0 = none, 1 = filled, 2 = empty, 3 = both"));
		
		static int DrawMLLevelSetFlag = 0;
		static FAutoConsoleVariableRef CVarChaosDrawMLLevelSet(TEXT("p.ChaosClothVisualization.DrawMLLevelSetFlag"), DrawMLLevelSetFlag, TEXT("Default: 0 -> Draws nothing. 1,3,5 : Draws MLLevelSet Isocounters. 2,3 : Draws Bounding Box. 4, 5: Draws Training Bounding Box Axes."));

		// TODO: move these options to be somewhere the new cloth editor visualization can use.
		enum class EBendingDrawMode : int
		{
			BuckleStatus = 0,
			ParallelGraphColor = 1,
			Anisotropy = 2,
			RestAngle = 3
		};
		static int32 BendingDrawMode = (int32)EBendingDrawMode::BuckleStatus;
		static FAutoConsoleVariableRef CVarClothVizBendDrawMode(TEXT("p.ChaosClothVisualization.BendingDrawMode"), BendingDrawMode, TEXT("Bending draw mode, 0 = BuckleStatus, 1 = Parallel graph color, 2 = Anisotropy, 3 = RestAngle"));

		enum class EStretchBiasDrawMode : int
		{
			ParallelGraphColor = 0,
			WarpStretch = 1,
			WeftStretch = 2,
			BiasStretch = 3
		};
		static int32 StretchBiasDrawMode = (int32)EStretchBiasDrawMode::ParallelGraphColor;
		static FAutoConsoleVariableRef CVarClothVizStretchBiasDrawMode(TEXT("p.ChaosClothVisualization.StretchBiasDrawMode"), StretchBiasDrawMode, TEXT("Stretch draw mode, 0 = Parallel graph color, 1 = Warp Stretch, 2 = Weft Stretch, 3 = BiasStretch"));
		static float StretchBiasDrawRangeMin = -1.f;
		static float StretchBiasDrawRangeMax = 1.f;
		static FAutoConsoleVariableRef CVarClothVizStretchBiasDrawRangeMin(TEXT("p.ChaosClothVisualization.StretchBiasDrawRangeMin"), StretchBiasDrawRangeMin, TEXT("Min stretch in draw color range. Negative = compressed, 0 = undeformed, positive = stretched. (When drawing warp/weft stretch)"));
		static FAutoConsoleVariableRef CVarClothVizStretchBiasDrawRangeMax(TEXT("p.ChaosClothVisualization.StretchBiasDrawRangeMax"), StretchBiasDrawRangeMax, TEXT("Max stretch in draw color range. Negative = compressed, 0 = undeformed, positive = stretched. (When drawing warp/weft stretch)"));
		static bool bStretchBiasDrawOutOfRange = true;
		static FAutoConsoleVariableRef CVarClothVizStretchBiasDrawOutOfRange(TEXT("p.ChaosClothVisualization.StretchBiasDrawOutOfRange"), bStretchBiasDrawOutOfRange, TEXT("Draw out of range elements (When drawing warp/weft stretch)"));

		enum class EAnisoSpringDrawMode : int
		{
			ParallelGraphColor = 0,
			Anisotropy = 1,
		};
		static int32 AnisoSpringDrawMode = (int32)EAnisoSpringDrawMode::ParallelGraphColor;
		static FAutoConsoleVariableRef CVarClothVizAnisoSpringDrawMode(TEXT("p.ChaosClothVisualization.AnisoSpringDrawMode"), AnisoSpringDrawMode, TEXT("Stretch draw mode, 0 = Parallel graph color, 1 = Anisotropy"));

		static FString WeightMapName = "";
		static FAutoConsoleVariableRef CVarClothVizWeightMapName(TEXT("p.ChaosClothVisualization.WeightMapName"), WeightMapName, TEXT("Weight map name to be visualized"));

		static bool bDrawInForeground = true;
		static FAutoConsoleVariableRef CVarClothVizDrawInForeground(TEXT("p.ChaosClothVisualization.DrawInForeground"), bDrawInForeground, TEXT("Draw in foreground when outside the cloth/SKM editor"));

		static int32 NumSphereSections = 12;
		static FAutoConsoleVariableRef CVarClothVizNumSphereSections(TEXT("p.ChaosClothVisualization.NumSphereSections"), NumSphereSections, TEXT("Number of sections used to draw debug wireframe spheres in PIE/game."));

		// copied from ClothEditorMode
		FLinearColor PseudoRandomColor(int32 NumColorRotations)
		{
			constexpr uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
			uint8 Seed = Spread;
			NumColorRotations = FMath::Abs(NumColorRotations);
			for (int32 Rotation = 0; Rotation < NumColorRotations; ++Rotation)
			{
				Seed += Spread;
			}
			return FLinearColor::MakeFromHSV8(Seed, 180, 140);
		}

		static uint8 GetDepthPriority()
		{
			// set depth to SDPG_MAX not SDPG_Foreground when drawing in foreground.
			// SDPG_Foreground does not draw when PIE is paused (its buffer is flushed).
			return bDrawInForeground ? SDPG_MAX : SDPG_World;
		}

		static bool ShouldDrawParticle(const FClothVisualizationNoGC::FDrawFilter& Filter, const TSet<int32>* const DebugFilterSet, int32 ParticleIndex, int32 GlobalParticleOffset)
		{
			return (Filter.SingleVertex == INDEX_NONE ||
				Filter.SingleVertex == ParticleIndex + GlobalParticleOffset)

				&&

				(!DebugFilterSet ||
					DebugFilterSet->Contains(ParticleIndex));
		}

		static bool ShouldDrawEdge(const FClothVisualizationNoGC::FDrawFilter& Filter, const TSet<int32>* const DebugFilterSet, const FUintVector2& Edge, int32 EdgeToGlobalOffset, int32 EdgeToLocalOffset)
		{
			return (Filter.SingleVertex == INDEX_NONE ||
				Filter.SingleVertex == Edge[0] + EdgeToGlobalOffset ||
				Filter.SingleVertex == Edge[1] + EdgeToGlobalOffset)

				&&

				(!DebugFilterSet ||
					DebugFilterSet->Contains(Edge[0] + EdgeToLocalOffset) ||
					DebugFilterSet->Contains(Edge[1] + EdgeToLocalOffset));
		}

		static bool ShouldDrawConstraint2(const FClothVisualizationNoGC::FDrawFilter& Filter, const TSet<int32>* const DebugFilterSet, const TVec2<int32>& Constraint, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
		{
			return (Filter.SingleVertex == INDEX_NONE ||
				Filter.SingleVertex == Constraint[0] + ConstraintToGlobalOffset ||
				Filter.SingleVertex == Constraint[1] + ConstraintToGlobalOffset)

				&&

				(!DebugFilterSet ||
					DebugFilterSet->Contains(Constraint[0] + ConstraintToLocalOffset) ||
					DebugFilterSet->Contains(Constraint[1] + ConstraintToLocalOffset));
		}

		static bool ShouldDrawElement(const FClothVisualizationNoGC::FDrawFilter& Filter, const TSet<int32>* const DebugFilterSet, const TVec3<int32>& Element, int32 ElementToGlobalOffset, int32 ElementToLocalOffset)
		{
			return (Filter.SingleVertex == INDEX_NONE ||
				Filter.SingleVertex == Element.X + ElementToGlobalOffset ||
				Filter.SingleVertex == Element.Y + ElementToGlobalOffset ||
				Filter.SingleVertex == Element.Z + ElementToGlobalOffset)

				&&

				(!DebugFilterSet ||
					DebugFilterSet->Contains(Element.X + ElementToLocalOffset) ||
					DebugFilterSet->Contains(Element.Y + ElementToLocalOffset) ||
					DebugFilterSet->Contains(Element.Z + ElementToLocalOffset));
		}

		static bool ShouldDrawConstraint4(const FClothVisualizationNoGC::FDrawFilter& Filter, const TSet<int32>* const DebugFilterSet, const TVec4<int32>& Constraint, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
		{
			return (Filter.SingleVertex == INDEX_NONE ||
				Filter.SingleVertex == Constraint[0] + ConstraintToGlobalOffset ||
				Filter.SingleVertex == Constraint[1] + ConstraintToGlobalOffset ||
				Filter.SingleVertex == Constraint[2] + ConstraintToGlobalOffset ||
				Filter.SingleVertex == Constraint[3] + ConstraintToGlobalOffset)

				&&

				(!DebugFilterSet ||
					DebugFilterSet->Contains(Constraint[0] + ConstraintToLocalOffset) ||
					DebugFilterSet->Contains(Constraint[1] + ConstraintToLocalOffset) ||
					DebugFilterSet->Contains(Constraint[2] + ConstraintToLocalOffset) ||
					DebugFilterSet->Contains(Constraint[3] + ConstraintToLocalOffset));
		}
	}  // namespace Private

	// Delay loading materials to their first point of use to avoid causing sync flush issues and slow loading when starting PIE
	class FClothVisualizationNoGC::FMaterials
#if WITH_EDITOR && CHAOS_DEBUG_DRAW
		: public FGCObject
	{
	private:
		FMaterials()
			: ClothMaterial(LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"), nullptr, LOAD_None, nullptr))
			, ClothMaterialColor(LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/Cloth/CameraLitVertexColor.CameraLitVertexColor"), nullptr, LOAD_None, nullptr))
			, ClothMaterialVertex(LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/WidgetVertexColorMaterial"), nullptr, LOAD_None, nullptr))
			, CollisionMaterial(LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/PhAT_UnselectedMaterial"), nullptr, LOAD_None, nullptr))
		{}

		~FMaterials() = default;

		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			Collector.AddReferencedObject(ClothMaterial);
			Collector.AddReferencedObject(ClothMaterialColor);
			Collector.AddReferencedObject(ClothMaterialVertex);
			Collector.AddReferencedObject(CollisionMaterial);
		}

		virtual FString GetReferencerName() const override { return TEXT("Chaos::Private::FClothVisualizationMaterials"); }
		//~ End FGCObject interface

		// Visualization material
		TObjectPtr<const UMaterial> ClothMaterial;
		TObjectPtr<const UMaterial> ClothMaterialColor;
		TObjectPtr<const UMaterial> ClothMaterialVertex;
		TObjectPtr<const UMaterial> CollisionMaterial;

	public:
		TObjectPtr<const UMaterial> GetClothMaterial() const { return ClothMaterial; }
		TObjectPtr<const UMaterial> GetClothMaterialColor() const { return ClothMaterialColor; }
		TObjectPtr<const UMaterial> GetClothMaterialVertex() const { return ClothMaterialVertex; }
		TObjectPtr<const UMaterial> GetCollisionMaterial() const { return CollisionMaterial; }
#else
	{
	private:
		FMaterials() = default;
		~FMaterials() = default;

	public:
		TObjectPtr<const UMaterial> GetClothMaterial() const { return nullptr; }
		TObjectPtr<const UMaterial> GetClothMaterialColor() const { return nullptr; }
		TObjectPtr<const UMaterial> GetClothMaterialVertex() const { return nullptr; }
		TObjectPtr<const UMaterial> GetCollisionMaterial() const { return nullptr; }
#endif  // #if WITH_EDITOR && CHAOS_DEBUG_DRAW

		friend class ::FLazySingleton;
		static FMaterials& GetInstance() { return TLazySingleton<FMaterials>::Get(); }  // Don't move the implementation outside of the cpp file
		static void TearDown() { TLazySingleton<FMaterials>::TearDown(); }  // Don't move the implementation outside of the cpp file
	};

	FClothVisualizationNoGC::FClothVisualizationNoGC(const ::Chaos::FClothingSimulationSolver* InSolver)
		: Solver(InSolver)
	{
	}

	FClothVisualizationNoGC::~FClothVisualizationNoGC() = default;

	void FClothVisualizationNoGC::SetSolver(const ::Chaos::FClothingSimulationSolver* InSolver)
	{
		Solver = InSolver;
	}

	static FMatrix GetLocalSpaceToWorld(const ::Chaos::FClothingSimulationSolver& Solver)
	{
		return FMatrix(
			FVector(Solver.GetLocalSpaceScale(), 0., 0.),
			FVector(0., Solver.GetLocalSpaceScale(), 0.),
			FVector(0., 0., Solver.GetLocalSpaceScale()),
			Solver.GetLocalSpaceLocation());
	}

	static FVector GetWorldPosition(const ::Chaos::FClothingSimulationSolver& Solver, const ::Chaos::Softs::FSolverVec3& SolverPosition)
	{
		return FVector(SolverPosition) * Solver.GetLocalSpaceScale() + Solver.GetLocalSpaceLocation();
	}

	static FVector GetWorldVector(const ::Chaos::FClothingSimulationSolver& Solver, const ::Chaos::Softs::FSolverVec3& SolverVector)
	{
		return FVector(SolverVector) * Solver.GetLocalSpaceScale();
	}

	static void DrawText(FCanvas* Canvas, const FSceneView* SceneView, const FVector& Pos, const FText& Text, const FLinearColor& Color, const float Scale = 1.f)
	{
#if WITH_EDITOR
		if (Canvas && SceneView)
		{
			FVector2D PixelLocation;
			if (SceneView->WorldToPixel(Pos, PixelLocation))
			{
				// WorldToPixel doesn't account for DPIScale
				const float DPIScale = Canvas->GetDPIScale();
				FCanvasTextItem TextItem(PixelLocation / DPIScale, Text, GEngine->GetSmallFont(), Color);
				TextItem.Scale = FVector2D::UnitVector  * Scale;
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(Canvas);
			}
		}
		else
#endif
		{
			FDebugDrawQueue::GetInstance().DrawDebugString(Pos, Text.ToString(), nullptr, Color.ToFColor(true), KINDA_SMALL_NUMBER, true, 0.75f * Scale);
		}
	}

#if WITH_EDITOR
	void FClothVisualizationNoGC::DrawPhysMeshShaded(const FDrawContext& Context) const
	{
		if (!Solver || !FMaterials::GetInstance().GetClothMaterial() || !Context.PDI)
		{
			return;
		}

		FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
		int32 VertexIndex = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}


			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};

			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(InvMasses.Num() == Positions.Num());

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];
				if(ShouldDrawElement(Element))
				{
					const FVector3f Pos0(Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector3f Pos1(Positions[Element.Y - Offset]);
					const FVector3f Pos2(Positions[Element.Z - Offset]);

					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					const bool bIsKinematic0 = (InvMasses[Element.X - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic1 = (InvMasses[Element.Y - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic2 = (InvMasses[Element.Z - Offset] == (Softs::FSolverReal)0.);

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), bIsKinematic0 ? FColor::Purple : FColor::White));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), bIsKinematic1 ? FColor::Purple : FColor::White));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), bIsKinematic2 ? FColor::Purple : FColor::White));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					VertexIndex += 3;
				}
			}
		}

		const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(*Solver);
		MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, FMaterials::GetInstance().GetClothMaterial()->GetRenderProxy(), Chaos::Private::GetDepthPriority(), false, false);
	}

	void FClothVisualizationNoGC::DrawWeightMapWithName(const FDrawContext& Context, const FString& Name) const
	{
		if (!Solver || !FMaterials::GetInstance().GetClothMaterialColor() || !Context.PDI)
		{
			return;
		}

		FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
		int32 VertexIndex = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};
			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<FRealSingle>& WeightMap = Cloth->GetWeightMapByName(Solver, Name.IsEmpty() ? Private::WeightMapName : Name);

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];

				if (ShouldDrawElement(Element))
				{
					const FVector3f Pos0(Positions[Element.X - Offset]);
					const FVector3f Pos1(Positions[Element.Y - Offset]);
					const FVector3f Pos2(Positions[Element.Z - Offset]);

					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					FLinearColor VertexColor1 = FLinearColor::Black;
					FLinearColor VertexColor2 = FLinearColor::Black;
					FLinearColor VertexColor3 = FLinearColor::Black;

					if (!WeightMap.IsEmpty() && WeightMap.Num() == Positions.Num()) // if map with that name exists and not empty
					{
						const FRealSingle Value0(WeightMap[Element.X - Offset]);
						const FRealSingle Value1(WeightMap[Element.Y - Offset]);
						const FRealSingle Value2(WeightMap[Element.Z - Offset]);

						VertexColor1 = FLinearColor::LerpUsingHSV(FLinearColor::Black, FLinearColor::White, (float)Value0);
						VertexColor2 = FLinearColor::LerpUsingHSV(FLinearColor::Black, FLinearColor::White, (float)Value1);
						VertexColor3 = FLinearColor::LerpUsingHSV(FLinearColor::Black, FLinearColor::White, (float)Value2);
					}

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), VertexColor1.ToFColor(true)));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), VertexColor2.ToFColor(true)));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), VertexColor3.ToFColor(true)));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					VertexIndex += 3;
				}
			}
		}

		const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(*Solver);
		MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, FMaterials::GetInstance().GetClothMaterialColor()->GetRenderProxy(), Chaos::Private::GetDepthPriority(), false, false);
	}

	void FClothVisualizationNoGC::DrawInpaintWeightsMatched(const FDrawContext& Context) const
	{
		DrawWeightMapWithName(Context, TEXT("_InpaintWeightMask"));
	}

	void FClothVisualizationNoGC::DrawSelfCollisionLayers(const FDrawContext& Context) const
	{

		if (!Solver || !FMaterials::GetInstance().GetClothMaterialColor() || !Context.PDI)
		{
			return;
		}

		FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
		int32 VertexIndex = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};
			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<int32>& WeightMap = Cloth->GetFaceIntMapByProperty(Solver, Softs::FPBDCollisionSpringConstraints::SelfCollisionLayersName);

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];
				if (ShouldDrawElement(Element))
				{
					const FVector3f Pos0(Positions[Element.X - Offset]);
					const FVector3f Pos1(Positions[Element.Y - Offset]);
					const FVector3f Pos2(Positions[Element.Z - Offset]);

					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					FLinearColor VertexColor1 = FLinearColor::Gray;
					FLinearColor VertexColor2 = FLinearColor::Gray;
					FLinearColor VertexColor3 = FLinearColor::Gray;

					if (!WeightMap.IsEmpty() && WeightMap.Num() == Elements.Num() && WeightMap[ElementIndex] != INDEX_NONE) // if map with that name exists and not empty
					{
						VertexColor1 = VertexColor2 = VertexColor3 = Chaos::Private::PseudoRandomColor(WeightMap[ElementIndex]);
					}

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), VertexColor1.ToFColor(true)));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), VertexColor2.ToFColor(true)));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), VertexColor3.ToFColor(true)));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					VertexIndex += 3;
				}
			}
		}

		FMatrix LocalSimSpaceToWorld(FMatrix::Identity);
		LocalSimSpaceToWorld.SetOrigin(Solver->GetLocalSpaceLocation());
		MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, FMaterials::GetInstance().GetClothMaterialColor()->GetRenderProxy(), Chaos::Private::GetDepthPriority(), false, false);
	}
#endif  // #if WITH_EDITOR

	void FClothVisualizationNoGC::DrawParticleIndices(const FDrawTextsContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor DynamicColor = FColor::White;
		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
				};

			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(InvMasses.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FVector Position = GetWorldPosition(*Solver, Positions[Index]);

					const FText Text = FText::AsNumber(Offset + Index);
					DrawText(Context.Canvas, Context.SceneView, Position, Text, InvMasses[Index] == (Softs::FSolverReal)0. ? KinematicColor : DynamicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawElementIndices(const FDrawTextsContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor DynamicColor = FColor::White;
		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};

			const TArray<TVec3<int32>>& Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(InvMasses.Num() == Positions.Num());

			for (int32 Index = 0; Index < Elements.Num(); ++Index)
			{
				const TVec3<int32>& Element = Elements[Index];
				if (ShouldDrawElement(Element))
				{
					const FVector Position = GetWorldPosition(*Solver,
						(Positions[Element[0] - Offset] +
							Positions[Element[1] - Offset] +
							Positions[Element[2] - Offset]) / (Softs::FSolverReal)3.f);

					const bool bIsKinematic0 = (InvMasses[Element.X - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic1 = (InvMasses[Element.Y - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic2 = (InvMasses[Element.Z - Offset] == (Softs::FSolverReal)0.);
					const FLinearColor& Color = (bIsKinematic0 && bIsKinematic1 && bIsKinematic2) ? KinematicColor : DynamicColor;
					const FText Text = FText::AsNumber(Index);
					DrawText(Context.Canvas, Context.SceneView, Position, Text, Color);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawMaxDistanceValues(const FDrawTextsContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor DynamicColor = FColor::White;
		static const FLinearColor KinematicColor = FColor::Purple;

		FNumberFormattingOptions NumberFormattingOptions;
		NumberFormattingOptions.AlwaysSign = false;
		NumberFormattingOptions.UseGrouping = false;
		NumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;
		NumberFormattingOptions.MinimumIntegralDigits = 1;
		NumberFormattingOptions.MaximumIntegralDigits = 6;
		NumberFormattingOptions.MinimumFractionalDigits = 2;
		NumberFormattingOptions.MaximumFractionalDigits = 2;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
				};
			const int32 NumParticles = Cloth->GetNumParticles(Solver);
			const Softs::FPBDFlatWeightMapView MaxDistances(
				(Softs::FSolverVec2)(Softs::FPBDSphericalConstraint::GetWeightedFloatMaxDistance(Cloth->GetConfig()->GetProperties(Cloth->GetLODIndex(Solver)), FVector2f(0.f, 1.f))),
				Cloth->GetWeightMapByProperty(Solver, Softs::FPBDSphericalConstraint::MaxDistanceName),
				NumParticles);

			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(NumParticles == Positions.Num());
			check(NumParticles == InvMasses.Num());

			for (int32 Index = 0; Index < MaxDistances.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FReal MaxDistance = (FReal)MaxDistances.GetValue(Index);
					const FVector Position = GetWorldPosition(*Solver, Positions[Index]);

					const FText Text = FText::AsNumber(MaxDistance, &NumberFormattingOptions);
					DrawText(Context.Canvas, Context.SceneView, Position, Text, InvMasses[Index] == (Softs::FSolverReal)0. ? KinematicColor : DynamicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawLocalSpaceBoneNames(const FDrawTextsContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		// Draw local space
		DrawText(Context.Canvas, Context.SceneView, Solver->GetLocalSpaceLocation() + FVec3(2.), LOCTEXT("SimulationSpace", "Simulation Space"), FLinearColor::White, 1.5f);

		// Draw reference spaces
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
			check(Cloth->GetMesh());
			DrawText(Context.Canvas, Context.SceneView, ReferenceSpaceTransform.GetLocation() + FVec3(-2.), FText::FromName(Cloth->GetMesh()->GetReferenceBoneName()), FLinearColor::White, 1.5f);
		}
	}

	static void DrawPoint(FPrimitiveDrawInterface* PDI, const FVector& Pos, const FLinearColor& Color, const UMaterial* ClothMaterialVertex, const float Thickness = 1.f)  // Use color or material
	{
		if (!PDI)
		{
			FDebugDrawQueue::GetInstance().DrawDebugPoint(Pos, Color.ToFColor(true), false, KINDA_SMALL_NUMBER, Chaos::Private::GetDepthPriority(), Thickness);
			return;
		}
#if WITH_EDITOR
		if (ClothMaterialVertex)
		{
			const FMatrix& ViewMatrix = PDI->View->ViewMatrices.GetWorldToView();
			const FVector XAxis = ViewMatrix.GetColumn(0); // Just using transpose here (orthogonal transform assumed)
			const FVector YAxis = ViewMatrix.GetColumn(1);
			DrawDisc(PDI, Pos, XAxis, YAxis, Color.ToFColor(true), 0.5f, 10, ClothMaterialVertex->GetRenderProxy(), Chaos::Private::GetDepthPriority());
		}
		else
		{
			PDI->DrawPoint(Pos, Color, Thickness, Chaos::Private::GetDepthPriority());
		}
#endif
	}

	static void DrawLine(FPrimitiveDrawInterface* PDI, const FVector& Pos0, const FVector& Pos1, const FLinearColor& Color)
	{
		if (!PDI)
		{
			FDebugDrawQueue::GetInstance().DrawDebugLine(Pos0, Pos1, Color.ToFColor(true), false, KINDA_SMALL_NUMBER, Chaos::Private::GetDepthPriority(), 0.f);
			return;
		}
#if WITH_EDITOR
		PDI->DrawLine(Pos0, Pos1, Color, Chaos::Private::GetDepthPriority(), 0.0f, 0.001f);
#endif
	}

	static void DrawArrow(FPrimitiveDrawInterface* PDI, const FVector& StartLocation, const FVector& EndLocation, const FLinearColor& Color)
	{
		// This is just taken from FChaosVDDebugDrawUtils::DrawArrowVector
		
		const FVector LineVectorToDraw = EndLocation - StartLocation;

		FVector ArrowDir;
		float ArrowLength;
		LineVectorToDraw.ToDirectionAndLength(ArrowDir, ArrowLength);

		FVector YAxis, ZAxis;
		ArrowDir.FindBestAxisVectors(YAxis, ZAxis);
		const FMatrix ArrowTransformMatrix(ArrowDir, YAxis, ZAxis, StartLocation);

		constexpr float MinTipOfArrowSize = 0.2f;
		constexpr float MaxTipOfArrowSize = 10.0f;
		constexpr float MaxVectorSizeForArrow = 100.0f; // The vector size that is the upper limit after which we just use the max size for the tip of the arrow

		const float ProportionalArrowSize = MaxTipOfArrowSize * (ArrowLength / MaxVectorSizeForArrow);
		const float ArrowSize = FMath::Clamp(ProportionalArrowSize, MinTipOfArrowSize, MaxTipOfArrowSize);

		DrawLine(PDI, ArrowTransformMatrix.TransformPosition(FVector(ArrowLength, 0, 0)), ArrowTransformMatrix.TransformPosition(FVector::ZeroVector), Color);
		DrawLine(PDI, ArrowTransformMatrix.TransformPosition(FVector(ArrowLength, 0, 0)), ArrowTransformMatrix.TransformPosition(FVector(ArrowLength - ArrowSize, +ArrowSize, +ArrowSize)), Color);
		DrawLine(PDI, ArrowTransformMatrix.TransformPosition(FVector(ArrowLength, 0, 0)), ArrowTransformMatrix.TransformPosition(FVector(ArrowLength - ArrowSize, +ArrowSize, -ArrowSize)), Color);
		DrawLine(PDI, ArrowTransformMatrix.TransformPosition(FVector(ArrowLength, 0, 0)), ArrowTransformMatrix.TransformPosition(FVector(ArrowLength - ArrowSize, -ArrowSize, +ArrowSize)), Color);
		DrawLine(PDI, ArrowTransformMatrix.TransformPosition(FVector(ArrowLength, 0, 0)), ArrowTransformMatrix.TransformPosition(FVector(ArrowLength - ArrowSize, -ArrowSize, -ArrowSize)), Color);
	}

	static void DrawArc(FPrimitiveDrawInterface* PDI, const FVector& Base, const FVector& X, const FVector& Y, ::Chaos::FReal MinAngle, float MaxAngle, ::Chaos::FReal Radius, const FLinearColor& Color)
	{
		static const int32 Sections = 10;
		const FReal AngleStep = FMath::DegreesToRadians((MaxAngle - MinAngle) / (FReal)Sections);
		FReal CurrentAngle = FMath::DegreesToRadians(MinAngle);
		FVector LastVertex = Base + Radius * (FMath::Cos(CurrentAngle) * X + FMath::Sin(CurrentAngle) * Y);

		for(int32 i = 0; i < Sections; i++)
		{
			CurrentAngle += AngleStep;
			const FVector ThisVertex = Base + Radius * (FMath::Cos(CurrentAngle) * X + FMath::Sin(CurrentAngle) * Y);
			DrawLine(PDI, LastVertex, ThisVertex, Color);
			LastVertex = ThisVertex;
		}
	}


	static void DrawSphere(FPrimitiveDrawInterface* PDI, const FVector& Center, const FQuat& Rotation, const FReal Radius, const FLinearColor& Color, const FMaterialRenderProxy* SolidMaterial = nullptr)
	{
		if (!PDI)
		{
			FDebugDrawQueue::GetInstance().DrawDebugSphere(Center, Radius, Chaos::Private::NumSphereSections, Color.ToFColor(true), false, KINDA_SMALL_NUMBER, Chaos::Private::GetDepthPriority(), 0.f);
			return;
		}
#if WITH_EDITOR
		if (SolidMaterial)
		{
			::DrawSphere(PDI, Center, FRotator(Rotation), FVector(Radius), 32, 16, SolidMaterial, Chaos::Private::GetDepthPriority());
		}
		else
		{
			const FTransform Transform(Rotation, Center, FVector(1.));
			DrawWireSphereAutoSides(PDI, Transform, Color, Radius, Chaos::Private::GetDepthPriority(), 0.0f, 0.001f, false);
		}
#endif
	}

	static void DrawSphere(FPrimitiveDrawInterface* PDI, const ::Chaos::TSphere<::Chaos::FReal, 3>& Sphere, const FQuat& Rotation, const FVector& Position, const FReal Scale, const FLinearColor& Color, const FMaterialRenderProxy* SolidMaterial = nullptr)
	{
		const FVec3 Center = Position + Rotation.RotateVector(Scale * FVec3(Sphere.GetCenterf()));
		const FReal Radius = Sphere.GetRadiusf() * Scale;
		DrawSphere(PDI, Center, Rotation, Radius, Color, SolidMaterial);
	}

	static void DrawBox(FPrimitiveDrawInterface* PDI, const ::Chaos::FAABB3& Box, const FQuat& Rotation, const FVector& Position, FReal Scale,  const FLinearColor& Color)
	{
		if (!PDI)
		{
			const FVec3 Center = Position + Rotation.RotateVector(Scale * Box.GetCenter());
			FDebugDrawQueue::GetInstance().DrawDebugBox(Center, Scale * Box.Extents() * 0.5f, Rotation, Color.ToFColor(true), false, KINDA_SMALL_NUMBER, Chaos::Private::GetDepthPriority(), 0.f);
			return;
		}
#if WITH_EDITOR
		const FMatrix BoxToWorld = FTransform(Rotation, Position, FVector(Scale)).ToMatrixWithScale();
		DrawWireBox(PDI, BoxToWorld, FBox(Box.Min(), Box.Max()), Color, Chaos::Private::GetDepthPriority(), 0.0f, 0.001f, false);
#endif
	}

	static void DrawCapsule(FPrimitiveDrawInterface* PDI, const ::Chaos::FCapsule& Capsule, const FQuat& Rotation, const FVector& Position, FReal Scale, const FLinearColor& Color)
	{
		const FReal Radius = Scale * Capsule.GetRadiusf();
		const FReal HalfHeight = Scale * Capsule.GetHeightf() * 0.5f + Radius;
		const FVec3 Center = Position + Rotation.RotateVector(Scale * FVec3(Capsule.GetCenterf()));
		if (!PDI)
		{
			const FQuat Orientation = FQuat::FindBetweenNormals(FVec3::UpVector, Capsule.GetAxis());
			FDebugDrawQueue::GetInstance().DrawDebugCapsule(Center, HalfHeight, Radius, Rotation * Orientation, Color.ToFColor(true), false, KINDA_SMALL_NUMBER, Chaos::Private::GetDepthPriority(), 0.f);
			return;
		}
#if WITH_EDITOR
		const FVec3 Up = Capsule.GetAxis();
		FVec3 Forward, Right;
		Up.FindBestAxisVectors(Forward, Right);
		const FVector X = Rotation.RotateVector(Forward);
		const FVector Y = Rotation.RotateVector(Right);
		const FVector Z = Rotation.RotateVector(Up);
		DrawWireCapsule(PDI, Center, X, Y, Z, Color, Radius, HalfHeight, 12, Chaos::Private::GetDepthPriority(), 0.0f, 0.001f, false);
#endif
	}

#if WITH_EDITOR
	static void AppendTaperedCylinderTriangles(FDynamicMeshBuilder& MeshBuilder, const FVector3f& Position1, const FVector3f& Position2, const FRealSingle Radius1, const FRealSingle Radius2, const int32 NumSides, const FLinearColor& Color)
	{
		const FQuat4f Q = (Position2 - Position1).ToOrientationQuat();
		const FVector3f I = Q.GetRightVector();
		const FVector3f J = Q.GetUpVector();
		const FVector3f K = Q.GetForwardVector();

		const FRealSingle AngleDelta = (FRealSingle)2. * (FRealSingle)PI / NumSides;
		int32 LastVertex1 = MeshBuilder.AddVertex(FDynamicMeshVertex(Position1 + I * Radius1, -K, I, FVector2f(0.f, 0.f), Color.ToFColor(true)));
		int32 LastVertex2 = MeshBuilder.AddVertex(FDynamicMeshVertex(Position2 + I * Radius2, -K, I, FVector2f(1.f, 0.f), Color.ToFColor(true))); 
		for (int32 SideIndex = 1; SideIndex <= NumSides; ++SideIndex)
		{
			const FRealSingle Angle = AngleDelta * FRealSingle(SideIndex);
			const FVector3f ArcPos = I * FMath::Cos(Angle) + J * FMath::Sin(Angle);
			
			const FVector3f Pos1 = Position1 + ArcPos * Radius1;
			const FVector3f Pos2 = Position2 + ArcPos * Radius2;
			const FVector3f Normal = (Pos1 - Position1).GetSafeNormal();

			const int32 Vertex1 = MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, -K, Normal, FVector2f(0.f, 0.f), Color.ToFColor(true)));
			const int32 Vertex2 = MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, -K, Normal, FVector2f(1.f, 0.f), Color.ToFColor(true)));
			MeshBuilder.AddTriangle(LastVertex1, LastVertex2, Vertex1);
			MeshBuilder.AddTriangle(LastVertex2, Vertex2, Vertex1);

			LastVertex1 = Vertex1;
			LastVertex2 = Vertex2;
		}
	}
#endif

	static void DrawTaperedCylinder(FPrimitiveDrawInterface* PDI, const FVector& Position1, const FVector& Position2, const FReal Radius1, const FReal Radius2, const int32 NumSides, const FLinearColor& Color)
	{
		const FQuat Q = (Position2 - Position1).ToOrientationQuat();
		const FVector I = Q.GetRightVector();
		const FVector J = Q.GetUpVector();

		const FReal	AngleDelta = (FReal)2. * (FReal)PI / NumSides;
		FVector LastVertex1 = Position1 + I * Radius1;
		FVector LastVertex2 = Position2 + I * Radius2;

		for (int32 SideIndex = 1; SideIndex <= NumSides; ++SideIndex)
		{
			const FReal Angle = AngleDelta * FReal(SideIndex);
			const FVector ArcPos = I * FMath::Cos(Angle) + J * FMath::Sin(Angle);
			const FVector Vertex1 = Position1 + ArcPos * Radius1;
			const FVector Vertex2 = Position2 + ArcPos * Radius2;

			DrawLine(PDI, LastVertex1, Vertex1, Color);
			DrawLine(PDI, LastVertex2, Vertex2, Color);
			DrawLine(PDI, LastVertex1, LastVertex2, Color);

			LastVertex1 = Vertex1;
			LastVertex2 = Vertex2;
		}
	}

	static void DrawTaperedCylinder(FPrimitiveDrawInterface* PDI, const ::Chaos::FTaperedCylinder& TaperedCylinder, const FQuat& Rotation, const FVector& Position, const FReal Scale, const FLinearColor& Color)
	{
		const FReal Radius1 = Scale * TaperedCylinder.GetRadius1();
		const FReal Radius2 = Scale * TaperedCylinder.GetRadius2();
		const FVector Position1 = Position + Rotation.RotateVector(Scale * TaperedCylinder.GetX1());
		const FVector Position2 = Position + Rotation.RotateVector(Scale * TaperedCylinder.GetX2());
		DrawTaperedCylinder(PDI, Position1, Position2, Radius1, Radius2, 12, Color);
	}

	static void DrawExtrudedTaperedCapsule(FPrimitiveDrawInterface* PDI, const ::Chaos::FExtrudedTaperedCapsule& TaperedCapsule, const FQuat& Rotation, const FVector& Position, const FReal Scale, const FLinearColor& Color)
	{
		const float ScaledHalfLength = TaperedCapsule.GetHalfLength() * Scale;
		const float ScaledLength = 2.f * ScaledHalfLength;
		float ScaledRadius0 = TaperedCapsule.GetRadius1() * Scale;
		float ScaledRadius1 = TaperedCapsule.GetRadius2() * Scale;
		const float ScaledHalfWidth = TaperedCapsule.GetHalfWidth() * Scale;

		float SphereOffset0 = ScaledHalfLength;
		float SphereOffset1 = -ScaledHalfLength;
		if (FMath::IsNearlyZero(ScaledLength) || ScaledLength <= FMath::Abs(ScaledRadius0 - ScaledRadius1))
		{
			// Degenerate or one end encompasses the other - we need to draw a sphere, so map one end to the other
			if (ScaledRadius0 > ScaledRadius1)
			{
				SphereOffset1 = SphereOffset0;
				ScaledRadius1 = ScaledRadius0;
			}
			else
			{
				SphereOffset0 = SphereOffset1;
				ScaledRadius0 = ScaledRadius1;
			}
		}

		constexpr int32 NumNonExtrudedSides = 12;
		constexpr int32 NumSides = NumNonExtrudedSides + 2;
		constexpr int32 NumRings = 8;
		// The first/last arc are on top of each other.
		constexpr int32 NumVerts = (NumSides + 1) * (NumRings + 1);
		TArray<FVector3f> Verts, ArcVerts;
		Verts.SetNumUninitialized(NumVerts);
		ArcVerts.SetNumUninitialized(NumRings + 1);

		// Calc arc split point
		const float OffsetZ = FMath::IsNearlyZero(ScaledLength) ? 0.f : ((ScaledRadius0 - ScaledRadius1) * ScaledRadius0) / ScaledLength;
		const float SplitAngle = FMath::IsNearlyZero(ScaledRadius0) ? 90.f : 90.0f - FMath::RadiansToDegrees(FMath::Asin(OffsetZ / ScaledRadius0));
		const int32 SplitPoint = (int32)((float)NumRings * ((180.0f - SplitAngle) / 180.0f));
		for (int32 RingIdx = 0; RingIdx < NumRings + 1; RingIdx++)
		{
			// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
			FVector3f SpherePos;
			SpherePos.X = 0.0f;
			float ZOffset;

			if (RingIdx <= SplitPoint)
			{
				float Angle = ((float)RingIdx / (NumRings - 1)) * UE_PI;
				ZOffset = SphereOffset0;
				SpherePos.Y = ScaledRadius0 * FMath::Sin(Angle);
				SpherePos.Z = ScaledRadius0 * FMath::Cos(Angle);
			}
			else
			{
				float Angle = ((float)(RingIdx - 1) / (NumRings - 1)) * UE_PI;
				ZOffset = SphereOffset1;
				SpherePos.Y = ScaledRadius1 * FMath::Sin(Angle);
				SpherePos.Z = ScaledRadius1 * FMath::Cos(Angle);
			}

			ArcVerts[RingIdx] = SpherePos + FVector3f(0.f, 0.f, ZOffset);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 SideIdx = 0; SideIdx < NumNonExtrudedSides + 1; SideIdx++)
		{
			const FRotator3f ArcRotator(0, 360.f * ((float)SideIdx / NumNonExtrudedSides), 0);
			const FRotationMatrix44f ArcRot(ArcRotator);

			const FVector3f XOffset(SideIdx <= NumNonExtrudedSides / 2 ? -ScaledHalfWidth : ScaledHalfWidth, 0, 0);
			const int32 SideIdxToVIxOffset = (SideIdx > NumNonExtrudedSides / 2) ? 1 : 0;
			for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
			{
				const int32 VIx = (NumRings + 1) * (SideIdx + SideIdxToVIxOffset) + VertIdx;
				Verts[VIx] = ArcRot.TransformPosition(ArcVerts[VertIdx]) + XOffset;
			}

			if (SideIdx == NumNonExtrudedSides / 2)
			{
				// Duplicate vertex on second half
				for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
				{
					const int32 VIx = (NumRings + 1) * (SideIdx + 1) + VertIdx;
					Verts[VIx] = ArcRot.TransformPosition(ArcVerts[VertIdx]) + FVector3f(ScaledHalfWidth, 0.f, 0.f);
				}
			}
			else if (SideIdx == NumNonExtrudedSides)
			{
				// Duplicate vertex on second half
				for (int32 VertIdx = 0; VertIdx < NumRings + 1; VertIdx++)
				{
					const int32 VIx = (NumRings + 1) * (SideIdx + 2) + VertIdx;
					Verts[VIx] = ArcRot.TransformPosition(ArcVerts[VertIdx]) - FVector3f(ScaledHalfWidth, 0.f, 0.f);
				}
			}
		}
		FMatrix CapsuleTransform = FRotationMatrix::MakeFromZX(FVector(TaperedCapsule.GetMajorAxis()), FVector(TaperedCapsule.GetMinorAxis()));
		CapsuleTransform.SetOrigin(FVector(TaperedCapsule.GetOrigin()));
		FMatrix BoneTransform = Rotation.ToMatrix();
		BoneTransform.SetOrigin(Position);
		const FMatrix FullTransform = CapsuleTransform * BoneTransform;
		// Add all of the lines
		for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
		{
			const int32 Side0Start = (SideIdx + 0) * (NumRings + 1);
			const int32 Side1Start = (SideIdx + 1) * (NumRings + 1);

			for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++)
			{
				const FVector S0R0 = FullTransform.TransformPosition(FVector(Verts[Side0Start + RingIdx]));
				const FVector S0R1 = FullTransform.TransformPosition(FVector(Verts[Side0Start + RingIdx + 1]));
				const FVector S1R0 = FullTransform.TransformPosition(FVector(Verts[Side1Start + RingIdx]));
				DrawLine(PDI, S0R0, S1R0, Color);
				DrawLine(PDI, S0R0, S0R1, Color);

				if (RingIdx == NumRings - 1)
				{
					const FVector S1R1 = FullTransform.TransformPosition(FVector(Verts[Side1Start + RingIdx + 1]));
					DrawLine(PDI, S0R1, S1R1, Color);
				}
			}
		}
	}

	static void DrawConvex(FPrimitiveDrawInterface* PDI, const ::Chaos::FConvex& Convex, const FQuat& Rotation, const FVector& Position, const FReal Scale, const FLinearColor& Color)
	{
		const TArray<FConvex::FPlaneType>& Planes = Convex.GetFaces();
		for (int32 PlaneIndex1 = 0; PlaneIndex1 < Planes.Num(); ++PlaneIndex1)
		{
			const FConvex::FPlaneType& Plane1 = Planes[PlaneIndex1];

			for (int32 PlaneIndex2 = PlaneIndex1 + 1; PlaneIndex2 < Planes.Num(); ++PlaneIndex2)
			{
				const FConvex::FPlaneType& Plane2 = Planes[PlaneIndex2];

				// Find the two surface points that belong to both Plane1 and Plane2
				uint32 ParticleIndex1 = INDEX_NONE;

				const TArray<FConvex::FVec3Type>& Vertices = Convex.GetVertices();
				for (int32 ParticleIndex = 0; ParticleIndex < Vertices.Num(); ++ParticleIndex)
				{
					const FConvex::FVec3Type& X = Vertices[ParticleIndex];

					if (FMath::Square(Plane1.SignedDistance(X)) < KINDA_SMALL_NUMBER && 
						FMath::Square(Plane2.SignedDistance(X)) < KINDA_SMALL_NUMBER)
					{
						if (ParticleIndex1 != INDEX_NONE)
						{
							const FVector X1(Vertices[ParticleIndex1]);
							const FVector X2(X);
							const FVector Position1 = Position + Rotation.RotateVector(Scale * X1);
							const FVector Position2 = Position + Rotation.RotateVector(Scale * X2);
							DrawLine(PDI, Position1, Position2, Color);
							break;
						}
						ParticleIndex1 = ParticleIndex;
					}
				}
			}
		}
	}

	static void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, const FQuat& Rotation, const FVector& Position, const float LengthScale = 10.f, const float ColorScale = 1.f)
	{
		const FVector X = Rotation.RotateVector(FVector::ForwardVector) * LengthScale;
		const FVector Y = Rotation.RotateVector(FVector::RightVector) * LengthScale;
		const FVector Z = Rotation.RotateVector(FVector::UpVector) * LengthScale;

		DrawLine(PDI, Position, Position + X, FLinearColor::Red * ColorScale);
		DrawLine(PDI, Position, Position + Y, FLinearColor::Green * ColorScale);
		DrawLine(PDI, Position, Position + Z, FLinearColor::Blue * ColorScale);
	}

	static void DrawLevelSet(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FMaterialRenderProxy* MaterialRenderProxy, const FLevelSet& LevelSet)
	{
#if WITH_EDITOR
		if (PDI && MaterialRenderProxy)
		{
			TArray<FVector3f> Vertices;
			TArray<FIntVector> Tris;
			LevelSet.GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
			for (const FVector3f& V : Vertices)
			{
				MeshBuilder.AddVertex(FDynamicMeshVertex(V));
			}
			for (const FIntVector& T : Tris)
			{
				MeshBuilder.AddTriangle(T[0], T[1], T[2]);
			}

			MeshBuilder.Draw(PDI, Transform.ToMatrixWithScale(), MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
		}
		else
#endif
		{
			DrawCoordinateSystem(PDI, Transform.GetRotation(), Transform.GetTranslation());
		}
	}

	static void DrawSkinnedLevelSet(FPrimitiveDrawInterface* PDI, const TWeightedLatticeImplicitObject<FLevelSet>& SkinnedLevelSet, const FQuat& Rotation, const FVector& Position, const FReal Scale, const FMaterialRenderProxy* MaterialRenderProxy)
	{
#if WITH_EDITOR
		if (PDI && MaterialRenderProxy)
		{
			TArray<FVector3f> Vertices;
			TArray<FIntVector> Tris;
			const FLevelSet* const LevelSet = SkinnedLevelSet.GetEmbeddedObject();
			LevelSet->GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
			for (const FVector3f& V : Vertices)
			{
				MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(SkinnedLevelSet.GetDeformedPoint(FVec3(V)))));
			}
			for (const FIntVector& T : Tris)
			{
				MeshBuilder.AddTriangle(T[0], T[1], T[2]);
			}

			const FTransform LocalToWorld(Rotation, Position, FVector(Scale));
			MeshBuilder.Draw(PDI, LocalToWorld.ToMatrixWithScale(), MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);

			if (Private::DrawSkinnedLattice)
			{
				const Chaos::TUniformGrid<double, 3>& LatticeGrid = SkinnedLevelSet.GetGrid();
				const Chaos::TArrayND<Chaos::FVec3, 3>& DeformedPoints = SkinnedLevelSet.GetDeformedPoints();
				const Chaos::TArrayND<bool, 3>& EmptyCells = SkinnedLevelSet.GetEmptyCells();
				const FColor LatticeColor = FColor::Cyan;
				const FColor EmptyLatticeColor = FColor::White;
				const Chaos::TVec3<int32> CellCounts = LatticeGrid.Counts();

				for (int32 I = 0; I < CellCounts.X; ++I)
				{
					for (int32 J = 0; J < CellCounts.Y; ++J)
					{
						for (int32 K = 0; K < CellCounts.Z; ++K)
						{
							const bool bIsEmpty = EmptyCells(I, J, K);
							const int32 EmptyDrawMask = bIsEmpty ? 2 : 1;
							if (EmptyDrawMask & Private::DrawSkinnedLattice)
							{
								const FVector P000 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J, K)));
								const FVector P001 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J, K + 1)));
								const FVector P010 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J + 1, K)));
								const FVector P011 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I, J + 1, K + 1)));
								const FVector P100 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J, K)));
								const FVector P101 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J, K + 1)));
								const FVector P110 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J + 1, K)));
								const FVector P111 = LocalToWorld.TransformPosition(FVector(DeformedPoints(I + 1, J + 1, K + 1)));

								PDI->AddReserveLines(Chaos::Private::GetDepthPriority(), 12);
								const FColor& Color = bIsEmpty ? EmptyLatticeColor : LatticeColor;
								PDI->DrawLine(P000, P001, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P000, P010, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P000, P100, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P001, P011, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P001, P101, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P010, P011, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P010, P110, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P011, P111, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P100, P101, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P100, P110, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P101, P111, Color, Chaos::Private::GetDepthPriority());
								PDI->DrawLine(P110, P111, Color, Chaos::Private::GetDepthPriority());

							}
						}
					}
				}
			}
		}
		else
#endif
		{
			DrawCoordinateSystem(PDI, Rotation, Position);
		}
	}

	static void DrawMLLevelSet(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FMaterialRenderProxy* MaterialRenderProxy, const FMLLevelSet& MLLevelSet)
	{
#if WITH_EDITOR
		if (PDI && MaterialRenderProxy)
		{
			//Draw MLLevelSet Isosurface (this takes significant amount of time. Only use for debugging)
			if (Private::DrawMLLevelSetFlag%2 == 1)
			{
				TArray<FVector3f> Vertices;
				TArray<FIntVector> Tris;
				MLLevelSet.GetZeroIsosurfaceGridCellFaces(Vertices, Tris);

				FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
				for (const FVector3f& V : Vertices)
				{
					MeshBuilder.AddVertex(FDynamicMeshVertex(V));
				}
				for (const FIntVector& T : Tris)
				{
					MeshBuilder.AddTriangle(T[0], T[1], T[2]);
				}

				MeshBuilder.Draw(PDI, Transform.ToMatrixWithScale(), MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
			}

			// Draw the Local grid that MLLevelSet lives in.
			if (Private::DrawMLLevelSetFlag == 2 || Private::DrawMLLevelSetFlag == 3)
			{
				TArray<FVector> LocalGridCorners;
				LocalGridCorners.SetNum(8);
				TUniformGrid<FReal, 3> MGrid = MLLevelSet.GetGrid();
				TArray<FVector> MinMaxCorners;
				MinMaxCorners.SetNum(2);
				MinMaxCorners[0] = MGrid.MinCorner();
				MinMaxCorners[1] = MGrid.MaxCorner();

				for (int32 i1 = 0; i1 < 2; i1++)
				{
					for (int32 i2 = 0; i2 < 2; i2++)
					{
						for (int32 i3 = 0; i3 < 2; i3++)
						{
							int32 i = 4 * i1 + 2 * i2 + i3;
							LocalGridCorners[i][0] = MinMaxCorners[i1][0];
							LocalGridCorners[i][1] = MinMaxCorners[i2][1];
							LocalGridCorners[i][2] = MinMaxCorners[i3][2];
						}
					}
				}

				const FVector P000 = Transform.TransformPosition(LocalGridCorners[0]);
				const FVector P001 = Transform.TransformPosition(LocalGridCorners[1]);
				const FVector P010 = Transform.TransformPosition(LocalGridCorners[2]);
				const FVector P011 = Transform.TransformPosition(LocalGridCorners[3]);
				const FVector P100 = Transform.TransformPosition(LocalGridCorners[4]);
				const FVector P101 = Transform.TransformPosition(LocalGridCorners[5]);
				const FVector P110 = Transform.TransformPosition(LocalGridCorners[6]);
				const FVector P111 = Transform.TransformPosition(LocalGridCorners[7]);

				PDI->AddReserveLines(Chaos::Private::GetDepthPriority(), 12);
				const FColor& Color = FColor::Purple;
				const FColor& ColorZ = FColor::Blue;
				const FColor& ColorY = FColor::Green;
				const FColor& ColorX = FColor::Red;
				PDI->DrawLine(P000, P001, ColorZ, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P000, P010, ColorY, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P000, P100, ColorX, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P001, P011, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P001, P101, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P010, P011, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P010, P110, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P011, P111, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P100, P101, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P100, P110, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P101, P111, Color, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P110, P111, Color, Chaos::Private::GetDepthPriority());
			}

			//Draw the training grid.
			if (Private::DrawMLLevelSetFlag == 4 || Private::DrawMLLevelSetFlag == 5)
			{
				FVector3f LocalGridCorner = MLLevelSet.GetTrainingGridMin();

				TArray<FVector3f> CornersXYZ;
				CornersXYZ.SetNum(3);
				for (int32 i = 0; i < 3; i++)
				{
					CornersXYZ[i] = LocalGridCorner + MLLevelSet.GetTrainingGridVector(i);
				}

				const FVector P000 = Transform.TransformPosition((FVector)LocalGridCorner);
				const FVector P001 = Transform.TransformPosition((FVector)CornersXYZ[2]);
				const FVector P010 = Transform.TransformPosition((FVector)CornersXYZ[1]);
				const FVector P100 = Transform.TransformPosition((FVector)CornersXYZ[0]);

				PDI->AddReserveLines(Chaos::Private::GetDepthPriority(), 3);
				const FColor& Color = FColor::Purple;
				const FColor& ColorZ = FColor::Blue;
				const FColor& ColorY = FColor::Green;
				const FColor& ColorX = FColor::Red;
				PDI->DrawLine(P000, P001, ColorZ, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P000, P010, ColorY, Chaos::Private::GetDepthPriority());
				PDI->DrawLine(P000, P100, ColorX, Chaos::Private::GetDepthPriority());
			}
		}
		else
#endif
		{
			DrawCoordinateSystem(PDI, Transform.GetRotation(), Transform.GetTranslation());
		}
	}

	static void DrawSkinnedTriangleMesh(FPrimitiveDrawInterface* PDI, const FMaterialRenderProxy* MaterialRenderProxy, const FClothingSimulationSolver* Solver, const FLinearColor& Color, const FSkinnedTriangleMesh& Mesh)
	{
#if WITH_EDITOR
		if (PDI && MaterialRenderProxy)
		{
			const FColor ColorRGB = Color.ToFColorSRGB();
			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
			const TArray<Softs::FSolverVec3>& Positions = Mesh.GetLocalPositions();
			for (const Softs::FSolverVec3& Pos : Positions)
			{
				MeshBuilder.AddVertex(FDynamicMeshVertex(FVector3f(Pos), FVector2f(0.f), ColorRGB));
			}
			const TArray<TVec3<int32>>& Elements = Mesh.GetTriangleMesh().GetElements();
			for (const TVec3<int32>& Element : Elements)
			{
				MeshBuilder.AddTriangle(Element[0], Element[1], Element[2]);
			}

			const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(*Solver);
			MeshBuilder.Draw(PDI, LocalSimSpaceToWorld, MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
		}
		else
#endif
		{
			const TArray<Softs::FSolverVec3>& Positions = Mesh.GetLocalPositions();
			const TArray<TVec3<int32>>& Elements = Mesh.GetTriangleMesh().GetElements();
			for (const TVec3<int32>& Element : Elements)
			{
				const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X]);
				const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y]);
				const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z]);

				DrawLine(PDI, Pos0, Pos1, Color);
				DrawLine(PDI, Pos1, Pos2, Color);
				DrawLine(PDI, Pos2, Pos0, Color);
			}
		}
	}

	void FClothVisualizationNoGC::DrawBounds(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		// Calculate World space bounds
		const FBoxSphereBounds Bounds = Solver->CalculateBounds();

		// Draw bounds
		DrawBox(Context.PDI, FAABB3(-Bounds.BoxExtent, Bounds.BoxExtent), FQuat::Identity, Bounds.Origin, 1., FLinearColor(FColor::Purple));
		DrawSphere(Context.PDI, FSphere(FVector::ZeroVector, Bounds.SphereRadius), FQuat::Identity, Bounds.Origin, 1., FLinearColor(FColor::Orange));

		// Draw individual cloth bounds
		static const FLinearColor Color = FLinearColor(FColor::Purple).Desaturate(0.5);
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FAABB3 BoundingBox = Cloth->CalculateBoundingBox(Solver);
			DrawBox(Context.PDI, BoundingBox, FQuat::Identity, FVector::ZeroVector, 1.f, Color);  // TODO: Express bounds in local coordinates for LWC
		}
	}

	void FClothVisualizationNoGC::DrawGravity(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		// Draw gravity
		constexpr FReal GravityVectorLengthMultiplier = 0.01; // Make the vector smaller
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FExternalForces* const ExternalForces = ClothConstraints.GetExternalForces().Get())
			{
				check(!Solver->IsLegacySolver());
				if (ExternalForces->HasPerParticleGravity())
				{
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
					auto ShouldDrawParticle = [&Context, &DebugFilterSet, GlobalParticleOffset](int32 ParticleIndex)
						{
							return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, ParticleIndex, GlobalParticleOffset);
						};
					const TConstArrayView<Softs::FSolverVec3> Positions = Solver->GetParticleXsView(ParticleRangeId);
					for (int32 ParticleIndex = 0; ParticleIndex < Positions.Num(); ++ParticleIndex)
					{
						if (ShouldDrawParticle(ParticleIndex))
						{
							const FVector Pos0 = GetWorldPosition(*Solver, Positions[ParticleIndex]);
							const FVector Pos1 = Pos0 + GravityVectorLengthMultiplier * GetWorldVector(*Solver, ExternalForces->GetScaledGravity(ParticleIndex));
							DrawArrow(Context.PDI, Pos0, Pos1, FLinearColor::Red);
						}
					}
				}
				else
				{
					const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
					const FVector Pos0 = ReferenceSpaceTransform.GetLocation();
					const FVector Pos1 = Pos0 + GravityVectorLengthMultiplier * GetWorldVector(*Solver, ExternalForces->GetScaledGravity(0));
					DrawArrow(Context.PDI, Pos0, Pos1, FLinearColor::Red);
				}
			}
			else
			{
				const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
				const FVector Pos0 = ReferenceSpaceTransform.GetLocation();
				const FVector Pos1 = Pos0 + GravityVectorLengthMultiplier * GetWorldVector(*Solver, Cloth->GetGravity(Solver));
				DrawArrow(Context.PDI, Pos0, Pos1, FLinearColor::Red);
			}
		}
	}


	void FClothVisualizationNoGC::DrawFictitiousAngularForces(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FExternalForces* const ExternalForces = ClothConstraints.GetExternalForces().Get())
			{
				check(!Solver->IsLegacySolver());
				{
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
					auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
						{
							return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
						};

					const TConstArrayView<Softs::FSolverVec3> Positions = Solver->GetParticleXsView(ParticleRangeId);
					const Softs::FSolverVec3& FictitousAngularVelocity = ExternalForces->GetFictitiousAngularVelocity();
					const Softs::FSolverVec3& ReferenceSpaceLocation = ExternalForces->GetReferenceSpaceLocation();

					for (int32 ParticleIndex = 0; ParticleIndex < Positions.Num(); ++ParticleIndex)
					{
						if (ShouldDrawParticle(ParticleIndex))
						{
							const FVector Pos0 = GetWorldPosition(*Solver, Positions[ParticleIndex]);
							const Softs::FSolverVec3 CentrifugalAccel = -Softs::FSolverVec3::CrossProduct(FictitousAngularVelocity, Softs::FSolverVec3::CrossProduct(FictitousAngularVelocity, Positions[ParticleIndex] - ReferenceSpaceLocation));

							const FVector Pos1 = Pos0 + GetWorldVector(*Solver, CentrifugalAccel);
							DrawLine(Context.PDI, Pos0, Pos1, FLinearColor::Red);
						}
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawPhysMeshWired(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor DynamicColor = FColor::White;
		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};
			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(InvMasses.Num() == Positions.Num());

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];

				if (ShouldDrawElement(Element))
				{
					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
					const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

					const bool bIsKinematic0 = (InvMasses[Element.X - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic1 = (InvMasses[Element.Y - Offset] == (Softs::FSolverReal)0.);
					const bool bIsKinematic2 = (InvMasses[Element.Z - Offset] == (Softs::FSolverReal)0.);

					DrawLine(Context.PDI, Pos0, Pos1, bIsKinematic0 && bIsKinematic1 ? KinematicColor : DynamicColor);
					DrawLine(Context.PDI, Pos1, Pos2, bIsKinematic1 && bIsKinematic2 ? KinematicColor : DynamicColor);
					DrawLine(Context.PDI, Pos2, Pos0, bIsKinematic2 && bIsKinematic0 ? KinematicColor : DynamicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawAnimMeshWired(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};

			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];

				if (ShouldDrawElement(Element))
				{

					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
					const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

					DrawLine(Context.PDI, Pos0, Pos1, KinematicColor);
					DrawLine(Context.PDI, Pos1, Pos2, KinematicColor);
					DrawLine(Context.PDI, Pos2, Pos0, KinematicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawAccessoryMesh(const FDrawContext& Context, const FName& AccessoryMeshName) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor AccessoryColor = FColor::Turquoise;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			if (!Cloth->AccessoryMeshExists(Solver, AccessoryMeshName))
			{
				continue;
			}

			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, &DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};

			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAccessoryMeshPositions(Solver, AccessoryMeshName);

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];
				if (ShouldDrawElement(Element))
				{

					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
					const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

					DrawLine(Context.PDI, Pos0, Pos1, AccessoryColor);
					DrawLine(Context.PDI, Pos1, Pos2, AccessoryColor);
					DrawLine(Context.PDI, Pos2, Pos0, AccessoryColor);
				}
			}
		}
	}


	void FClothVisualizationNoGC::DrawAccessoryMeshNormals(const FDrawContext& Context, const FName& AccessoryMeshName, const FReal NormalLength) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor AccessoryColor = FColor::Turquoise;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			if (!Cloth->AccessoryMeshExists(Solver, AccessoryMeshName))
			{
				continue;
			}

			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAccessoryMeshPositions(Solver, AccessoryMeshName);
			const TConstArrayView<Softs::FSolverVec3> Normals = Cloth->GetAccessoryMeshNormals(Solver, AccessoryMeshName);
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
				};
			check(Normals.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Index]);
					const FVector Pos1 = Pos0 + GetWorldVector(*Solver, Normals[Index]) * NormalLength;

					DrawLine(Context.PDI, Pos0, Pos1, AccessoryColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawOpenEdges(const FDrawContext& Context) const
	{
		auto MakeSortedUintVector2 = [](uint32 Index0, uint32 Index1) -> FUintVector2
			{
				return Index0 < Index1 ? FUintVector2(Index0, Index1) : FUintVector2(Index1, Index0);
			};

		auto BuildEdgeMap = [&MakeSortedUintVector2](const TConstArrayView<TVec3<int32>>& Elements, TMap<FUintVector2, TArray<uint32>>& OutEdgeToTrianglesMap)
			{
			OutEdgeToTrianglesMap.Empty(Elements.Num() * 2);  // Rough estimate for the number of edges

				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const TVec3<int32>& Element = Elements[ElementIndex];
					const uint32 Index0 = Element[0];
					const uint32 Index1 = Element[1];
					const uint32 Index2 = Element[2];

					const FUintVector2 Edge0 = MakeSortedUintVector2(Index0, Index1);
					const FUintVector2 Edge1 = MakeSortedUintVector2(Index1, Index2);
					const FUintVector2 Edge2 = MakeSortedUintVector2(Index2, Index0);

					OutEdgeToTrianglesMap.FindOrAdd(Edge0).Add(ElementIndex);
					OutEdgeToTrianglesMap.FindOrAdd(Edge1).Add(ElementIndex);
					OutEdgeToTrianglesMap.FindOrAdd(Edge2).Add(ElementIndex);
				}
			};

		if (!Solver)
		{
			return;
		}

		static const FLinearColor OpenedEdgeColor = FColor::Emerald;
		static const FLinearColor ClosedEdgeColor = FColor::White;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawEdge = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const  FUintVector2& Edge)
				{
					return Private::ShouldDrawEdge(Context.Filter, DebugFilterSet, Edge, GlobalParticleOffset - Offset, -Offset);
				};

			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);

			TMap<FUintVector2, TArray<uint32>> EdgeToTrianglesMap;
			BuildEdgeMap(Elements, EdgeToTrianglesMap);

			for (const TPair<FUintVector2, TArray<uint32>>& EdgeToTriangles : EdgeToTrianglesMap)
			{
				const FUintVector2& Edge = EdgeToTriangles.Key;
				if (ShouldDrawEdge(Edge))
				{
					const TArray<uint32>& Triangles = EdgeToTriangles.Value;

					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Edge[0] - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, Positions[Edge[1] - Offset]);
					const FLinearColor& Color = (Triangles.Num() > 1) ? ClosedEdgeColor : OpenedEdgeColor;

					DrawLine(Context.PDI, Pos0, Pos1, Color);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawAnimNormals(const FDrawContext& Context, const FReal Length) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor KinematicColor = FColor::Magenta;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
				};
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);
			const TConstArrayView<Softs::FSolverVec3> Normals = Cloth->GetAnimationNormals(Solver);
			check(Normals.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Index]);
					const FVector Pos1 = Pos0 + GetWorldVector(*Solver, Normals[Index]) * Length;

					DrawLine(Context.PDI, Pos0, Pos1, KinematicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawAnimVelocities(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor KinematicColor = FLinearColor::Black;
		static const FLinearColor KinematicMeshColor = (FLinearColor::Black + FLinearColor(FColor::Purple)) * .5;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, &DebugFilterSet, GlobalParticleOffset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
				};
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);
			const TConstArrayView<Softs::FSolverVec3> OldPositions = Cloth->GetOldAnimationPositions(Solver);
			checkSlow(OldPositions.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Index]);
					const FVector Pos1 = GetWorldPosition(*Solver, OldPositions[Index]);
					DrawLine(Context.PDI, Pos0, Pos1, KinematicColor);
				}
			}

			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};
			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];
				if (ShouldDrawElement(Element))
				{

					const FVector Pos0 = GetWorldPosition(*Solver, OldPositions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, OldPositions[Element.Y - Offset]);
					const FVector Pos2 = GetWorldPosition(*Solver, OldPositions[Element.Z - Offset]);

					DrawLine(Context.PDI, Pos0, Pos1, KinematicMeshColor);
					DrawLine(Context.PDI, Pos1, Pos2, KinematicMeshColor);
					DrawLine(Context.PDI, Pos2, Pos0, KinematicMeshColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawPointNormals(const FDrawContext& Context, const FReal NormalLength) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor DynamicColor = FColor::White;
		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverVec3> Normals = Cloth->GetParticleNormals(Solver);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
				};
			check(Normals.Num() == Positions.Num());
			check(InvMasses.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const bool bIsKinematic = (InvMasses[Index] == (Softs::FSolverReal)0.);
					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Index]);
					const FVector Pos1 = Pos0 + GetWorldVector(*Solver, Normals[Index]) * NormalLength;

					DrawLine(Context.PDI, Pos0, Pos1, bIsKinematic ? KinematicColor : DynamicColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawPointVelocities(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
			const TConstArrayView<Softs::FSolverVec3> Velocities = Cloth->GetParticleVelocities(Solver);
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
				};
			check(Velocities.Num() == Positions.Num());

			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			check(InvMasses.Num() == Positions.Num());

			for (int32 Index = 0; Index < Positions.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					constexpr FReal DefaultFPS = (FReal)60.;   // TODO: A CVAR would be nice for this
					const bool bIsKinematic = (InvMasses[Index] == 0.f);

					const FVec3 Pos0 = GetWorldPosition(*Solver, Positions[Index]);
					const FVec3 Pos1 = Pos0 + GetWorldVector(*Solver, Velocities[Index]) / DefaultFPS;  // Velocity per frame if running at DefaultFPS

					DrawLine(Context.PDI, Pos0, Pos1, bIsKinematic ? FLinearColor::Black : FLinearColor::Yellow);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawCollision(const FDrawContext& Context, bool bWireframe) const
	{
		if (!Solver)
		{
			return;
		}

		auto DrawCollision =
			[this, &Context, bWireframe](const FClothingSimulationCollider* Collider, const FClothingSimulationCloth* Cloth, FClothingSimulationCollider::ECollisionDataType CollisionDataType)
			{
				static const FLinearColor GlobalColor(FColor::Cyan);
				static const FLinearColor DynamicColor(FColor::Orange);
				static const FLinearColor LODsColor(FColor::Silver);
				static const FLinearColor CollidedColor(FColor::Red);

				const FLinearColor TypeColor =
					(CollisionDataType == FClothingSimulationCollider::ECollisionDataType::LODless) ? GlobalColor :
					(CollisionDataType == FClothingSimulationCollider::ECollisionDataType::External) ? DynamicColor : LODsColor;

				const TConstArrayView<FImplicitObjectPtr> CollisionGeometries = Collider->GetCollisionGeometry(Solver, Cloth, CollisionDataType);
				const TConstArrayView<Softs::FSolverVec3> Translations = Collider->GetCollisionTranslations(Solver, Cloth, CollisionDataType);
				const TConstArrayView<Softs::FSolverRotation3> Rotations = Collider->GetCollisionRotations(Solver, Cloth, CollisionDataType);
				const TConstArrayView<bool> CollisionStatus = Collider->GetCollisionStatus(Solver, Cloth, CollisionDataType);
				check(CollisionGeometries.Num() == Translations.Num());
				check(CollisionGeometries.Num() == Rotations.Num());

				for (int32 Index = 0; Index < CollisionGeometries.Num(); ++Index)
				{
					if (const FImplicitObject* const Object = CollisionGeometries[Index].GetReference())
					{
						const FLinearColor Color = CollisionStatus[Index] ? CollidedColor : TypeColor;
						const FVec3 Position = GetWorldPosition(*Solver, Translations[Index]);
						const FRotation3 Rotation(Rotations[Index]);

						switch (Object->GetType())
						{
						case ImplicitObjectType::Sphere:
							DrawSphere(Context.PDI, Object->GetObjectChecked<FSphere>(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
							break;

						case ImplicitObjectType::Box:
							DrawBox(Context.PDI, Object->GetObjectChecked<TBox<FReal, 3>>().BoundingBox(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
							break;

						case ImplicitObjectType::Capsule:
							DrawCapsule(Context.PDI, Object->GetObjectChecked<FCapsule>(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
							break;

						case ImplicitObjectType::Union:  // Union only used as old style tapered capsules
							for (const FImplicitObjectPtr& SubObjectPtr : Object->GetObjectChecked<FImplicitObjectUnion>().GetObjects())
							{
								if (const FImplicitObject* const SubObject = SubObjectPtr.GetReference())
								{
									switch (SubObject->GetType())
									{
									case ImplicitObjectType::Sphere:
										DrawSphere(Context.PDI, SubObject->GetObjectChecked<FSphere>(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
										break;

									case ImplicitObjectType::TaperedCylinder:
										DrawTaperedCylinder(Context.PDI, SubObject->GetObjectChecked<FTaperedCylinder>(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
										break;

									default:
										break;
									}
								}
							}
							break;

						case ImplicitObjectType::TaperedCapsule:  // New collision tapered capsules implicit type that replaces the union
							{
								const FTaperedCapsule& TaperedCapsule = Object->GetObjectChecked<FTaperedCapsule>();
								const FVec3 X1 = TaperedCapsule.GetX1();
								const FVec3 X2 = TaperedCapsule.GetX2();
								const FReal Radius1 = TaperedCapsule.GetRadius1();
								const FReal Radius2 = TaperedCapsule.GetRadius2();
								DrawSphere(Context.PDI, FSphere(X1, Radius1), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
								DrawSphere(Context.PDI, FSphere(X2, Radius2), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
								DrawTaperedCylinder(Context.PDI, FTaperedCylinder(X1, X2, Radius1, Radius2), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
								if (TaperedCapsule.IsOneSided())
								{
									const FVec3 Center = Position + Rotation.RotateVector(Solver->GetLocalSpaceScale() * .5 * (X1 + X2));
									const FVec3 NormalEnd = Center + Rotation.RotateVector(Solver->GetLocalSpaceScale() * .5 * (Radius1 + Radius2) * TaperedCapsule.GetOneSidedPlaneNormal());
									DrawLine(Context.PDI, Center, NormalEnd, Color);
								}
							}
							break;

						case ImplicitObjectType::Convex:
							DrawConvex(Context.PDI, Object->GetObjectChecked<FConvex>(), Rotation, Position, Solver->GetLocalSpaceScale(), Color);
							break;

						case ImplicitObjectType::Transformed: // Transformed only used for levelsets
							if (Object->GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>().GetGeometry()->GetType() == ImplicitObjectType::LevelSet)
							{
								const TRigidTransform<FReal, 3>& Transform = Object->GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>().GetTransform();
								const FTransform CombinedTransform = Transform * FTransform(Rotation, Position, FVector(Solver->GetLocalSpaceScale()));
								const FLevelSet& LevelSet = Object->GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>().GetGeometry()->GetObjectChecked<FLevelSet>();
								const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
									FMaterials::GetInstance().GetCollisionMaterial() ? FMaterials::GetInstance().GetCollisionMaterial()->GetRenderProxy() : nullptr;
#else
									nullptr;
#endif
								DrawLevelSet(Context.PDI, CombinedTransform, MaterialRenderProxy, LevelSet);
							}
							break;
						case (ImplicitObjectType::LevelSet | ImplicitObjectType::IsWeightedLattice):
							{
								const TWeightedLatticeImplicitObject<FLevelSet>& WeightedLevelset = Object->GetObjectChecked< TWeightedLatticeImplicitObject<FLevelSet> >();
								const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
									FMaterials::GetInstance().GetCollisionMaterial() ? FMaterials::GetInstance().GetCollisionMaterial()->GetRenderProxy() : nullptr;
#else
									nullptr;
#endif
								DrawSkinnedLevelSet(Context.PDI, WeightedLevelset, Rotation, Position, Solver->GetLocalSpaceScale(), MaterialRenderProxy);

							}
							break;
						case (ImplicitObjectType::MLLevelSet):
							{
								const FTransform CombinedTransform = FTransform(Rotation, Position);
								const FMLLevelSet& MLLevelSet = Object->GetObjectChecked<FMLLevelSet>();
								const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
									FMaterials::GetInstance().GetCollisionMaterial() ? FMaterials::GetInstance().GetCollisionMaterial()->GetRenderProxy() : nullptr;
#else
									nullptr;
#endif
								DrawMLLevelSet(Context.PDI, CombinedTransform, MaterialRenderProxy, MLLevelSet);
							}
							break;
						case ImplicitObjectType::SkinnedTriangleMesh:
						{
							const FSkinnedTriangleMesh& Mesh = Object->GetObjectChecked<FSkinnedTriangleMesh>();
							const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
								!bWireframe && FMaterials::GetInstance().GetClothMaterialColor() ? FMaterials::GetInstance().GetClothMaterialColor()->GetRenderProxy() : nullptr;
#else
								nullptr;
#endif
							DrawSkinnedTriangleMesh(Context.PDI, MaterialRenderProxy, Solver, Color, Mesh);

						}
						break;
						case ImplicitObjectType::ExtrudedTaperedCapsule:
						{
							const FExtrudedTaperedCapsule& Capsule = Object->GetObjectChecked<FExtrudedTaperedCapsule>();
							DrawExtrudedTaperedCapsule(Context.PDI, Capsule, Rotation, Position, Solver->GetLocalSpaceScale(), Color);
						}
						break;
						default:
							DrawCoordinateSystem(Context.PDI, Rotation, Position);  // Draw everything else as a coordinate for now
							break;
						}
					}
				}
			};

		// Draw collisions
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			for (const FClothingSimulationCollider* const Collider : Cloth->GetColliders())
			{
				DrawCollision(Collider, Cloth, FClothingSimulationCollider::ECollisionDataType::LODless);
				DrawCollision(Collider, Cloth, FClothingSimulationCollider::ECollisionDataType::External);
				DrawCollision(Collider, Cloth, FClothingSimulationCollider::ECollisionDataType::LODs);
			}
		}

		// Draw contacts
		check(Solver->GetCollisionContacts().Num() == Solver->GetCollisionNormals().Num());
		const bool bDrawPhis = Solver->GetCollisionContacts().Num() == Solver->GetCollisionPhis().Num();
		constexpr FReal NormalLength = (FReal)10.;

		for (int32 i = 0; i < Solver->GetCollisionContacts().Num(); ++i)
		{
			const FVec3 Pos0 = GetWorldPosition(*Solver, Solver->GetCollisionContacts()[i]);
			const FVec3 Normal = GetWorldVector(*Solver, Solver->GetCollisionNormals()[i]);

			// Draw contact
			FVec3 TangentU, TangentV;
			Normal.FindBestAxisVectors(TangentU, TangentV);

			DrawLine(Context.PDI, Pos0 + TangentU, Pos0 + TangentV, FLinearColor::Black);
			DrawLine(Context.PDI, Pos0 + TangentU, Pos0 - TangentV, FLinearColor::Black);
			DrawLine(Context.PDI, Pos0 - TangentU, Pos0 - TangentV, FLinearColor::Black);
			DrawLine(Context.PDI, Pos0 - TangentU, Pos0 + TangentV, FLinearColor::Black);

			// Draw normal
			static const FLinearColor Brown(0.1f, 0.05f, 0.f);
			const FVec3 Pos1 = Pos0 + NormalLength * Normal;
			DrawLine(Context.PDI, Pos0, Pos1, Brown);

			if (bDrawPhis)
			{
				const FVec3 PhiLocation = Pos0 - Solver->GetCollisionPhis()[i] * Normal;
				DrawLine(Context.PDI, Pos0, PhiLocation, Brown);
				DrawPoint(Context.PDI, PhiLocation, FLinearColor::Red, nullptr, 5);
			}
		}
	}

	void FClothVisualizationNoGC::DrawBackstops(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		auto DrawBackstop = [&Context](const FVector& Position, const FVector& Normal, FReal Radius, const FVector& Axis, const FLinearColor& Color)
			{
				static const FReal MaxCosAngle = (FReal)0.99;
				if (FMath::Abs(FVector::DotProduct(Normal, Axis)) < MaxCosAngle)
				{
					static const FReal ArcLength = (FReal)5.; // Arch length in cm
					const FReal ArcAngle = (FReal)360. * ArcLength / FMath::Max((Radius * (FReal)2. * (FReal)PI), ArcLength);
					DrawArc(Context.PDI, Position, Normal, FVector::CrossProduct(Axis, Normal).GetSafeNormal(), -ArcAngle / (FReal)2., ArcAngle / (FReal)2., Radius, Color);
				}
			};

		uint8 ColorSeed = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDSphericalBackstopConstraint* const BackstopConstraint = ClothConstraints.GetBackstopConstraints().Get())
			{
				const bool bUseLegacyBackstop = BackstopConstraint->UseLegacyBackstop();
				const FName BackstopMeshName = FName(Cloth->GetPropertyString(Solver, Softs::FPBDSphericalBackstopConstraint::BackstopMeshNameName));
				const bool AccessoryMeshExists = Cloth->AccessoryMeshExists(Solver, BackstopMeshName);

				const TConstArrayView<Softs::FSolverVec3> AnimationPositions = AccessoryMeshExists ? Cloth->GetAccessoryMeshPositions(Solver, BackstopMeshName) : Cloth->GetAnimationPositions(Solver);
				const TConstArrayView<Softs::FSolverVec3> AnimationNormals = AccessoryMeshExists ? Cloth->GetAccessoryMeshNormals(Solver, BackstopMeshName) : Cloth->GetAnimationNormals(Solver);
				const TConstArrayView<Softs::FSolverVec3> ParticlePositions = Cloth->GetParticlePositions(Solver);
				const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);

				if (Context.Filter.SingleVertex != INDEX_NONE && AnimationPositions.IsValidIndex(Context.Filter.SingleVertex-Offset))
				{
					const FLinearColor ColorLight = FLinearColor::MakeFromHSV8(ColorSeed, 160, 128);
					const FLinearColor ColorDark = FLinearColor::MakeFromHSV8(ColorSeed, 160, 64);
					const int32 BackstopVertexIndex = Context.Filter.SingleVertex - Offset;

					const FReal BackstopRadius = BackstopConstraint->GetBackstopRadius(BackstopVertexIndex) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();
					const FReal BackstopDistance = BackstopConstraint->GetBackstopDistance(BackstopVertexIndex) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();

					const FVector AnimationNormal(AnimationNormals[BackstopVertexIndex]);

					// Draw a line to show the current distance to the sphere
					const FVector Pos0 = GetWorldPosition(*Solver, AnimationPositions[BackstopVertexIndex]);
					const FVector Pos1 = Pos0 - (bUseLegacyBackstop ? BackstopDistance - BackstopRadius : BackstopDistance) * AnimationNormal;
					const FVector Pos2 = GetWorldPosition(*Solver, ParticlePositions[BackstopVertexIndex]);
					DrawPoint(Context.PDI, Pos2, ColorLight, nullptr, 5.f);
					DrawLine(Context.PDI, Pos1, Pos2, ColorLight);

					// Draw the sphere
					if (BackstopRadius > 0.f)
					{
						const FVector Center = Pos0 - (bUseLegacyBackstop ? BackstopDistance : BackstopRadius + BackstopDistance) * AnimationNormal;
						DrawLine(Context.PDI, Pos1, Center, ColorDark);
						const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
							FMaterials::GetInstance().GetCollisionMaterial() ? FMaterials::GetInstance().GetCollisionMaterial()->GetRenderProxy() : nullptr;
#else
							nullptr;
#endif
						DrawSphere(Context.PDI, FSphere(FVector::ZeroVector, BackstopRadius), FQuat::Identity, Center, 1.f, ColorDark, MaterialRenderProxy);
					}
				}
				else
				{
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					auto ShouldDrawParticle = [&Context, &DebugFilterSet, Offset](const int32 Index)
						{
							return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
						};
					for (int32 Index = 0; Index < AnimationPositions.Num(); ++Index)
					{
						if (ShouldDrawParticle(Index))
						{
							ColorSeed += 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
							const FLinearColor ColorLight = FLinearColor::MakeFromHSV8(ColorSeed, 160, 128);
							const FLinearColor ColorDark = FLinearColor::MakeFromHSV8(ColorSeed, 160, 64);

							const FReal BackstopRadius = BackstopConstraint->GetBackstopRadius(Index) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();
							const FReal BackstopDistance = BackstopConstraint->GetBackstopDistance(Index) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();

							const FVector AnimationNormal(AnimationNormals[Index]);

							// Draw a line to show the current distance to the sphere
							const FVector Pos0 = GetWorldPosition(*Solver, AnimationPositions[Index]);
							const FVector Pos1 = Pos0 - (bUseLegacyBackstop ? BackstopDistance - BackstopRadius : BackstopDistance) * AnimationNormal;
							const FVector Pos2 = GetWorldPosition(*Solver, ParticlePositions[Index]);
							DrawLine(Context.PDI, Pos1, Pos2, ColorLight);

							// Draw the sphere
							if (BackstopRadius > 0.f)
							{
								const FVector Center = Pos0 - (bUseLegacyBackstop ? BackstopDistance : BackstopRadius + BackstopDistance) * AnimationNormal;
								DrawBackstop(Center, AnimationNormal, BackstopRadius, FVector::ForwardVector, ColorDark);
								DrawBackstop(Center, AnimationNormal, BackstopRadius, FVector::UpVector, ColorDark);
								DrawBackstop(Center, AnimationNormal, BackstopRadius, FVector::RightVector, ColorDark);
							}
						}
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawBackstopDistances(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		uint8 ColorSeed = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDSphericalBackstopConstraint* const BackstopConstraint = ClothConstraints.GetBackstopConstraints().Get())
			{
				const bool bUseLegacyBackstop = BackstopConstraint->UseLegacyBackstop();
				const FName BackstopMeshName = FName(Cloth->GetPropertyString(Solver, Softs::FPBDSphericalBackstopConstraint::BackstopMeshNameName));
				const bool AccessoryMeshExists = Cloth->AccessoryMeshExists(Solver, BackstopMeshName);

				const TConstArrayView<Softs::FSolverVec3> AnimationPositions = AccessoryMeshExists ? Cloth->GetAccessoryMeshPositions(Solver, BackstopMeshName) : Cloth->GetAnimationPositions(Solver);
				const TConstArrayView<Softs::FSolverVec3> AnimationNormals = AccessoryMeshExists ? Cloth->GetAccessoryMeshNormals(Solver, BackstopMeshName) : Cloth->GetAnimationNormals(Solver);
				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
					};

				for (int32 Index = 0; Index < AnimationPositions.Num(); ++Index)
				{
					if (ShouldDrawParticle(Index))
					{
						ColorSeed += 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
						const FLinearColor ColorLight = FLinearColor::MakeFromHSV8(ColorSeed, 160, 128);
						const FLinearColor ColorDark = FLinearColor::MakeFromHSV8(ColorSeed, 160, 64);

						const FReal BackstopRadius = BackstopConstraint->GetBackstopRadius(Index) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();
						const FReal BackstopDistance = BackstopConstraint->GetBackstopDistance(Index) * BackstopConstraint->GetScale() * Solver->GetLocalSpaceScale();

						const FVector AnimationNormal(AnimationNormals[Index]);

						// Draw a line to the sphere boundary
						const FVector Pos0 = GetWorldPosition(*Solver, AnimationPositions[Index]);
						const FVector Pos1 = Pos0 - (bUseLegacyBackstop ? BackstopDistance - BackstopRadius : BackstopDistance) * AnimationNormal;
						DrawLine(Context.PDI, Pos0, Pos1, ColorDark);
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawMaxDistances(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		// Draw max distances
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
				{
					return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
				};
			const int32 NumParticles = Cloth->GetNumParticles(Solver);
			const Softs::FPBDFlatWeightMapView MaxDistances(
				(Softs::FSolverVec2)(Softs::FPBDSphericalConstraint::GetWeightedFloatMaxDistance(Cloth->GetConfig()->GetProperties(Cloth->GetLODIndex(Solver)), FVector2f(0.f, 1.f))),
				Cloth->GetWeightMapByProperty(Solver, Softs::FPBDSphericalConstraint::MaxDistanceName),
				NumParticles);

			const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetAnimationPositions(Solver);
			const TConstArrayView<Softs::FSolverVec3> Normals = Cloth->GetAnimationNormals(Solver);
			check(NumParticles == InvMasses.Num());
			check(NumParticles == Positions.Num());
			check(NumParticles == Normals.Num());

			for (int32 Index = 0; Index < MaxDistances.Num(); ++Index)
			{
				if (ShouldDrawParticle(Index))
				{
					const FReal MaxDistance = (FReal)MaxDistances.GetValue(Index);
					const FVector Position = GetWorldPosition(*Solver, Positions[Index]);
					if (InvMasses[Index] == (Softs::FSolverReal)0.)
					{
#if WITH_EDITOR
						DrawPoint(Context.PDI, Position, FLinearColor::Red, FMaterials::GetInstance().GetClothMaterialVertex());
#else
						DrawPoint(nullptr, Position, FLinearColor::Red, nullptr);
#endif
					}
					else
					{
						DrawLine(Context.PDI, Position, Position + GetWorldVector(*Solver, Normals[Index]) * MaxDistance, FLinearColor::White);
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawAnimDrive(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDAnimDriveConstraint* const AnimDriveConstraint = ClothConstraints.GetAnimDriveConstraints().Get())
			{
				const TConstArrayView<FRealSingle>& AnimDriveStiffnessMultipliers = Cloth->GetWeightMapByProperty(Solver, Softs::FPBDAnimDriveConstraint::AnimDriveStiffnessName);
				const TConstArrayView<Softs::FSolverVec3> AnimationPositions = Cloth->GetAnimationPositions(Solver);
				const TConstArrayView<Softs::FSolverVec3> ParticlePositions = Cloth->GetParticlePositions(Solver);
				check(ParticlePositions.Num() == AnimationPositions.Num());

				const FVec2 AnimDriveStiffness = AnimDriveConstraint->GetStiffness();
				const FRealSingle StiffnessOffset = AnimDriveStiffness[0];
				const FRealSingle StiffnessRange = AnimDriveStiffness[1] - AnimDriveStiffness[0];

				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
					};

				for (int32 Index = 0; Index < ParticlePositions.Num(); ++Index)
				{
					if (ShouldDrawParticle(Index))
					{
						const FRealSingle Stiffness = AnimDriveStiffnessMultipliers.IsValidIndex(Index) ?
							StiffnessOffset + AnimDriveStiffnessMultipliers[Index] * StiffnessRange :
							StiffnessOffset;

						const FVector AnimationPosition = GetWorldPosition(*Solver, AnimationPositions[Index]);
						const FVector ParticlePosition = GetWorldPosition(*Solver, ParticlePositions[Index]);
						DrawLine(Context.PDI, AnimationPosition, ParticlePosition, FLinearColor(FColor::Cyan) * Stiffness);
					}
				}
			}
		}
	}
	template<typename SpringConstraintType, typename GetEndPointsFuncType, typename ShouldDrawConstraintFuncType>
	static void DrawSpringConstraintColors(FPrimitiveDrawInterface* PDI, const ::Chaos::FClothingSimulationSolver& Solver, const SpringConstraintType* const SpringConstraints, GetEndPointsFuncType GetEndPoints, ShouldDrawConstraintFuncType ShouldDrawConstraint)
	{
		check(SpringConstraints);

		const TArray<int32>& ConstraintsPerColorStartIndex = SpringConstraints->GetConstraintsPerColorStartIndex();
		if (ConstraintsPerColorStartIndex.Num() > 1)
		{
			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
			const uint8 HueOffset = 196 / ConstraintColorNum;
			auto ConstraintColor =
				[HueOffset](int32 ColorIndex)->FLinearColor
				{
					return FLinearColor::MakeFromHSV8(ColorIndex * HueOffset, 255, 255);
				};

			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorEnd = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1];
				const FLinearColor DrawColor = ConstraintColor(ConstraintColorIndex);
				for (int32 ConstraintIndex = ColorStart; ConstraintIndex < ColorEnd; ++ConstraintIndex)
				{
					if (ShouldDrawConstraint(ConstraintIndex))
					{
						// Draw line
						Softs::FSolverVec3 P1, P2;
						GetEndPoints(ConstraintIndex, P1, P2);

						const FVec3 Pos0 = GetWorldPosition(Solver, P1);
						const FVec3 Pos1 = GetWorldPosition(Solver, P2);
						DrawLine(PDI, Pos0, Pos1, DrawColor);
					}
				}
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < SpringConstraints->GetConstraints().Num(); ++ConstraintIndex)
			{
				if (ShouldDrawConstraint(ConstraintIndex))
				{
					// Draw line
					Softs::FSolverVec3 P1, P2;
					GetEndPoints(ConstraintIndex, P1, P2);

					const FVec3 Pos0 = GetWorldPosition(Solver, P1);
					const FVec3 Pos1 = GetWorldPosition(Solver, P2);

					DrawLine(PDI, Pos0, Pos1, FLinearColor::Black);
				}
			}
		}
	}

	template<typename SpringConstraintType, typename ShouldDrawConstraintFuncType>
	static void DrawSpringConstraintColors(FPrimitiveDrawInterface* PDI, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const SpringConstraintType* const SpringConstraints, const ShouldDrawConstraintFuncType& ShouldDrawConstraint)
	{
		DrawSpringConstraintColors(PDI, Solver, SpringConstraints, [&SpringConstraints, &Positions](const int32 ConstraintIndex, Softs::FSolverVec3& P1, Softs::FSolverVec3& P2)
		{
			P1 = Positions[SpringConstraints->GetConstraints()[ConstraintIndex][0]];
			P2 = Positions[SpringConstraints->GetConstraints()[ConstraintIndex][1]];
		}, ShouldDrawConstraint);
	}

	static void DrawStretchBiasConstraints_ParallelGraphColor(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const FMaterialRenderProxy* MaterialRenderProxy, const Softs::FXPBDStretchBiasElementConstraints* const SpringConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{
#if WITH_EDITOR
		if (Context.PDI && MaterialRenderProxy)
		{
			auto ShouldDrawElement = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
				};

			const TArray<TVec3<int32>>& Constraints = SpringConstraints->GetConstraints();
			const TArray<int32>& ConstraintsPerColorStartIndex = SpringConstraints->GetConstraintsPerColorStartIndex();

			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
			const uint8 HueOffset = 196 / ConstraintColorNum;

			auto ConstraintColor =
				[HueOffset](int32 ColorIndex)->FColor
			{
				return FLinearColor::MakeFromHSV8(ColorIndex * HueOffset, 255, 255).ToFColor(true);
			};

			FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
			int32 VertexIndex = 0;
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorEnd = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1];
				const FColor DrawColor = ConstraintColor(ConstraintColorIndex);
				for (int32 ConstraintIndex = ColorStart; ConstraintIndex < ColorEnd; ++ConstraintIndex, VertexIndex += 3)
				{
					const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
					if (ShouldDrawElement(Constraint))
					{
						const FVector3f Pos0 = FVector3f(Positions[Constraint[0]]);
						const FVector3f Pos1 = FVector3f(Positions[Constraint[1]]);
						const FVector3f Pos2 = FVector3f(Positions[Constraint[2]]);
						const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
						const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

						MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), DrawColor));
						MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), DrawColor));
						MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), DrawColor));
						MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					}
				}
			}

			const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(Solver);
			MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
		}
#endif
	}

	static void DrawStretchBiasConstraints_WarpWeftStretch(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const::Chaos::FClothingSimulationSolver& Solver, const FMaterialRenderProxy* MaterialRenderProxy, const Softs::FXPBDStretchBiasElementConstraints* const SpringConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{
#if WITH_EDITOR
		if (Context.PDI && MaterialRenderProxy)
		{
			auto ShouldDrawElement = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
				};
			const TArray<TVec3<int32>>& Constraints = SpringConstraints->GetConstraints();
			const float StretchRangeMinClamped = Private::StretchBiasDrawRangeMin <= Private::StretchBiasDrawRangeMax ? Private::StretchBiasDrawRangeMin : 0.f;
			const float StretchRangeMaxClamped = Private::StretchBiasDrawRangeMin <= Private::StretchBiasDrawRangeMax ? Private::StretchBiasDrawRangeMax : 0.f;
			const float StretchRange = StretchRangeMaxClamped - StretchRangeMinClamped;
			const float StretchRangeInv = StretchRange > UE_KINDA_SMALL_NUMBER ? 1.f / StretchRange : 0.f;
			const bool bIsWeftStretch = (Private::EStretchBiasDrawMode)Private::StretchBiasDrawMode == Private::EStretchBiasDrawMode::WeftStretch;
			auto ConstraintColor =
				[bIsWeftStretch, StretchRangeMinClamped, StretchRangeMaxClamped, StretchRangeInv, &Constraints, &Positions, &SpringConstraints](int32 ConstraintIndex, bool& bOutOfRange)->FColor
			{
				bOutOfRange = false;

				// TODO: make these configurable
				constexpr float OutOfRangeMinHue = 240.f; // blue
				constexpr float MinHue = 180.f; // cyan
				constexpr float MaxHue = 60.f; // yellow
				constexpr float OutOfRangeMaxHue = 0.f; // red
				constexpr float StretchedValue = 1.f;
				constexpr float CompressedValue = 0.5;

				const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
				const Softs::FSolverVec3& P0 = Positions[Constraint[0]];
				const Softs::FSolverVec3& P1 = Positions[Constraint[1]];
				const Softs::FSolverVec3& P2 = Positions[Constraint[2]];
				Softs::FSolverVec3 dX_dU, dX_dV;
				SpringConstraints->CalculateUVStretch(ConstraintIndex, P0, P1, P2, dX_dU, dX_dV);
				const Softs::FSolverReal Stretch = bIsWeftStretch ? dX_dV.Length() : dX_dU.Length();

				const int32 WarpWeftIndex = bIsWeftStretch ? 1 : 0;

				const Softs::FSolverVec2 StretchScale = SpringConstraints->GetWarpWeftScale(ConstraintIndex);
				const Softs::FSolverReal RestStretch = SpringConstraints->GetRestStretchLengths()[ConstraintIndex][WarpWeftIndex] * StretchScale[WarpWeftIndex];
				const Softs::FSolverReal RestStretchInv = (Softs::FSolverReal)1. / FMath::Max(RestStretch, UE_KINDA_SMALL_NUMBER);

				const float StretchRatio = (Stretch - RestStretch) * RestStretchInv;
				if (StretchRatio < StretchRangeMinClamped)
				{
					bOutOfRange = true;
					return FLinearColor(OutOfRangeMinHue, 1.f, StretchRatio < 0 ? CompressedValue : StretchedValue).HSVToLinearRGB().ToFColor(true);
				}
				if (StretchRatio > StretchRangeMaxClamped)
				{
					bOutOfRange = true;
					return FLinearColor(OutOfRangeMaxHue, 1.f, StretchRatio < 0 ? CompressedValue : StretchedValue).HSVToLinearRGB().ToFColor(true);
				}

				// Convert from [StretchRangeMinClamped, StretchRangeMaxClamped] --> [MinHue, MaxHue]
				const float Hue = (MinHue + (MaxHue - MinHue) * (StretchRatio - StretchRangeMinClamped) * StretchRangeInv);
				return FLinearColor(Hue, 1.f, StretchRatio < 0 ? CompressedValue : StretchedValue).HSVToLinearRGB().ToFColor(true);
			};

			FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
			int32 VertexIndex = 0;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
				if (ShouldDrawElement(Constraint))
				{

					bool bIsOutOfRange;
					const FColor DrawColor = ConstraintColor(ConstraintIndex, bIsOutOfRange);
					if (bIsOutOfRange && !Private::bStretchBiasDrawOutOfRange)
					{
						continue;
					}

					const FVector3f Pos0 = FVector3f(Positions[Constraint[0]]);
					const FVector3f Pos1 = FVector3f(Positions[Constraint[1]]);
					const FVector3f Pos2 = FVector3f(Positions[Constraint[2]]);
					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), DrawColor));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), DrawColor));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), DrawColor));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					VertexIndex += 3;
				}
			}

			const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(Solver);
			MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
		}
#endif
	}

	static void DrawStretchBiasConstraints_BiasStretch(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const::Chaos::FClothingSimulationSolver& Solver, const FMaterialRenderProxy* MaterialRenderProxy, const Softs::FXPBDStretchBiasElementConstraints* const SpringConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{

#if WITH_EDITOR
		if (Context.PDI && MaterialRenderProxy)
		{
			auto ShouldDrawElement = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
				};
			const TArray<TVec3<int32>>& Constraints = SpringConstraints->GetConstraints();
			auto ConstraintColor = [&Constraints, &Positions, &SpringConstraints](int32 ConstraintIndex)->FColor
			{
				const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
				const Softs::FSolverVec3& P0 = Positions[Constraint[0]];
				const Softs::FSolverVec3& P1 = Positions[Constraint[1]];
				const Softs::FSolverVec3& P2 = Positions[Constraint[2]];
				Softs::FSolverVec3 dX_dU, dX_dV;
				SpringConstraints->CalculateUVStretch(ConstraintIndex, P0, P1, P2, dX_dU, dX_dV);
				
				const Softs::FSolverVec3 dX_dU_normalized = dX_dU.GetSafeNormal();
				const Softs::FSolverVec3 dX_dV_normalized = dX_dV.GetSafeNormal();
				const Softs::FSolverReal Shear = FMath::Abs(Softs::FSolverVec3::DotProduct(dX_dU_normalized, dX_dV_normalized));

				constexpr float UndeformedHue = 240.f; // blue
				constexpr float MaxDeformedHue = 360.f; // red

				return FLinearColor(UndeformedHue + Shear * (MaxDeformedHue - UndeformedHue), 1.f, 1.f).HSVToLinearRGB().ToFColor(true);
			};

			FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
			int32 VertexIndex = 0;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
				if (ShouldDrawElement(Constraint))
				{
					const FColor DrawColor = ConstraintColor(ConstraintIndex);

					const FVector3f Pos0 = FVector3f(Positions[Constraint[0]]);
					const FVector3f Pos1 = FVector3f(Positions[Constraint[1]]);
					const FVector3f Pos2 = FVector3f(Positions[Constraint[2]]);
					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), DrawColor));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), DrawColor));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), DrawColor));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
					VertexIndex += 3;
				}
			}

			const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(Solver);
			MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, MaterialRenderProxy, Chaos::Private::GetDepthPriority(), false, false);
		}
#endif
	}

	template<typename ConstraintType, typename ShouldDrawConstraintFuncType>
	static void DrawEdgeAnisotropy(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const ConstraintType* const BendingConstraints, ShouldDrawConstraintFuncType ShouldDrawConstraint)
	{
		const auto& Constraints = BendingConstraints->GetConstraints(); // auto because this could be Vec2 or Vec4, but we always just care about first two indices
		const TArray<Softs::FSolverVec3>& WarpWeftBiasBaseMultipliers = BendingConstraints->GetWarpWeftBiasBaseMultipliers();

		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			if (ShouldDrawConstraint(ConstraintIndex))
			{
				const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
				const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
				const Softs::FSolverVec3& Multiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];

				const FVec3 Pos0 = GetWorldPosition(Solver, P1);
				const FVec3 Pos1 = GetWorldPosition(Solver, P2);
				DrawLine(Context.PDI, Pos0, Pos1, FLinearColor(Multiplier[0], Multiplier[1], Multiplier[2]));
			}
		}
	}

	template<typename ConstraintType>
	static void DrawAxialSpringAnisotropy(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const ConstraintType* const AxialConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{
		auto ShouldDrawElement = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec3<int32>& Element)
			{
				return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
			};
		const TArray<TVec3<int32>>& Constraints = AxialConstraints->GetConstraints();
		const TArray<Softs::FSolverVec3>& WarpWeftBiasBaseMultipliers = AxialConstraints->GetWarpWeftBiasBaseMultipliers();
		const TArray<Softs::FSolverReal>& Barys = AxialConstraints->GetBarys();
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			if (ShouldDrawElement(Constraints[ConstraintIndex]))
			{
				const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
				const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
				const Softs::FSolverVec3& P3 = Positions[Constraints[ConstraintIndex][2]];
				const Softs::FSolverVec3 P = Barys[ConstraintIndex] * P2 + ((FSolverReal)1. - Barys[ConstraintIndex]) * P3;
				const Softs::FSolverVec3& Multiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];

				const FVec3 Pos0 = GetWorldPosition(Solver, P1);
				const FVec3 Pos1 = GetWorldPosition(Solver, P);
				DrawLine(Context.PDI, Pos0, Pos1, FLinearColor(Multiplier[0], Multiplier[1], Multiplier[2]));
			}
		}
	}

	template<typename SpringConstraintType, typename ShouldDrawConstraintFuncType>
	static void DrawAxialSpringConstraintColors(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const SpringConstraintType* const SpringConstraints, const ShouldDrawConstraintFuncType& ShouldDrawConstraint)
	{
		DrawSpringConstraintColors(Context.PDI, Solver, SpringConstraints, 
			[&SpringConstraints, &Positions](const int32 ConstraintIndex, Softs::FSolverVec3& P1, Softs::FSolverVec3& P2)
		{
			P1 = Positions[SpringConstraints->GetConstraints()[ConstraintIndex][0]];
			const Softs::FSolverReal Bary = SpringConstraints->GetBarys()[ConstraintIndex];
			
			P2 = Bary * Positions[SpringConstraints->GetConstraints()[ConstraintIndex][1]] + ((FSolverReal)1. - Bary)* Positions[SpringConstraints->GetConstraints()[ConstraintIndex][2]];
		}, ShouldDrawConstraint);
	}

	void FClothVisualizationNoGC::DrawEdgeConstraint(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

			// Constraints are locally indexed for new solver
			const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);

			if (const Softs::FPBDEdgeSpringConstraints* const EdgeConstraints = ClothConstraints.GetEdgeSpringConstraints().Get())
			{
				DrawSpringConstraintColors(Context.PDI, Positions, *Solver, EdgeConstraints,
					[&Context, EdgeConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
					{
						const TVec2<int32>& Constraint = EdgeConstraints->GetConstraints()[ConstraintIndex];
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
					});
			}

			if (const Softs::FXPBDEdgeSpringConstraints* const EdgeConstraints = ClothConstraints.GetXEdgeSpringConstraints().Get())
			{
				DrawSpringConstraintColors(Context.PDI, Positions, *Solver, EdgeConstraints,
					[&Context, EdgeConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
				{
					const TVec2<int32>& Constraint = EdgeConstraints->GetConstraints()[ConstraintIndex];
					return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
				});
			}

			if (const Softs::FXPBDAnisotropicSpringConstraints* const AnisoSpringConstraints = ClothConstraints.GetXAnisoSpringConstraints().Get())
			{
				switch ((Private::EAnisoSpringDrawMode)Private::AnisoSpringDrawMode)
				{
				case Private::EAnisoSpringDrawMode::Anisotropy:
					DrawEdgeAnisotropy(Context, Positions, *Solver, &AnisoSpringConstraints->GetEdgeConstraints(),
						[&Context, AnisoSpringConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
					{
						const TVec2<int32>& Constraint = AnisoSpringConstraints->GetEdgeConstraints().GetConstraints()[ConstraintIndex];
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
					});
					DrawAxialSpringAnisotropy(Context, Positions, *Solver, &AnisoSpringConstraints->GetAxialConstraints(), DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EAnisoSpringDrawMode::ParallelGraphColor: // fallthrough
				default:
					DrawSpringConstraintColors(Context.PDI, Positions, *Solver, &AnisoSpringConstraints->GetEdgeConstraints(),
						[&Context, AnisoSpringConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
						{
							const TVec2<int32>& Constraint = AnisoSpringConstraints->GetEdgeConstraints().GetConstraints()[ConstraintIndex];
							return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
						});
					DrawAxialSpringConstraintColors(Context, Positions, *Solver, &AnisoSpringConstraints->GetAxialConstraints(),
						[&Context, AnisoSpringConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
					{
						const TVec3<int32>& Constraint = AnisoSpringConstraints->GetAxialConstraints().GetConstraints()[ConstraintIndex];
						return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
					});
					break;
				}
			}

			if (const Softs::FXPBDStretchBiasElementConstraints* const StretchConstraints = ClothConstraints.GetXStretchBiasConstraints().Get())
			{
				const FMaterialRenderProxy* MaterialRenderProxy =
#if WITH_EDITOR
					FMaterials::GetInstance().GetClothMaterialColor()->GetRenderProxy();
#else
					nullptr;
#endif
				switch ((Private::EStretchBiasDrawMode)Private::StretchBiasDrawMode)
				{
				case Private::EStretchBiasDrawMode::WarpStretch: // fallthrough
				case Private::EStretchBiasDrawMode::WeftStretch:
					DrawStretchBiasConstraints_WarpWeftStretch(Context, Positions, *Solver, MaterialRenderProxy, StretchConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EStretchBiasDrawMode::BiasStretch:
					DrawStretchBiasConstraints_BiasStretch(Context, Positions, *Solver, MaterialRenderProxy, StretchConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EStretchBiasDrawMode::ParallelGraphColor: // fallthrough
				default:
					DrawStretchBiasConstraints_ParallelGraphColor(Context, Positions, *Solver, MaterialRenderProxy, StretchConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
				}

			}
		}
	}

	static void DrawBendingElementBuckleStatus(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const ::Chaos::Softs::FPBDBendingConstraintsBase* const BendingConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{
		const TArray<TVec4<int32>>& Constraints = BendingConstraints->GetConstraints();
		const TArray<bool>& IsBuckled = BendingConstraints->GetIsBuckled();

		auto ShouldDrawConstraint4 = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec4<int32>& Element)
			{
				return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
			};

		// Color constraint edge with red or blue: Red = Buckled, Blue = Not Buckled. 
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			if (ShouldDrawConstraint4(Constraints[ConstraintIndex]))
			{
				const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
				const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];

				const bool bIsBuckled = IsBuckled.IsValidIndex(ConstraintIndex) ? IsBuckled[ConstraintIndex] : false; // IsBuckled is empty if the simulation is paused.

				const FVec3 Pos0 = GetWorldPosition(Solver, P1);
				const FVec3 Pos1 = GetWorldPosition(Solver, P2);
				DrawLine(Context.PDI, Pos0, Pos1, bIsBuckled ? FLinearColor::Red : FLinearColor::Blue);
			}
		}
	}



	static void DrawBendingElementRestAngle(const FClothVisualizationNoGC::FDrawContext& Context, const TConstArrayView<::Chaos::Softs::FSolverVec3>& Positions, const ::Chaos::FClothingSimulationSolver& Solver, const ::Chaos::Softs::FPBDBendingConstraintsBase* const BendingConstraints, const TSet<int32>* const DebugFilterSet, int32 ConstraintToGlobalOffset, int32 ConstraintToLocalOffset)
	{
		const TArray<TVec4<int32>>& Constraints = BendingConstraints->GetConstraints();
		const TArray<Softs::FSolverReal>& RestAngles = BendingConstraints->GetRestAngles();

		auto ShouldDrawConstraint4 = [&Context, DebugFilterSet, ConstraintToGlobalOffset, ConstraintToLocalOffset](const TVec4<int32>& Element)
			{
				return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Element, ConstraintToGlobalOffset, ConstraintToLocalOffset);
			};
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			if (ShouldDrawConstraint4(Constraints[ConstraintIndex]))
			{
				const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
				const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
				const Softs::FSolverReal RestAngle = RestAngles[ConstraintIndex];
				const uint8 ColorSat = (uint8)FMath::Clamp(FMath::Abs(RestAngle) / UE_PI * 256, 0, 255);

				const FVec3 Pos0 = GetWorldPosition(Solver, P1);
				const FVec3 Pos1 = GetWorldPosition(Solver, P2);
				DrawLine(Context.PDI, Pos0, Pos1, FLinearColor::MakeFromHSV8(RestAngle > 0 ? 170 : 0, ColorSat, 255));
			}
		}
	}

	void FClothVisualizationNoGC::DrawBendingConstraint(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);


			// Constraints are locally indexed for new solver
			const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);

			if (const Softs::FPBDBendingSpringConstraints* const BendingConstraints = ClothConstraints.GetBendingSpringConstraints().Get())
			{
				DrawSpringConstraintColors(Context.PDI, Positions, *Solver, BendingConstraints,
					[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
					{
						const TVec2<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
					});
			}

			if (const Softs::FXPBDBendingSpringConstraints* const BendingConstraints = ClothConstraints.GetXBendingSpringConstraints().Get())
			{
				DrawSpringConstraintColors(Context.PDI, Positions, *Solver, BendingConstraints,
					[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
					{
						const TVec2<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
					});
			}

			if (const Softs::FPBDBendingConstraints* const BendingConstraints = ClothConstraints.GetBendingElementConstraints().Get())
			{
				switch ((Private::EBendingDrawMode)Private::BendingDrawMode)
				{
				case Private::EBendingDrawMode::RestAngle:
					DrawBendingElementRestAngle(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EBendingDrawMode::ParallelGraphColor:
					DrawSpringConstraintColors(Context.PDI, Positions, *Solver, BendingConstraints,
						[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
						{
							const TVec4<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
							return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
						});
					break;
				case Private::EBendingDrawMode::BuckleStatus:
				default:
					DrawBendingElementBuckleStatus(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				}
			}

			if (const Softs::FXPBDBendingConstraints* const BendingConstraints = ClothConstraints.GetXBendingElementConstraints().Get())
			{
				switch ((Private::EBendingDrawMode)Private::BendingDrawMode)
				{
				case Private::EBendingDrawMode::RestAngle:
					DrawBendingElementRestAngle(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EBendingDrawMode::ParallelGraphColor:
					DrawSpringConstraintColors(Context.PDI, Positions, *Solver, BendingConstraints,
						[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
						{
							const TVec4<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
							return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
						});
					break;
				case Private::EBendingDrawMode::BuckleStatus:
				default:
					DrawBendingElementBuckleStatus(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				}
			}

			if (const Softs::FXPBDAnisotropicBendingConstraints* const BendingConstraints = ClothConstraints.GetXAnisoBendingElementConstraints().Get())
			{
				switch ((Private::EBendingDrawMode)Private::BendingDrawMode)
				{
				case Private::EBendingDrawMode::RestAngle:
					DrawBendingElementRestAngle(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				case Private::EBendingDrawMode::ParallelGraphColor:
					DrawSpringConstraintColors(Context.PDI, Positions, *Solver, BendingConstraints,
						[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
						{
							const TVec4<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
							return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
						});
					break;
				case Private::EBendingDrawMode::Anisotropy:
					DrawEdgeAnisotropy(Context, Positions, *Solver, BendingConstraints,
						[&Context, BendingConstraints, DebugFilterSet, GlobalParticleOffset, Offset](int32 ConstraintIndex)
						{
							const TVec4<int32>& Constraint = BendingConstraints->GetConstraints()[ConstraintIndex];
							return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Constraint, GlobalParticleOffset - Offset, -Offset);
						});
					break;
				case Private::EBendingDrawMode::BuckleStatus:
				default:
					DrawBendingElementBuckleStatus(Context, Positions, *Solver, BendingConstraints, DebugFilterSet, GlobalParticleOffset - Offset, -Offset);
					break;
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawLongRangeConstraint(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		auto PseudoRandomColor =
			[](int32 NumColorRotations) -> FLinearColor
			{
				static const uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
				uint8 Seed = Spread;
				for (int32 i = 0; i < NumColorRotations; ++i)
				{
					Seed += Spread;
				}
				return FLinearColor::MakeFromHSV8(Seed, 160, 128);
			};

		auto Darken =
			[](const FLinearColor& Color) -> FLinearColor
			{
				FLinearColor ColorHSV = Color.LinearRGBToHSV();
				ColorHSV.B *= .5f;
				return ColorHSV.HSVToLinearRGB();
			};

		int32 ColorOffset = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);

			if (const Softs::FPBDLongRangeConstraints* const LongRangeConstraints = ClothConstraints.GetLongRangeConstraints().Get())
			{
				const TArray<TConstArrayView<Softs::FPBDLongRangeConstraints::FTether>>& Tethers = LongRangeConstraints->GetTethers();

				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 Offset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawParticle = [&Context, DebugFilterSet, Offset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, Offset);
					};

				for (int32 BatchIndex = 0; BatchIndex < Tethers.Num(); ++BatchIndex)
				{
					const FLinearColor Color = PseudoRandomColor(ColorOffset + BatchIndex);
					const FLinearColor DarkenedColor = Darken(Color);

					const TConstArrayView<Softs::FPBDLongRangeConstraints::FTether>& TetherBatch = Tethers[BatchIndex];

					// Draw tethers
					for (const Softs::FPBDLongRangeConstraints::FTether& Tether : TetherBatch)
					{
						const int32 KinematicIndex = LongRangeConstraints->GetStartIndex(Tether);
						const int32 DynamicIndex = LongRangeConstraints->GetEndIndex(Tether);
						const FReal TargetLength = LongRangeConstraints->GetTargetLength(Tether);
						if (ShouldDrawParticle(KinematicIndex) ||
							ShouldDrawParticle(DynamicIndex))
						{

							const Softs::FSolverVec3 KinematicPos = Positions[KinematicIndex];
							const Softs::FSolverVec3 DynamicPos = Positions[DynamicIndex];

							const FVec3 Pos0 = GetWorldPosition(*Solver, KinematicPos);
							const FVec3 Pos1 = GetWorldPosition(*Solver, DynamicPos);

							DrawLine(Context.PDI, Pos0, Pos1, Color);
#if WITH_EDITOR
							DrawPoint(Context.PDI, Pos1, Color, FMaterials::GetInstance().GetClothMaterialVertex());
#else
							DrawPoint(nullptr, Pos1, Color, nullptr);
#endif
							Softs::FSolverVec3 Direction = DynamicPos - KinematicPos;
							const float Length = Direction.SafeNormalize();
							if (Length > SMALL_NUMBER)
							{
								const FVec3 Pos2 = Pos1 + GetWorldVector(*Solver, Direction * (TargetLength - Length));
								DrawLine(Context.PDI, Pos1, Pos2, DarkenedColor);
							}
						}
					}
				}

				// Rotate the colors for each cloth
				ColorOffset += Tethers.Num();
			}
		}
	}

	void FClothVisualizationNoGC::DrawWindAndPressureForces(const FDrawContext& Context, const FReal ForceLength) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const Softs::FVelocityAndPressureField* VelocityField = nullptr;
			if (!Solver->IsLegacySolver())
			{
				const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
				VelocityField = ClothConstraints.GetVelocityAndPressureField().Get();
				if (!VelocityField)
				{
					continue;
				}
			}
			else
			{
				VelocityField = &Solver->GetWindVelocityAndPressureField(Cloth->GetGroupId());
			}

			// Constraints are locally indexed for new solver
			const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
			const TConstArrayView<Softs::FSolverReal> InvMasses = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverReal>(Solver->GetParticleInvMasses()) : Solver->GetParticleInvMassesView(ParticleRangeId);
			const TConstArrayView<Softs::FSolverVec3> Velocities = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleVs()) : Solver->GetParticleVsView(ParticleRangeId);

			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
				{
					return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};

			const TConstArrayView<TVec3<int32>>& Elements = VelocityField->GetElements();
			check(InvMasses.Num() == Positions.Num());
			check(InvMasses.Num() == Velocities.Num());

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];
				if (ShouldDrawElement(Element))
				{
					const FVec3 Position = GetWorldPosition(*Solver,
						(Positions[Element.X] +
							Positions[Element.Y] +
							Positions[Element.Z]) / (Softs::FSolverReal)3.f);

					const bool bIsKinematic0 = !InvMasses[Element.X];
					const bool bIsKinematic1 = !InvMasses[Element.Y];
					const bool bIsKinematic2 = !InvMasses[Element.Z];
					const bool bIsKinematic = bIsKinematic0 || bIsKinematic1 || bIsKinematic2;

					const FVec3 Force = GetWorldVector(*Solver, FVec3(VelocityField->CalculateForce(Positions, Velocities, ElementIndex)) * ForceLength);
					DrawLine(Context.PDI, Position, Position + Force, bIsKinematic ? FColor::Cyan : FColor::Green);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawWindVelocity(const FDrawContext& Context, const FReal LengthScale) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const Softs::FVelocityAndPressureField* VelocityField = nullptr;
			if (!Solver->IsLegacySolver())
			{
				const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
				VelocityField = ClothConstraints.GetVelocityAndPressureField().Get();
			}
			else
			{
				VelocityField = &Solver->GetWindVelocityAndPressureField(Cloth->GetGroupId());
			}

			if (VelocityField)
			{
				const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
				const FVector Pos0 = ReferenceSpaceTransform.GetLocation();
				const FVector Pos1 = Pos0 + LengthScale * GetWorldVector(*Solver, VelocityField->GetVelocity());
				DrawArrow(Context.PDI, Pos0, Pos1, FLinearColor::Blue);
			}
		}
	}

	void FClothVisualizationNoGC::DrawLocalSpace(const FDrawContext& Context, const FReal LocalSpaceLengthScale) const
	{
		if (!Solver)
		{
			return;
		}

		// Draw local space
		DrawCoordinateSystem(Context.PDI, FQuat::Identity, Solver->GetLocalSpaceLocation(), 5.f * Solver->GetLocalSpaceScale() * LocalSpaceLengthScale, 0.25f);
		DrawCoordinateSystem(Context.PDI, Solver->GetLocalSpaceRotation(), Solver->GetLocalSpaceLocation(), 7.f * Solver->GetLocalSpaceScale() * LocalSpaceLengthScale, 0.5f);
		DrawText(nullptr, nullptr, Solver->GetLocalSpaceLocation() + FVec3(2.), LOCTEXT("SimulationSpace", "Simulation Space"), FLinearColor::White, 1.5f);

		// Draw reference spaces
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
			DrawCoordinateSystem(Context.PDI, ReferenceSpaceTransform.GetRotation(), ReferenceSpaceTransform.GetLocation(), 10.f * Solver->GetLocalSpaceScale() * LocalSpaceLengthScale);
			check(Cloth->GetMesh());
			DrawText(nullptr, nullptr, ReferenceSpaceTransform.GetLocation() + FVec3(-2.), FText::FromName(Cloth->GetMesh()->GetReferenceBoneName()), FLinearColor::White, 1.5f);
		}
	}

	void FClothVisualizationNoGC::DrawVelocityScale(const FDrawContext& Context, const FReal LengthScale) const
	{
		if (!Solver)
		{
			return;
		}
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			const uint32 GroupId = Cloth->GetGroupId();

			const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
			const EChaosSoftsSimulationSpace VelocityScaleSpace = (EChaosSoftsSimulationSpace)FClothingSimulationCloth::GetVelocityScaleSpace(Cloth->GetConfig()->GetProperties(Cloth->GetLODIndex(Solver)), (int32)EChaosSoftsSimulationSpace::ReferenceBoneSpace);

			const FVector OriginalVelocity = FVector(Solver->GetOriginalReferenceSpaceVelocity(GroupId));
			const FVector OriginalAngularVelocity = FVector(Solver->GetOriginalReferenceSpaceAngularVelocity(GroupId));
			const FVector AppliedVelocity = FVector(Solver->GetAppliedReferenceSpaceVelocity(GroupId));
			const FVector AppliedAngularVelocity = FVector(Solver->GetAppliedReferenceSpaceAngularVelocity(GroupId));

			const FQuat& VelocityScaleSpaceRot = VelocityScaleSpace == EChaosSoftsSimulationSpace::WorldSpace ? FQuat::Identity : (VelocityScaleSpace == EChaosSoftsSimulationSpace::ComponentSpace ? FQuat(Solver->GetLocalSpaceRotation()) : ReferenceSpaceTransform.GetRotation());
			
			const FVector XDir = VelocityScaleSpaceRot.RotateVector(FVector::ForwardVector);
			const FVector YDir = VelocityScaleSpaceRot.RotateVector(FVector::RightVector);
			const FVector ZDir = VelocityScaleSpaceRot.RotateVector(FVector::UpVector);

			const FReal XOrig = FVector::DotProduct(OriginalVelocity, XDir);
			const FReal YOrig = FVector::DotProduct(OriginalVelocity, YDir);
			const FReal ZOrig = FVector::DotProduct(OriginalVelocity, ZDir);

			const FReal XApplied = FVector::DotProduct(AppliedVelocity, XDir);
			const FReal YApplied = FVector::DotProduct(AppliedVelocity, YDir);
			const FReal ZApplied = FVector::DotProduct(AppliedVelocity, ZDir);

			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + LengthScale * AppliedVelocity , FLinearColor::White);
			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + LengthScale * XApplied * XDir, FLinearColor::Red);
			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + LengthScale * YApplied * YDir, FLinearColor::Green);
			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + LengthScale * ZApplied * ZDir, FLinearColor::Blue);

			// Draw OrigVelocity lighter
			constexpr float OrigColorScale = 0.5f;
			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + LengthScale * OriginalVelocity, FLinearColor::White * OrigColorScale);
			if (XOrig > XApplied)
			{
				DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation() + LengthScale * XApplied * XDir, ReferenceSpaceTransform.GetLocation() + LengthScale * XOrig * XDir, FLinearColor::Red * OrigColorScale);
			}
			if (YOrig > YApplied)
			{
				DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation() + LengthScale * YApplied * YDir, ReferenceSpaceTransform.GetLocation() + LengthScale * YOrig * YDir, FLinearColor::Green * OrigColorScale);
			}
			if (ZOrig > ZApplied)
			{
				DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation() + LengthScale * ZApplied * ZDir, ReferenceSpaceTransform.GetLocation() + LengthScale * ZOrig * ZDir, FLinearColor::Blue * OrigColorScale);
			}


			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + OriginalAngularVelocity, FLinearColor::Yellow * OrigColorScale);
			DrawArrow(Context.PDI, ReferenceSpaceTransform.GetLocation(), ReferenceSpaceTransform.GetLocation() + AppliedAngularVelocity, FLinearColor::Yellow);

			// Text
			const FText Label = FText::Format(FText::FromString("VelocityScale\nV: {0} ({1})\nX: {2} ({3})\nY: {4} ({5})\nZ: {6} ({7})\nA: {8} ({9})"),
				FText::AsNumber(AppliedVelocity.Length()), FText::AsNumber(OriginalVelocity.Length()),
				FText::AsNumber(XApplied), FText::AsNumber(XOrig),
				FText::AsNumber(YApplied), FText::AsNumber(YOrig),
				FText::AsNumber(ZApplied), FText::AsNumber(ZOrig),
				FText::AsNumber(AppliedAngularVelocity.Length()), FText::AsNumber(OriginalAngularVelocity.Length()));
			DrawText(nullptr, nullptr, ReferenceSpaceTransform.GetLocation() + FVec3(5.), Label, FLinearColor::White, 1.5f);
		}
	}

	void FClothVisualizationNoGC::DrawSelfCollision(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

			if (const Softs::FPBDCollisionSpringConstraints* const SelfCollisionConstraints = ClothConstraints.GetSelfCollisionConstraints().Get())
			{
				// Constraints are locally indexed for new solver
				const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec4<int32>& Element)
					{
						return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
					};
				const TArray<TVec4<int32>>& Constraints = SelfCollisionConstraints->GetConstraints();
				const TArray<Softs::FSolverVec3>& Barys = SelfCollisionConstraints->GetBarys();
				const TArray<bool>& FlipNormals = SelfCollisionConstraints->GetFlipNormals();

				for (int32 Index = 0; Index < Constraints.Num(); ++Index)
				{
					const TVec4<int32>& Constraint = Constraints[Index];
					if (ShouldDrawElement(Constraint))
					{
						const FReal Height = (FReal)SelfCollisionConstraints->GetConstraintThickness(Index);
						const FVec3 Bary(Barys[Index]);

						const FVector P = GetWorldPosition(*Solver, Positions[Constraint[0]]);
						const FVector P0 = GetWorldPosition(*Solver, Positions[Constraint[1]]);
						const FVector P1 = GetWorldPosition(*Solver, Positions[Constraint[2]]);
						const FVector P2 = GetWorldPosition(*Solver, Positions[Constraint[3]]);

						const FVector Pos0 = P0 * Bary[0] + P1 * Bary[1] + P2 * Bary[2];

						static const FLinearColor Brown(0.1f, 0.05f, 0.f);
						static const FLinearColor Red(0.3f, 0.f, 0.f);
						const FTriangle Triangle(P0, P1, P2);
						const FVector Normal = FlipNormals[Index] ? -Triangle.GetNormal() : Triangle.GetNormal();

						// Draw point to surface line (=normal)
						const FVector Pos1 = Pos0 + GetWorldVector(*Solver, Height * Normal);
						DrawPoint(Context.PDI, Pos0, Brown, nullptr, 2.f);
						DrawLine(Context.PDI, Pos0, Pos1, FlipNormals[Index] ? Red : Brown);

						// Draw pushup to point
						static const FLinearColor Orange(0.3f, 0.15f, 0.f);
						DrawPoint(Context.PDI, P, Orange, nullptr, 2.f);
						DrawLine(Context.PDI, Pos1, P, Orange);
					}
				}

				const TArray<int32>& KinematicCollidingParticles = SelfCollisionConstraints->GetKinematicCollidingParticles();
				const TArray<TMap<int32, Softs::FSolverReal>>& KinematicColliderTimers = SelfCollisionConstraints->GetKinematicColliderTimers();
				if (const FTriangleMesh* TriangleMesh = SelfCollisionConstraints->GetKinematicColliderTriangleMesh())
				{

					for (int32 Index1 : KinematicCollidingParticles)
					{
						const FVector P = GetWorldPosition(*Solver, Positions[Index1]);

						bool bShouldDrawPoint = false;

						const TMap<int32, Softs::FSolverReal>& Timers = KinematicColliderTimers[Index1 - Offset];
						for (const TPair<int32, Softs::FSolverReal>& ElemAndTimer : Timers)
						{
							const int32 Index2 = TriangleMesh->GetElements()[ElemAndTimer.Get<0>()][0];
							const int32 Index3 = TriangleMesh->GetElements()[ElemAndTimer.Get<0>()][1];
							const int32 Index4 = TriangleMesh->GetElements()[ElemAndTimer.Get<0>()][2];

							if (ShouldDrawElement(TVec4<int32>(Index1, Index2, Index3, Index4)))
							{
								bShouldDrawPoint = true;
								const Softs::FSolverVec3& P1 = Positions[Index1];
								const Softs::FSolverVec3& P2 = Positions[Index2];
								const Softs::FSolverVec3& P3 = Positions[Index3];
								const Softs::FSolverVec3& P4 = Positions[Index4];
								Softs::FSolverVec3 Bary;
								const FVector Pos1 = GetWorldPosition(*Solver, FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary));

								static const FLinearColor LtRed(0.6f, 0.f, 0.f);
								static const FLinearColor DkRed(0.3f, 0.f, 0.f);
								const FLinearColor& Color = ElemAndTimer.Get<1>() > 0.f ? LtRed : DkRed;
								DrawPoint(Context.PDI, Pos1, Color, nullptr, 2.f);
								DrawLine(Context.PDI, Pos1, P, Color);
							}
						}

						if (bShouldDrawPoint)
						{
							static const FLinearColor Orange(0.3f, 0.15f, 0.f);
							DrawPoint(Context.PDI, P, Orange, nullptr, 2.f);
						}
					}
				}
			}

			if (const Softs::FPBDSelfCollisionSphereConstraints* const SelfCollisionSphereConstraints =
				ClothConstraints.GetSelfCollisionSphereConstraints().Get())
			{
				// Constraints are locally indexed for new solver
				const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawConstraint2 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec2<int32>& Element)
					{
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
					};
				auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
					};
				const TArray<TVec2<int32>>& Constraints = SelfCollisionSphereConstraints->GetConstraints();
				for (int32 Index = 0; Index < Constraints.Num(); ++Index)
				{
					const TVec2<int32>& Constraint = Constraints[Index];
					if (ShouldDrawConstraint2(Constraint))
					{
						const FVector P0 = GetWorldPosition(*Solver, Positions[Constraint[0]]);
						const FVector P1 = GetWorldPosition(*Solver, Positions[Constraint[1]]);
						static const FLinearColor Brown(0.1f, 0.05f, 0.f);
						DrawLine(Context.PDI, P0, P1, Brown);
					}
				}

				if (const TSet<int32>* const VertexSet = SelfCollisionSphereConstraints->GetVertexSet())
				{
					const FReal Radius = (FReal)SelfCollisionSphereConstraints->GetRadius() * Solver->GetLocalSpaceScale();
					for (const int32 Vertex : *VertexSet)
					{
						if (ShouldDrawParticle(Vertex))
						{
							const FVector P0 = GetWorldPosition(*Solver, Positions[Vertex + Offset]);
							DrawSphere(Context.PDI, FSphere(FVector::ZeroVector, Radius), FQuat::Identity,
								P0, 1., FLinearColor(FColor::Orange));
						}
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawSelfIntersection(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

			static const FLinearColor Red(1.f, 0.f, 0.f);
			static const FLinearColor White(1.f, 1.f, 1.f);
			static const FLinearColor Black(0.f, 0.f, 0.f);
			static const FLinearColor Teal(0.f, 0.5f, 0.5f);
			static const FLinearColor Orange(1.f, .5f, 0.f);
			static const FLinearColor Green(0.f, 1.f, 0.f);
			static const FLinearColor Yellow(1.f, 1.f, 0.f);
			static const FLinearColor Blue(0.f, 0.f, 1.f);

			if (const Softs::FPBDTriangleMeshCollisions* const SelfCollisionInit = ClothConstraints.GetSelfCollisionInit().Get())
			{
				// Constraints are locally indexed for new solver
				const TConstArrayView<Softs::FSolverVec3> Positions = Solver->IsLegacySolver() ? TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs()) : Solver->GetParticleXsView(ParticleRangeId);
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;

				const FTriangleMesh& TriangleMesh = Cloth->GetTriangleMesh(Solver);

				// Draw contours
				const TArray<TArray<Softs::FPBDTriangleMeshCollisions::FBarycentricPoint>>& ContourPoints = SelfCollisionInit->GetIntersectionContourPoints();
				const TArray<TArray<Softs::FPBDTriangleMeshCollisions::FBarycentricPoint>>& PostStepContourPoints = SelfCollisionInit->GetPostStepIntersectionContourPoints();
				const TArray<Softs::FPBDTriangleMeshCollisions::FContourType>& ContourTypes = SelfCollisionInit->GetIntersectionContourTypes();
				check(ContourPoints.Num() == ContourTypes.Num());

				static const FLinearColor ColorsForType[(int8)Softs::FPBDTriangleMeshCollisions::FContourType::Count] =
				{
					Teal,
					Red,
					Blue,
					Yellow,
					White,
					Black
				};

				// Don't filter contours
				auto DrawContour = [this, &Context, &Positions](const TArray<Softs::FPBDTriangleMeshCollisions::FBarycentricPoint>& Contour,
					const FLinearColor& ContourColor)
				{
					for (int32 PointIdx = 0; PointIdx < Contour.Num() - 1; ++PointIdx)
					{
						const Softs::FPBDTriangleMeshCollisions::FBarycentricPoint& Point0 = Contour[PointIdx];
						const FVector EndPoint0 = GetWorldPosition(*Solver, (1.f - Point0.Bary[0] - Point0.Bary[1]) * Positions[Point0.Vertices[0]] + Point0.Bary[0] * Positions[Point0.Vertices[1]] + Point0.Bary[1] * Positions[Point0.Vertices[2]]);
						const Softs::FPBDTriangleMeshCollisions::FBarycentricPoint& Point1 = Contour[PointIdx + 1];
						const FVector EndPoint1 = GetWorldPosition(*Solver, (1.f - Point1.Bary[0] - Point1.Bary[1]) * Positions[Point1.Vertices[0]] + Point1.Bary[0] * Positions[Point1.Vertices[1]] + Point1.Bary[1] * Positions[Point1.Vertices[2]]);
						DrawLine(Context.PDI, EndPoint0, EndPoint1, ContourColor);
						DrawPoint(Context.PDI, EndPoint0, ContourColor, nullptr, 1.f);
						DrawPoint(Context.PDI, EndPoint1, ContourColor, nullptr, 1.f);
					}

				};

				for( int32 ContourIndex = 0; ContourIndex < ContourPoints.Num(); ++ContourIndex)
				{
					const TArray<Softs::FPBDTriangleMeshCollisions::FBarycentricPoint>& Contour = ContourPoints[ContourIndex];
					const FLinearColor& ContourColor = ColorsForType[(int8)ContourTypes[ContourIndex]];
					DrawContour(Contour, ContourColor);
				}
				for (int32 ContourIndex = 0; ContourIndex < PostStepContourPoints.Num(); ++ContourIndex)
				{
					const TArray<Softs::FPBDTriangleMeshCollisions::FBarycentricPoint>& Contour = PostStepContourPoints[ContourIndex];
					DrawContour(Contour, Orange);
				}

				// Draw GIA colors (don't filter)
				const TConstArrayView<Softs::FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors = SelfCollisionInit->GetVertexGIAColors();
				static const FLinearColor Gray(0.5f, 0.5f, 0.5f);
				if (VertexGIAColors.Num())
				{
					const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
					for (int32 ParticleIdx = Offset; ParticleIdx < VertexGIAColors.Num(); ++ParticleIdx)
					{
						if (VertexGIAColors[ParticleIdx].ContourIndexBits)
						{
							const bool bIsLoop = VertexGIAColors[ParticleIdx].IsLoop();
							const bool bIsBoundary = VertexGIAColors[ParticleIdx].IsBoundary();
							const bool bAnyWhite = (VertexGIAColors[ParticleIdx].ContourIndexBits & ~VertexGIAColors[ParticleIdx].ColorBits);
							const bool bAnyBlack = (VertexGIAColors[ParticleIdx].ContourIndexBits & VertexGIAColors[ParticleIdx].ColorBits);
							const FLinearColor& VertColor = bIsLoop ? Red : bIsBoundary ? Blue : (bAnyWhite && bAnyBlack) ? Gray : bAnyWhite ? White : Black;
						
							DrawPoint(Context.PDI, GetWorldPosition(*Solver, Positions[ParticleIdx]), VertColor, nullptr, 5.f);
						}
					}
				}
				const TArray<Softs::FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors = SelfCollisionInit->GetTriangleGIAColors();
				if (TriangleGIAColors.Num() == TriangleMesh.GetNumElements())
				{
					const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
					for (int32 TriangleIdx = 0; TriangleIdx < TriangleGIAColors.Num(); ++TriangleIdx)
					{
						if (TriangleGIAColors[TriangleIdx].ContourIndexBits)
						{
							const bool bIsLoop = TriangleGIAColors[TriangleIdx].IsLoop();
							const bool bAnyWhite = (TriangleGIAColors[TriangleIdx].ContourIndexBits & ~TriangleGIAColors[TriangleIdx].ColorBits);
							const bool bAnyBlack = (TriangleGIAColors[TriangleIdx].ContourIndexBits & TriangleGIAColors[TriangleIdx].ColorBits);
							const FLinearColor& TriColor = bIsLoop ? Red : (bAnyWhite && bAnyBlack) ? Gray : bAnyWhite ? White : Black;
							DrawLine(Context.PDI, GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][0]]), GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][1]]), TriColor);
							DrawLine(Context.PDI, GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][1]]), GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][2]]), TriColor);
							DrawLine(Context.PDI, GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][0]]), GetWorldPosition(*Solver, Positions[Elements[TriangleIdx][2]]), TriColor);
						}
					}
				}

				// Draw contour minimization gradients (don't filter)
				const TArray<Softs::FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& ContourMinimizationIntersections = SelfCollisionInit->GetContourMinimizationIntersections();
				constexpr FReal MaxDrawImpulse = 1.;
				constexpr FReal RegularizeEpsilonSq = 1.;
				for (const Softs::FPBDTriangleMeshCollisions::FContourMinimizationIntersection& Intersection : ContourMinimizationIntersections)
				{
					Softs::FSolverReal GradientLength;
					Softs::FSolverVec3 GradientDir;
					Intersection.GlobalGradientVector.ToDirectionAndLength(GradientDir, GradientLength);
					const FVector Delta = FVector(GradientDir) * MaxDrawImpulse * GradientLength * FMath::InvSqrt(GradientLength * GradientLength + RegularizeEpsilonSq);

					const FVector EdgeCenter = GetWorldPosition(*Solver, .5 * (Positions[Intersection.EdgeVertices[0]] + Positions[Intersection.EdgeVertices[1]]));
					const FVector TriCenter = GetWorldPosition(*Solver, (Positions[Intersection.FaceVertices[0]] + Positions[Intersection.FaceVertices[1]] + Positions[Intersection.FaceVertices[2]]) / 3.);

					DrawPoint(Context.PDI, EdgeCenter, Green, nullptr, 2.f);
					DrawLine(Context.PDI, EdgeCenter, EdgeCenter + Delta, Green);
					DrawPoint(Context.PDI, TriCenter, Green, nullptr, 2.f);
					DrawLine(Context.PDI, TriCenter, TriCenter - Delta, Green);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawSelfCollisionThickness(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

#if WITH_EDITOR
		if (FMaterials::GetInstance().GetClothMaterialColor() && Context.PDI)
		{
			FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());

			for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
			{
				const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
				if (ParticleRangeId == INDEX_NONE)
				{
					continue;
				}

				const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

				if (const Softs::FPBDCollisionSpringConstraints* const SelfCollisionConstraints = ClothConstraints.GetSelfCollisionConstraints().Get())
				{
					const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
					auto ShouldDrawEdge = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec2<int32>& Element)
						{
							return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
						};
					const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
					const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
					const TConstArrayView<int32>& WeightMap = Cloth->GetFaceIntMapByProperty(Solver, Softs::FPBDCollisionSpringConstraints::SelfCollisionLayersName);

					const TConstArrayView<TVec2<int32>> Edges = Cloth->GetTriangleMesh(Solver).GetSegmentMesh().GetElements();
					const TArray<TVec2<int32>>& EdgeToFaces = Cloth->GetTriangleMesh(Solver).GetEdgeToFaces();
					for(int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); ++EdgeIndex)
					{
						const TVec2<int32>& Edge = Edges[EdgeIndex];
					
						const bool bIsKinematic0 = (InvMasses[Edge[0] - Offset] == (Softs::FSolverReal)0.);
						const bool bIsKinematic1 = (InvMasses[Edge[1] - Offset] == (Softs::FSolverReal)0.);
						if ((bIsKinematic0 && bIsKinematic1) || !ShouldDrawEdge(Edge))
						{
							continue;
						}

						const FVector3f Position1(Positions[Edge[0] - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
						const FVector3f Position2(Positions[Edge[1] - Offset]);

						const FRealSingle Radius1 = (FRealSingle)SelfCollisionConstraints->GetParticleThickness(Edge[0]);
						const FRealSingle Radius2 = (FRealSingle)SelfCollisionConstraints->GetParticleThickness(Edge[1]);
						const int32 Face1Layer = WeightMap.IsValidIndex(EdgeToFaces[EdgeIndex][0]) ? WeightMap[EdgeToFaces[EdgeIndex][0]] : INDEX_NONE;
						const int32 Face2Layer = WeightMap.IsValidIndex(EdgeToFaces[EdgeIndex][1]) ? WeightMap[EdgeToFaces[EdgeIndex][1]] : INDEX_NONE;
						const FLinearColor Color = Chaos::Private::PseudoRandomColor((Face1Layer == Face2Layer || Face2Layer == INDEX_NONE) ? Face1Layer : INDEX_NONE);
						
						AppendTaperedCylinderTriangles(MeshBuilder, Position1, Position2, Radius1, Radius2, 6, Color);
					}
				}
			}
			const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(*Solver);
			MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, FMaterials::GetInstance().GetClothMaterialColor()->GetRenderProxy(), Chaos::Private::GetDepthPriority(), false, false);
		}
		else
#endif
		{
			for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
			{
				const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
				if (ParticleRangeId == INDEX_NONE)
				{
					continue;
				}

				const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

				if (const Softs::FPBDCollisionSpringConstraints* const SelfCollisionConstraints = ClothConstraints.GetSelfCollisionConstraints().Get())
				{
					const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
					auto ShouldDrawEdge = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec2<int32>& Element)
						{
							return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
						};
					auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
						{
							return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
						};
					const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
					const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);

					const TConstArrayView<TVec2<int32>> Edges = Cloth->GetTriangleMesh(Solver).GetSegmentMesh().GetElements();
					for (const TVec2<int32>& Edge : Edges)
					{
						const bool bIsKinematic0 = (InvMasses[Edge[0] - Offset] == (Softs::FSolverReal)0.);
						const bool bIsKinematic1 = (InvMasses[Edge[1] - Offset] == (Softs::FSolverReal)0.);
						if ((bIsKinematic0 && bIsKinematic1) || !ShouldDrawEdge(Edge))
						{
							continue;
						}

						const FVector Position1(Positions[Edge[0] - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
						const FVector Position2(Positions[Edge[1] - Offset]);

						const FReal Radius1 = (FReal)SelfCollisionConstraints->GetParticleThickness(Edge[0]) * Solver->GetLocalSpaceScale();
						const FReal Radius2 = (FReal)SelfCollisionConstraints->GetParticleThickness(Edge[1]) * Solver->GetLocalSpaceScale();
						DrawTaperedCylinder(Context.PDI, GetWorldPosition(*Solver, Position1), GetWorldPosition(*Solver, Position2), Radius1, Radius2, 6, FLinearColor::Gray);
					}

					for (int32 VertexIndex = 0; VertexIndex < Positions.Num(); ++VertexIndex)
					{
						const bool bIsKinematic0 = (InvMasses[VertexIndex] == (Softs::FSolverReal)0.);
						if (bIsKinematic0 || !ShouldDrawParticle(VertexIndex))
						{
							continue;
						}

						const FVector Position1(Positions[VertexIndex]);
						const FReal Radius1 = (FReal)SelfCollisionConstraints->GetParticleThickness(VertexIndex + Offset);
						const FTransform Transform(FRotator::ZeroRotator, GetWorldPosition(*Solver, Position1), FVector(Solver->GetLocalSpaceScale()));
						DrawWireSphere(Context.PDI, Transform, FLinearColor::Gray, Radius1, 6, Chaos::Private::GetDepthPriority(), 0.0f, 0.001f, false);
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawClothCollisionThickness(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

			if (const Softs::FPBDSoftBodyCollisionConstraint* const CollisionConstraints = ClothConstraints.GetCollisionConstraint().Get())
			{
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
					};
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
				for (int32 VertexIndex = 0; VertexIndex < Positions.Num(); ++VertexIndex)
				{
					const bool bIsKinematic0 = (InvMasses[VertexIndex] == (Softs::FSolverReal)0.);
					if (bIsKinematic0 || !ShouldDrawParticle(VertexIndex))
					{
						continue;
					}

					const FVec3 Pos0 = GetWorldPosition(*Solver, Positions[VertexIndex]);
					const FReal Radius1 = (FReal)CollisionConstraints->GetParticleClothThickness(VertexIndex + Offset);
					DrawSphere(Context.PDI, Pos0, FQuat::Identity, Radius1, FLinearColor::White, nullptr);
				}
			}
		}		
	}

	void FClothVisualizationNoGC::DrawInnerCollisionThickness(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

			if (const Softs::FPBDSoftBodyCollisionConstraint* const CollisionConstraints = ClothConstraints.GetCollisionConstraint().Get())
			{
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
				const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
				auto ShouldDrawParticle = [&Context, DebugFilterSet, GlobalParticleOffset](const int32 Index)
					{
						return Private::ShouldDrawParticle(Context.Filter, DebugFilterSet, Index, GlobalParticleOffset);
					};
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
				for (int32 VertexIndex = 0; VertexIndex < Positions.Num(); ++VertexIndex)
				{
					const bool bIsKinematic0 = (InvMasses[VertexIndex] == (Softs::FSolverReal)0.);
					if (bIsKinematic0 || !ShouldDrawParticle(VertexIndex))
					{
						continue;
					}

					const FVec3 Pos0 = GetWorldPosition(*Solver, Positions[VertexIndex]);
					const FReal Radius1 = (FReal)CollisionConstraints->GetParticleInnerThickness(VertexIndex + Offset);
					DrawSphere(Context.PDI, Pos0, FQuat::Identity, Radius1, FLinearColor::Gray, nullptr);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawKinematicColliderWired(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}

		static const FLinearColor WireframeColor = FColor::Silver;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDTriangleMeshCollisions* const SelfCollisionInit = ClothConstraints.GetSelfCollisionInit().Get())
			{
				const FTriangleMesh& KinematicColliderMesh = SelfCollisionInit->GetCollidableSubMesh().GetKinematicColliderSubMesh();
				// Elements are local indexed for new solver
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TConstArrayView<TVec3<int32>> Elements = KinematicColliderMesh.GetElements();
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const TVec3<int32>& Element = Elements[ElementIndex];

					const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
					const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

					DrawLine(Context.PDI, Pos0, Pos1, WireframeColor);
					DrawLine(Context.PDI, Pos1, Pos2, WireframeColor);
					DrawLine(Context.PDI, Pos2, Pos0, WireframeColor);
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawClothClothConstraints(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}
		static const FLinearColor ActiveConstraintColor = FColor::Turquoise;
		static const FLinearColor InactiveConstraintColor = FColor::Silver;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			if (const Softs::FXPBDVertexConstraints* const ClothVertexConstraints = ClothConstraints.GetClothVertexSpringConstraints().Get())
			{
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TArray<TVec2<int32>>& Constraints = ClothVertexConstraints->GetConstraints();
				const TArray<Softs::FSolverReal>& SpringLengths = ClothVertexConstraints->GetSpringLengths();
				auto ShouldDrawConstraint2 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec2<int32>& Element)
					{
						return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
					};
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					if (ShouldDrawConstraint2(Constraints[ConstraintIndex]))
					{
						const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
						const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];

						const Softs::FSolverReal DistSq = Softs::FSolverVec3::DistSquared(P1, P2);
						const Softs::FSolverReal Stiffness = DistSq >= FMath::Square(SpringLengths[ConstraintIndex]) ? ClothVertexConstraints->GetExtensionStiffness(ConstraintIndex) : ClothVertexConstraints->GetCompressionStiffness(ConstraintIndex);
						const bool bIsActive = Stiffness > (Softs::FSolverReal)0.f;
						const FVec3 Pos0 = GetWorldPosition(*Solver, P1);
						const FVec3 Pos1 = GetWorldPosition(*Solver, P2);
						const FLinearColor& ConstraintColor = bIsActive ? ActiveConstraintColor : InactiveConstraintColor;
						DrawLine(Context.PDI, Pos0, Pos1, ConstraintColor);
						DrawPoint(Context.PDI, Pos0, ConstraintColor, nullptr);
						DrawPoint(Context.PDI, Pos1, ConstraintColor, nullptr);
					}
				}
			}
			if (const Softs::FXPBDVertexFaceConstraints* const ClothVertexFaceConstraints = ClothConstraints.GetClothVertexFaceSpringConstraints().Get())
			{
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TArray<TVec4<int32>>& Constraints = ClothVertexFaceConstraints->GetConstraints();
				const TArray<TVec4<Softs::FSolverReal>>& Weights = ClothVertexFaceConstraints->GetWeights();
				const TArray<Softs::FSolverReal>& SpringLengths = ClothVertexFaceConstraints->GetSpringLengths();
				auto ShouldDrawConstraint4 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec4<int32>& Element)
					{
						return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
					};
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					if (ShouldDrawConstraint4(Constraints[ConstraintIndex]))
					{
						const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
						const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
						const Softs::FSolverVec3& P3 = Positions[Constraints[ConstraintIndex][2]];
						const Softs::FSolverVec3& P4 = Positions[Constraints[ConstraintIndex][3]];

						const Softs::FSolverVec3 P = -Weights[ConstraintIndex][1] * P2 - Weights[ConstraintIndex][2] * P3 - Weights[ConstraintIndex][3] * P4;

						const Softs::FSolverReal DistSq = Softs::FSolverVec3::DistSquared(P1, P);
						const Softs::FSolverReal Stiffness = DistSq >= FMath::Square(SpringLengths[ConstraintIndex]) ? ClothVertexFaceConstraints->GetExtensionStiffness(ConstraintIndex) : ClothVertexFaceConstraints->GetCompressionStiffness(ConstraintIndex);
						const bool bIsActive = Stiffness > (Softs::FSolverReal)0.f;
						const FVec3 Pos0 = GetWorldPosition(*Solver, P1);
						const FVec3 Pos1 = GetWorldPosition(*Solver, P);
						const FLinearColor& ConstraintColor = bIsActive ? ActiveConstraintColor : InactiveConstraintColor;
						DrawLine(Context.PDI, Pos0, Pos1, ConstraintColor);
						DrawPoint(Context.PDI, Pos0, ConstraintColor, nullptr);
						DrawPoint(Context.PDI, Pos1, ConstraintColor, nullptr);
					}
				}
			}
			if (const Softs::FXPBDFaceConstraints* const ClothFaceConstraints = ClothConstraints.GetClothFaceSpringConstraints().Get())
			{
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TArray<TVector<int32,6>>& Constraints = ClothFaceConstraints->GetConstraints();
				const TArray<TVector<Softs::FSolverReal,6>>& Weights = ClothFaceConstraints->GetWeights();
				const TArray<Softs::FSolverReal>& SpringLengths = ClothFaceConstraints->GetSpringLengths();
				auto ShouldDrawConstraint6 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVector<int32, 6>& Element)
					{
						return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, TVec3<int32>(Element[0], Element[1], Element[2]), GlobalParticleOffset - Offset, -Offset) ||
							Private::ShouldDrawElement(Context.Filter, DebugFilterSet, TVec3<int32>(Element[3], Element[4], Element[5]), GlobalParticleOffset - Offset, -Offset);
					};
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					if (ShouldDrawConstraint6(Constraints[ConstraintIndex]))
					{
						const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
						const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
						const Softs::FSolverVec3& P3 = Positions[Constraints[ConstraintIndex][2]];
						const Softs::FSolverVec3& P4 = Positions[Constraints[ConstraintIndex][3]];
						const Softs::FSolverVec3& P5 = Positions[Constraints[ConstraintIndex][4]];
						const Softs::FSolverVec3& P6 = Positions[Constraints[ConstraintIndex][5]];

						const Softs::FSolverVec3 Ps = Weights[ConstraintIndex][0] * P1 + Weights[ConstraintIndex][1] * P2 + Weights[ConstraintIndex][2] * P3;
						const Softs::FSolverVec3 Pt = -Weights[ConstraintIndex][3] * P4 - Weights[ConstraintIndex][4] * P5 - Weights[ConstraintIndex][5] * P6;

						const Softs::FSolverReal DistSq = Softs::FSolverVec3::DistSquared(Ps, Pt);
						const Softs::FSolverReal Stiffness = DistSq >= FMath::Square(SpringLengths[ConstraintIndex]) ? ClothFaceConstraints->GetExtensionStiffness(ConstraintIndex) : ClothFaceConstraints->GetCompressionStiffness(ConstraintIndex);
						const bool bIsActive = Stiffness > (Softs::FSolverReal)0.f;
						const FVec3 Pos0 = GetWorldPosition(*Solver, Ps);
						const FVec3 Pos1 = GetWorldPosition(*Solver, Pt);
						const FLinearColor& ConstraintColor = bIsActive ? ActiveConstraintColor : InactiveConstraintColor;
						DrawLine(Context.PDI, Pos0, Pos1, ConstraintColor);
						DrawPoint(Context.PDI, Pos0, ConstraintColor, nullptr);
						DrawPoint(Context.PDI, Pos1, ConstraintColor, nullptr);
					}
				}
			}
			if (const Softs::FPBDVertexFaceRepulsionConstraints* const RepulsionConstraints = ClothConstraints.GetRepulsionConstraints().Get())
			{
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TArray<TVec4<int32>>& Constraints = RepulsionConstraints->GetConstraints();
				const TArray<bool>& IsActive = RepulsionConstraints->GetConstraintIsActive();
				auto ShouldDrawConstraint4 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec4<int32>& Element)
					{
						return Private::ShouldDrawConstraint4(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
					};
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					if (ShouldDrawConstraint4(Constraints[ConstraintIndex]))
					{
						const Softs::FSolverVec3& P1 = Positions[Constraints[ConstraintIndex][0]];
						const Softs::FSolverVec3& P2 = Positions[Constraints[ConstraintIndex][1]];
						const Softs::FSolverVec3& P3 = Positions[Constraints[ConstraintIndex][2]];
						const Softs::FSolverVec3& P4 = Positions[Constraints[ConstraintIndex][3]];

						FSolverVec3 Bary;
						const Softs::FSolverVec3 P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);

						const Softs::FSolverReal DistSq = Softs::FSolverVec3::DistSquared(P1, P);
						const FVec3 Pos0 = GetWorldPosition(*Solver, P1);
						const FVec3 Pos1 = GetWorldPosition(*Solver, P);
						const FLinearColor& ConstraintColor = IsActive[ConstraintIndex] ? ActiveConstraintColor : InactiveConstraintColor;
						DrawLine(Context.PDI, Pos0, Pos1, ConstraintColor);
						DrawPoint(Context.PDI, Pos0, ConstraintColor, nullptr);
						DrawPoint(Context.PDI, Pos1, ConstraintColor, nullptr);
					}
				}
			}
		}
	}

	void FClothVisualizationNoGC::DrawTeleportReset(const FDrawContext& Context) const
	{
		if (!Solver)
		{
			return;
		}
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			FLinearColor Color;
			constexpr FRealSingle ColorHysteresis = .25f;
			if (Cloth->GetTimeSinceLastReset() <= Cloth->GetTimeSinceLastTeleport())
			{
				// Blend from Red to White as time approaches ColorHysteresis
				Color.R = 1.f;
				Color.A = 1.f;
				Color.G = Color.B = Cloth->GetTimeSinceLastReset() == 0.f ? 0.f : FMath::Clamp(.5f * (Cloth->GetTimeSinceLastReset() / ColorHysteresis + 1.f), 0.f, 1.f);
			}
			else
			{
				// Blend from Green to white as time approaches ColorHysteresis
				Color.G = 1.f;
				Color.A = 1.f;
				Color.R = Color.B = Cloth->GetTimeSinceLastTeleport() == 0.f ? 0.f : FMath::Clamp(.5f * (Cloth->GetTimeSinceLastReset() / ColorHysteresis + 1.f), 0.f, 1.f);
			}

			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;

			const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
			const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
			{
				const TVec3<int32>& Element = Elements[ElementIndex];

				const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
				const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
				const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

				DrawLine(Context.PDI, Pos0, Pos1, Color);
				DrawLine(Context.PDI, Pos1, Pos2, Color);
				DrawLine(Context.PDI, Pos2, Pos0, Color);
			}
		}
	}

	void FClothVisualizationNoGC::DrawExtremlyDeformedEdges(const FDrawContext& Context) const
	{
		static const FLinearColor DynamicColor = FColor::Red;
		static const FLinearColor KinematicColor = FColor::Purple;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}
			// Elements are local indexed for new solver
			const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
			const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
			const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
			auto ShouldDrawConstraint2 = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec2<int32>& Element)
				{
					return Private::ShouldDrawConstraint2(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
				};
			const FClothConstraints& ClothConstraint = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDExtremeDeformationConstraints* const ExtremeDeformationConstraints = Solver->GetClothConstraints(ParticleRangeId).GetExtremeDeformationConstraints().Get())
			{
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				const TConstArrayView<Softs::FSolverReal> InvMasses = Cloth->GetParticleInvMasses(Solver);
				const TArray<TVec2<int32>> ExtremelyDeformedEdges = Solver->IsLegacySolver() ?
					ExtremeDeformationConstraints->GetExtremelyDeformedEdges(Solver->GetParticleXs()) :
					ExtremeDeformationConstraints->GetExtremelyDeformedEdges(Solver->GetParticleXsView(ParticleRangeId));
				check(InvMasses.Num() == Positions.Num());
				for (int32 EdgeIndex = 0; EdgeIndex < ExtremelyDeformedEdges.Num(); ++EdgeIndex)
				{
					const TVec2<int32>& Edge = ExtremelyDeformedEdges[EdgeIndex];
					if (ShouldDrawConstraint2(Edge))
					{
						const FVector Pos0 = GetWorldPosition(*Solver, Positions[Edge[0] - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
						const FVector Pos1 = GetWorldPosition(*Solver, Positions[Edge[1] - Offset]);
						const bool bIsKinematic0 = (InvMasses[Edge[0] - Offset] == (Softs::FSolverReal)0.);
						const bool bIsKinematic1 = (InvMasses[Edge[1] - Offset] == (Softs::FSolverReal)0.);
						DrawLine(Context.PDI, Pos0, Pos1, bIsKinematic0 && bIsKinematic1 ? KinematicColor : DynamicColor);
					}
				}
			}
		}
	}

#if WITH_EDITOR
	TArray<FString> FClothVisualizationNoGC::GetAllWeightMapNames() const
	{
		TSet<FString> AllNames;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			AllNames.Append(Cloth->GetAllWeightMapNames());
		}
		return AllNames.Array();
	}

	TArray<FString> FClothVisualizationNoGC::GetAllMorphTargetNames() const
	{
		TSet<FString> AllNames;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			AllNames.Append(Cloth->GetAllMorphTargetNames());
		}
		return AllNames.Array();
	}

	TArray<FString> FClothVisualizationNoGC::GetAllVertexSetNames() const
	{
		TSet<FString> AllNames;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			AllNames.Append(Cloth->GetAllVertexSetNames());
		}
		return AllNames.Array();
	}

	TArray<FName> FClothVisualizationNoGC::GetAllAccessoryMeshNames() const
	{
		TSet<FName> AllNames;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			AllNames.Append(Cloth->GetAllAccessoryMeshNames());
		}
		return AllNames.Array();
	}

	void FClothVisualizationNoGC::DrawKinematicColliderShaded(const FDrawContext& Context) const
	{
		if (!Solver || !FMaterials::GetInstance().GetCollisionMaterial())
		{
			return;
		}

		FDynamicMeshBuilder MeshBuilder(Context.PDI->View->GetFeatureLevel());
		int32 VertexIndex = 0;

		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);
			if (const Softs::FPBDTriangleMeshCollisions* const SelfCollisionInit = ClothConstraints.GetSelfCollisionInit().Get())
			{
				const FTriangleMesh& KinematicColliderMesh = SelfCollisionInit->GetCollidableSubMesh().GetKinematicColliderSubMesh();
				// Elements are local indexed for new solver
				const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
				const TConstArrayView<TVec3<int32>> Elements = KinematicColliderMesh.GetElements();
				const TConstArrayView<Softs::FSolverVec3> Positions = Cloth->GetParticlePositions(Solver);
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex, VertexIndex += 3)
				{
					const TVec3<int32>& Element = Elements[ElementIndex];
					const FVector3f Pos0(Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
					const FVector3f Pos1(Positions[Element.Y - Offset]);
					const FVector3f Pos2(Positions[Element.Z - Offset]);

					const FVector3f Normal = FVector3f::CrossProduct(Pos2 - Pos0, Pos1 - Pos0).GetSafeNormal();
					const FVector3f Tangent = ((Pos1 + Pos2) * 0.5f - Pos0).GetSafeNormal();

					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos0, Tangent, Normal, FVector2f(0.f, 0.f), FColor::White));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos1, Tangent, Normal, FVector2f(0.f, 1.f), FColor::White));
					MeshBuilder.AddVertex(FDynamicMeshVertex(Pos2, Tangent, Normal, FVector2f(1.f, 1.f), FColor::White));
					MeshBuilder.AddTriangle(VertexIndex, VertexIndex + 1, VertexIndex + 2);
				}
			}
		}

		const FMatrix LocalSimSpaceToWorld = GetLocalSpaceToWorld(*Solver);
		MeshBuilder.Draw(Context.PDI, LocalSimSpaceToWorld, FMaterials::GetInstance().GetCollisionMaterial()->GetRenderProxy(), Chaos::Private::GetDepthPriority(), false, false);
	}

	void FClothVisualizationNoGC::DrawSimMorphTarget(const FDrawContext& Context, const FString& Name) const
	{
		if (!Solver)
		{
			return;
		}
		static const FLinearColor Color = FColor::Turquoise;
		for (const FClothingSimulationCloth* const Cloth : Solver->GetCloths())
		{
			const int32 ParticleRangeId = Cloth->GetParticleRangeId(Solver);
			if (ParticleRangeId == INDEX_NONE)
			{
				continue;
			}

			const int32 LODIndex = Cloth->GetLODIndex(Solver);

			if (const FClothingSimulationMesh* const Mesh = Cloth->GetMesh())
			{
				int32 MorphTargetIndex = Mesh->FindMorphTargetByName(LODIndex, Name);
				if (MorphTargetIndex == INDEX_NONE)
				{
					MorphTargetIndex = Cloth->GetCurrentMorphTargetIndex(Solver);
				}

				if (MorphTargetIndex != INDEX_NONE)
				{
					TConstArrayView<FVector3f> PositionDeltas = Mesh->GetMorphTargetPositionDeltas(LODIndex, MorphTargetIndex);
					TConstArrayView<int32> Indices = Mesh->GetMorphTargetIndices(LODIndex, MorphTargetIndex);
					TConstArrayView<FVector3f> OrigPositions = Mesh->GetPositions(LODIndex);

					TArray<FVector3f> Positions(OrigPositions.GetData(), OrigPositions.Num());
					for (int32 Index = 0; Index < Indices.Num(); ++Index)
					{
						const int32 Vertex = Indices[Index];
						Positions[Vertex] += PositionDeltas[Index];
					}


					// Elements are local indexed for new solver
					const int32 Offset = Solver->IsLegacySolver() ? ParticleRangeId : 0;
					const TSet<int32>* const DebugFilterSet = Context.Filter.VertexSet.IsEmpty() ? nullptr : Cloth->GetVertexSetByName(Solver, Context.Filter.VertexSet);
					const int32 GlobalParticleOffset = Solver->GetGlobalParticleOffset(ParticleRangeId);
					auto ShouldDrawElement = [&Context, DebugFilterSet, GlobalParticleOffset, Offset](const TVec3<int32>& Element)
						{
							return Private::ShouldDrawElement(Context.Filter, DebugFilterSet, Element, GlobalParticleOffset - Offset, -Offset);
						};

					const TConstArrayView<TVec3<int32>> Elements = Cloth->GetTriangleMesh(Solver).GetElements();
					for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
					{
						const TVec3<int32>& Element = Elements[ElementIndex];
						if (ShouldDrawElement(Element))
						{

							const FVector Pos0 = GetWorldPosition(*Solver, Positions[Element.X - Offset]); // TODO: Triangle Mesh shouldn't really be solver dependent (ie not use an offset)
							const FVector Pos1 = GetWorldPosition(*Solver, Positions[Element.Y - Offset]);
							const FVector Pos2 = GetWorldPosition(*Solver, Positions[Element.Z - Offset]);

							DrawLine(Context.PDI, Pos0, Pos1, Color);
							DrawLine(Context.PDI, Pos1, Pos2, Color);
							DrawLine(Context.PDI, Pos2, Pos0, Color);
						}
					}
				}
			}
		}
	}
#endif // #if WITH_EDITOR
#else  // #if CHAOS_DEBUG_DRAW
namespace Chaos
{
	FClothVisualizationNoGC::FClothVisualizationNoGC(const ::Chaos::FClothingSimulationSolver* /*InSolver*/)
	{}

	FClothVisualizationNoGC::~FClothVisualizationNoGC() = default;
#endif  // #else #if CHAOS_DEBUG_DRAW
}  // End namespace Chaos

#undef LOCTEXT_NAMESPACE
