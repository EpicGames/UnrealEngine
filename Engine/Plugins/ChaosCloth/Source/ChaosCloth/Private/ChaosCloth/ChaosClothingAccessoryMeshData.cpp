// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingAccessoryMeshData.h"
#include "ChaosCloth/ChaosClothingSimulationAccessoryMesh.h"
#if INTEL_ISPC
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosClothingSimulationSolver.ispc.generated.h"
#endif

namespace Chaos
{
	FClothingAccessoryMeshData::FClothingAccessoryMeshData(
		int32 InNumParticles,
		const FClothingSimulationAccessoryMesh& InAccessoryMesh)
		: NumParticles(InNumParticles)
		, AccessoryMesh(InAccessoryMesh)
	{
		check(NumParticles == AccessoryMesh.GetNumPoints());
	}

	void FClothingAccessoryMeshData::AllocateParticles()
	{
		OldAnimationPositions.SetNum(NumParticles);
		AnimationPositions.SetNum(NumParticles);
		InterpolatedAnimationPositions.SetNum(NumParticles);
		OldAnimationNormals.SetNum(NumParticles);
		AnimationNormals.SetNum(NumParticles);
		InterpolatedAnimationNormals.SetNum(NumParticles);
		AnimationVelocities.SetNum(NumParticles);
	}

	void FClothingAccessoryMeshData::DeallocateParticles()
	{
		OldAnimationPositions.Empty();
		AnimationPositions.Empty();
		InterpolatedAnimationPositions.Empty();
		OldAnimationNormals.Empty();
		AnimationNormals.Empty();
		InterpolatedAnimationNormals.Empty();
		AnimationVelocities.Empty();
	}

	void FClothingAccessoryMeshData::ResetStartPose()
	{
		OldAnimationPositions = InterpolatedAnimationPositions = AnimationPositions;
		OldAnimationNormals = InterpolatedAnimationNormals = AnimationNormals;

		AnimationVelocities.Reset();
		AnimationVelocities.SetNumZeroed(NumParticles);
	}

	void FClothingAccessoryMeshData::Update(const FClothingSimulationSolver* Solver)
	{
		if (AnimationPositions.Num() == NumParticles)
		{
			Swap(OldAnimationPositions, AnimationPositions);
			Swap(OldAnimationNormals, AnimationNormals);
			AccessoryMesh.Update(Solver, TArrayView<Softs::FSolverVec3>(AnimationPositions), TArrayView<Softs::FSolverVec3>(AnimationNormals));
		}
	}

	void FClothingAccessoryMeshData::ApplyPreSimulationTransforms(const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingAccessoryMeshData_ApplyPreSimulationTransforms);
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_PreSimulationTransforms_ISPC_Enabled)
		{
			ispc::ApplyAnimationPreSimulationTransform(
				(ispc::FVector3f*)OldAnimationPositions.GetData(),
				(ispc::FVector3f*)AnimationVelocities.GetData(),
				(const ispc::FVector3f*)AnimationPositions.GetData(),
				(const ispc::FTransform3f&)GroupSpaceTransform,
				(const ispc::FVector3f&)DeltaLocalSpaceLocation,
				DeltaTime,
				OldAnimationPositions.Num());
		}
		else
#endif
		{
			for (int32 Index = 0; Index < OldAnimationPositions.Num(); ++Index)
			{
				OldAnimationPositions[Index] = GroupSpaceTransform.TransformPositionNoScale(OldAnimationPositions[Index]) - DeltaLocalSpaceLocation;
				AnimationVelocities[Index] = (AnimationPositions[Index] - OldAnimationPositions[Index]) / DeltaTime;
			}
		}
	}

	void FClothingAccessoryMeshData::PreSubstep(const Softs::FSolverReal InterpolationAlpha)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingAccessoryMeshData_PreSubstep);
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_PreSubstepInterpolation_ISPC_Enabled)
		{
			ispc::PreSubstepInterpolation(
				(ispc::FVector3f*)InterpolatedAnimationPositions.GetData(),
				(ispc::FVector3f*)InterpolatedAnimationNormals.GetData(),
				(const ispc::FVector3f*)AnimationPositions.GetData(),
				(const ispc::FVector3f*)OldAnimationPositions.GetData(),
				(const ispc::FVector3f*)AnimationNormals.GetData(),
				(const ispc::FVector3f*)OldAnimationNormals.GetData(),
				InterpolationAlpha,
				0,
				InterpolatedAnimationPositions.Num());
		}
		else
#endif
		{
			for (int32 Index = 0; Index < InterpolatedAnimationPositions.Num(); ++Index)
			{
				InterpolatedAnimationPositions[Index] = InterpolationAlpha * AnimationPositions[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationPositions[Index];
				InterpolatedAnimationNormals[Index] = (InterpolationAlpha * AnimationNormals[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationNormals[Index]).GetSafeNormal();
			}
		}
	}
} // namespace Chaos
