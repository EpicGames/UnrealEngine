// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Containers/ContainersFwd.h"
namespace Chaos
{
class FClothingSimulationAccessoryMesh;
class FClothingSimulationSolver;

struct FClothingAccessoryMeshData
{
	const int32 NumParticles;
	const FClothingSimulationAccessoryMesh& AccessoryMesh;

	FClothingAccessoryMeshData(
		int32 InNumParticles,
		const FClothingSimulationAccessoryMesh& InAccessoryMesh);

	void AllocateParticles();
	void DeallocateParticles();
	void ResetStartPose();
	void Update(const FClothingSimulationSolver* Solver);
	void ApplyPreSimulationTransforms(const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime);
	void PreSubstep(const Softs::FSolverReal InterpolationAlpha);

	const TArray<Softs::FSolverVec3>& GetOldAnimationPositions() const
	{
		return OldAnimationPositions;
	}
	const TArray<Softs::FSolverVec3>& GetAnimationPositions() const
	{
		return AnimationPositions;
	}
	const TArray<Softs::FSolverVec3>& GetInterpolatedAnimationPositions() const
	{
		return InterpolatedAnimationPositions;
	}
	const TArray<Softs::FSolverVec3>& GetOldAnimationNormals() const
	{
		return OldAnimationNormals;
	}
	const TArray<Softs::FSolverVec3>& GetAnimationNormals() const
	{
		return AnimationNormals;
	}
	const TArray<Softs::FSolverVec3>& GetInterpolatedAnimationNormals() const
	{
		return InterpolatedAnimationNormals;
	}
	const TArray<Softs::FSolverVec3>& GetAnimationVelocities() const
	{
		return AnimationVelocities;
	}
private:
	TArray<Softs::FSolverVec3> OldAnimationPositions;
	TArray<Softs::FSolverVec3> AnimationPositions;
	TArray<Softs::FSolverVec3> InterpolatedAnimationPositions;
	TArray<Softs::FSolverVec3> OldAnimationNormals;
	TArray<Softs::FSolverVec3> AnimationNormals;
	TArray<Softs::FSolverVec3> InterpolatedAnimationNormals;
	TArray<Softs::FSolverVec3> AnimationVelocities;
};
} // namespace Chaos
