// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothingCacheSchema.h"
#include "ClothingSimulationCacheData.h"
#include "Chaos/ChaosCache.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"

namespace Chaos
{

void FClothingCacheSchema::RecordPostSolve(const FClothingSimulationSolver& ClothSolver, FPendingFrameWrite& OutFrame, FReal InTime)
{
	const int32 NumParticles = ClothSolver.GetNumActiveParticles();
	if (NumParticles > 0)
	{
		const Softs::FSolverVec3* const ParticleXs = ClothSolver.GetParticleXs(0);
		const Softs::FSolverVec3* const ParticleVs = ClothSolver.GetParticleVs(0);

		TArray<float> PendingVX, PendingVY, PendingVZ, PendingPX, PendingPY, PendingPZ;
		TArray<int32>& PendingID = OutFrame.PendingChannelsIndices;

		PendingID.Reserve(NumParticles);
		PendingVX.Reserve(NumParticles);
		PendingVY.Reserve(NumParticles);
		PendingVZ.Reserve(NumParticles);
		PendingPX.Reserve(NumParticles);
		PendingPY.Reserve(NumParticles);
		PendingPZ.Reserve(NumParticles);

		const TConstArrayView<const FClothingSimulationCloth*> Cloths = ClothSolver.GetCloths();
		for (int32 ClothIndex = 0; ClothIndex < Cloths.Num(); ++ClothIndex)
		{
			const FClothingSimulationCloth* Cloth = Cloths[ClothIndex];
			const FTransform& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();
			const FName ReferenceTransformNameAndIndex(ReferenceTransformsName, Cloth->GetGroupId());
			OutFrame.PendingNamedTransformData.Add(ReferenceTransformNameAndIndex, ReferenceSpaceTransform);

			const int32 ParticleRangeId = Cloth->GetParticleRangeId(&ClothSolver);
			const int32 NumClothParticles = Cloth->GetNumParticles(&ClothSolver);
			const int32 GlobalOffset = ClothSolver.GetGlobalParticleOffset(ParticleRangeId);
			for (int32 ParticleIndex = GlobalOffset; ParticleIndex < GlobalOffset + NumClothParticles; ++ParticleIndex)
			{
				const Softs::FSolverVec3& ParticleV = ParticleVs[ParticleIndex];
				const Softs::FSolverVec3& ParticleX = ParticleXs[ParticleIndex];

				// Adding the vertices relative position to the particle write datas
				PendingID.Add(ParticleIndex);
				PendingVX.Add(ParticleV.X);
				PendingVY.Add(ParticleV.Y);
				PendingVZ.Add(ParticleV.Z);

				PendingPX.Add(ParticleX.X);
				PendingPY.Add(ParticleX.Y);
				PendingPZ.Add(ParticleX.Z);
			}
		}

		OutFrame.PendingChannelsData.Add(VelocityXName, PendingVX);
		OutFrame.PendingChannelsData.Add(VelocityYName, PendingVY);
		OutFrame.PendingChannelsData.Add(VelocityZName, PendingVZ);
		OutFrame.PendingChannelsData.Add(PositionXName, PendingPX);
		OutFrame.PendingChannelsData.Add(PositionYName, PendingPY);
		OutFrame.PendingChannelsData.Add(PositionZName, PendingPZ);
	}
}

void FClothingCacheSchema::PlaybackPreSolve(UChaosCache& InCache, FReal InTime, FPlaybackTickRecord& TickRecord, FClothingSimulationSolver& ClothSolver)
{
	FCacheEvaluationContext Context(TickRecord);
	Context.bEvaluateTransform = false;
	Context.bEvaluateCurves = false;
	Context.bEvaluateEvents = false;
	Context.bEvaluateChannels = true;
	Context.bEvaluateNamedTransforms = true;

	// The evaluated result are already in world space since we are passing the tickrecord.spacetransform in the context
	FCacheEvaluationResult EvaluatedResult = InCache.Evaluate(Context, nullptr);

	const TArray<float>* PendingVX = EvaluatedResult.Channels.Find(VelocityXName);
	const TArray<float>* PendingVY = EvaluatedResult.Channels.Find(VelocityYName);
	const TArray<float>* PendingVZ = EvaluatedResult.Channels.Find(VelocityZName);
	const TArray<float>* PendingPX = EvaluatedResult.Channels.Find(PositionXName);
	const TArray<float>* PendingPY = EvaluatedResult.Channels.Find(PositionYName);
	const TArray<float>* PendingPZ = EvaluatedResult.Channels.Find(PositionZName);
	const int32 NumCachedParticles = EvaluatedResult.ParticleIndices.Num();

	if (PendingVX && PendingVY && PendingVZ && PendingPX && PendingPY && PendingPZ && NumCachedParticles)
	{
		// Directly set the result of the cache into the solver particles
		Softs::FSolverVec3* const ParticleXs = ClothSolver.GetParticleXs(0);
		Softs::FSolverVec3* const ParticleVs = ClothSolver.GetParticleVs(0);
		const int32 NumParticles = ClothSolver.GetNumParticles();
		for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
		{
			const int32 ParticleIndex = EvaluatedResult.ParticleIndices[CachedIndex];
			check(ParticleIndex < NumParticles);
			Softs::FSolverVec3& ParticleV = ParticleVs[ParticleIndex];
			Softs::FSolverVec3& ParticleX = ParticleXs[ParticleIndex];
			ParticleV.X = (*PendingVX)[CachedIndex];
			ParticleV.Y = (*PendingVY)[CachedIndex];
			ParticleV.Z = (*PendingVZ)[CachedIndex];
			ParticleX.X = (*PendingPX)[CachedIndex];
			ParticleX.Y = (*PendingPY)[CachedIndex];
			ParticleX.Z = (*PendingPZ)[CachedIndex];
		}

		// Fix up positions based on any discrepency between cached ReferenceSpaceTransform and current ReferenceSpaceTransform
		TConstArrayView<const FClothingSimulationCloth*> Cloths = ClothSolver.GetCloths();
		auto GetCloth = [&Cloths](const int32 GroupId)
		{
			const FClothingSimulationCloth* const* Found = Cloths.FindByPredicate(
				[GroupId](const FClothingSimulationCloth* InCloth)
				{
					return InCloth->GetGroupId() == GroupId;
				});
			return Found ? *Found : nullptr;
		};

		const FVec3& LocalSpaceLocation = ClothSolver.GetLocalSpaceLocation();
		constexpr bool bNoCompareNumber = false;
		for (const TPair<FName, FTransform>& NamedTransform : EvaluatedResult.NamedTransforms)
		{
			if (NamedTransform.Key.IsEqual(ReferenceTransformsName, ENameCase::CaseSensitive, bNoCompareNumber))
			{
				const int32 ClothGroupId = NamedTransform.Key.GetNumber();
				const FClothingSimulationCloth* Cloth = GetCloth(ClothGroupId);
				if (Cloth)
				{
					const FRigidTransform3& CurrentTransform = Cloth->GetReferenceSpaceTransform();
					const FTransform& CachedTransform = NamedTransform.Value;
					if (!CurrentTransform.Equals(CachedTransform))
					{
						const int32 ParticleRangeId = Cloth->GetParticleRangeId(&ClothSolver);
						const int32 GlobalOffset = ClothSolver.GetGlobalParticleOffset(ParticleRangeId);

						const FTransform RelativeTransform = CurrentTransform.GetRelativeTransform(CachedTransform);
						const int32 NumClothParticles = Cloth->GetNumParticles(&ClothSolver);
						check(GlobalOffset + NumClothParticles <= NumParticles);
						for (int32 ParticleIndex = GlobalOffset; ParticleIndex < GlobalOffset + NumClothParticles; ++ParticleIndex)
						{
							Softs::FSolverVec3& ParticleX = ParticleXs[ParticleIndex];
							ParticleX = Softs::FSolverVec3(RelativeTransform.TransformPosition(FVec3(ParticleX + LocalSpaceLocation)) - LocalSpaceLocation);
							Softs::FSolverVec3& ParticleV = ParticleVs[ParticleIndex];
							ParticleV = Softs::FSolverVec3(RelativeTransform.TransformVector(FVec3(ParticleV)));
						}
					}
				}
			}
		}
	}
}

void FClothingCacheSchema::LoadCacheData(UChaosCache* InCache, Chaos::FReal InTime, FClothingSimulationCacheData& CacheData)
{
	FPlaybackTickRecord TickRecord;
	TickRecord.SetLastTime(InTime);

	FCacheEvaluationContext Context(TickRecord);
	Context.bEvaluateTransform = false;
	Context.bEvaluateCurves = false;
	Context.bEvaluateEvents = false;
	Context.bEvaluateChannels = true;
	Context.bEvaluateNamedTransforms = true;

	const FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(Context, nullptr);

	const TArray<float>* PendingVX = EvaluatedResult.Channels.Find(VelocityXName);
	const TArray<float>* PendingVY = EvaluatedResult.Channels.Find(VelocityYName);
	const TArray<float>* PendingVZ = EvaluatedResult.Channels.Find(VelocityZName);
	const TArray<float>* PendingPX = EvaluatedResult.Channels.Find(PositionXName);
	const TArray<float>* PendingPY = EvaluatedResult.Channels.Find(PositionYName);
	const TArray<float>* PendingPZ = EvaluatedResult.Channels.Find(PositionZName);

	const bool bHasPositions = PendingPX && PendingPY && PendingPZ;
	const bool bHasVelocities = PendingVX && PendingVY && PendingVZ;

	if ( bHasPositions )
	{
		CacheData.CacheIndices = EvaluatedResult.ParticleIndices;
		const int32 NumParticles = EvaluatedResult.ParticleIndices.Num();

		CacheData.CachedPositions.SetNum(NumParticles);
		CacheData.CachedVelocities.SetNum(bHasVelocities ? NumParticles : 0);

		for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			CacheData.CachedPositions[ParticleIndex].X = (*PendingPX)[ParticleIndex];
			CacheData.CachedPositions[ParticleIndex].Y = (*PendingPY)[ParticleIndex];
			CacheData.CachedPositions[ParticleIndex].Z = (*PendingPZ)[ParticleIndex];
			if (bHasVelocities)
			{
				CacheData.CachedVelocities[ParticleIndex].X = (*PendingVX)[ParticleIndex];
				CacheData.CachedVelocities[ParticleIndex].Y = (*PendingVY)[ParticleIndex];
				CacheData.CachedVelocities[ParticleIndex].Z = (*PendingVZ)[ParticleIndex];
			}
		}
	}

	CacheData.CachedReferenceSpaceTransforms.Reset();
	for (const TPair<FName, FTransform>& NamedTransform : EvaluatedResult.NamedTransforms)
	{
		constexpr bool bNoCompareNumber = false;
		if (NamedTransform.Key.IsEqual(ReferenceTransformsName, ENameCase::CaseSensitive, bNoCompareNumber))
		{
			CacheData.CachedReferenceSpaceTransforms.Add(NamedTransform.Key.GetNumber(), NamedTransform.Value);
		}
	}
}

bool FClothingCacheSchema::CacheIsValidForPlayback(UChaosCache* InCache)
{
	// Old caches use TrackToParticle. New caches use ChannelCurveToParticle
	return InCache->TrackToParticle.Num() > 0 || InCache->ChannelCurveToParticle.Num() > 0; 
}

} // namespace Chaos