// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingPatternData.h"
#include "ChaosClothPrivate.h"
#include "Containers/ArrayView.h"
#include "Chaos/TriangleMesh.h"

namespace Chaos {

FClothingPatternData::FClothingPatternData(
	int32 InNumParticles,
	const TConstArrayView<uint32>& InIndices,
	const TConstArrayView<FVector2f>& InPatternPositions,
	const TConstArrayView<uint32>& InPatternIndices,
	const TConstArrayView<uint32>& InPatternToWeldedIndices)
	: PatternPositions(InPatternPositions)
	, PatternToWeldedIndices(InPatternToWeldedIndices)
{
	GenerateDerivedPatternData(InNumParticles, InIndices, InPatternIndices);
}

static void InitializeTriangleMesh(FTriangleMesh& TriangleMesh, const TConstArrayView<uint32>& Indices, const int32 NumParticles)
{
	checkSlow(Indices.Num() % 3 == 0);
	const int32 NumElements = Indices.Num() / 3;
	TArray<TVec3<int32>> Elements;
	Elements.Reserve(NumElements);

	for (int32 i = 0; i < NumElements; ++i)
	{
		const int32 Index = 3 * i;
		Elements.Add(
			{ static_cast<int32>(Indices[Index]),
			 static_cast<int32>(Indices[Index + 1]),
			 static_cast<int32>(Indices[Index + 2]) });
	}

	constexpr bool bCullDegenerateElementsFalse = false;
	TriangleMesh.Init(MoveTemp(Elements), 0, NumParticles - 1, bCullDegenerateElementsFalse);  // Init with the Offset to avoid discrepancies since the Triangle Mesh only relies on the used indices
}

void FClothingPatternData::Reset()
{
	// Resets all of the pattern data to empty.
	PatternPositions = TConstArrayView<FVector2f>();
	PatternToWeldedIndices = TConstArrayView<uint32>();
	InitializeTriangleMesh(PatternTriangleMesh, TConstArrayView<uint32>(), 0);
	WeldedFaceVertexPatternPositions.Reset();
}

void FClothingPatternData::GenerateDerivedPatternData(
	int32 NumParticles,
	const TConstArrayView<uint32>& Indices,
	const TConstArrayView<uint32>& PatternIndices)
{
	if (Indices.Num() != PatternIndices.Num())
	{
		// Reset all of the pattern data and return
		Reset();
		return;
	}

	const int32 NumPatternParticles = PatternPositions.Num();
	InitializeTriangleMesh(PatternTriangleMesh, PatternIndices, NumPatternParticles);

	if (NumPatternParticles)
	{
		FTriangleMesh NoOffsetWeldedMesh;
		InitializeTriangleMesh(NoOffsetWeldedMesh, Indices, NumParticles);
		const TArray<TVec3<int32>>& WeldedElements = NoOffsetWeldedMesh.GetElements();

		auto SortedElement = [](const TVec3<int32>& Element)
		{
			return TVec3<int32>(Element.Min(), Element.Mid(), Element.Max());
		};

		TArray<TVec3<int32>> SortedWeldedElements;
		SortedWeldedElements.SetNumUninitialized(WeldedElements.Num());
		for (int32 ElemIdx = 0; ElemIdx < WeldedElements.Num(); ++ElemIdx)
		{
			SortedWeldedElements[ElemIdx] = SortedElement(WeldedElements[ElemIdx]);
		}

		TConstArrayView<TArray<int32>> WeldedPointToTriangleMap = NoOffsetWeldedMesh.GetPointToTriangleMap();
		const TArray<TVec3<int32>>& PatternElements = PatternTriangleMesh.GetElements();

		TArray<TVec3<int32>> WeldedToPatternFaceVertexIndices;
		WeldedToPatternFaceVertexIndices.Init(TVec3<int32>(INDEX_NONE), NoOffsetWeldedMesh.GetNumElements());
		for(int32 PatternElemIdx = 0; PatternElemIdx < PatternElements.Num(); ++PatternElemIdx)
		{
			const TVec3<int32>& PatternElement = PatternElements[PatternElemIdx];

			// Find equivalent element in the welded mesh
			const TVec3<int32> WeldedPatternElement = TVec3<int32>(PatternToWeldedIndices[PatternElement[0]], PatternToWeldedIndices[PatternElement[1]], PatternToWeldedIndices[PatternElement[2]]);
			const TVec3<int32> SortedWeldedPatternElement = SortedElement(WeldedPatternElement);

			// Cloth asset 2D and 3D elements should match each other, but this wasn't always a requirement.
			int32 TriangleIndex = INDEX_NONE;
			if (SortedWeldedPatternElement == SortedWeldedElements[PatternElemIdx] &&
				WeldedToPatternFaceVertexIndices[PatternElemIdx] == TVec3<int32>(INDEX_NONE))
			{
				TriangleIndex = PatternElemIdx;
			}
			else
			{
				// Handle if that's not the case and try to find the matching triangle.
				// Assume that there is some one-to-one correspondence between triangles, so pick a triangle that hasn't already been found.
				const TArray<int32>& PossibleTriangles = WeldedPointToTriangleMap[WeldedPatternElement[0]];

				for (int32 Triangle : PossibleTriangles)
				{
					const TVec3<int32>& TriangleElements = SortedWeldedElements[Triangle];
					if (TriangleElements == SortedWeldedPatternElement && 
						WeldedToPatternFaceVertexIndices[Triangle] == TVec3<int32>(INDEX_NONE))
					{
						TriangleIndex = Triangle;
						break;
					}
				}
			}
			if (TriangleIndex == INDEX_NONE)
			{
				// We failed to find a corresponding triangle. This pattern data is invalid.
				UE_LOGF(LogChaosCloth, Warning, "Invalid pattern data. Failed to find a matching welded triangle. Panel-based cloth is disabled for this LOD.");

				// Reset all of the pattern data and return
				Reset();
				return;
			}

			const TVec3<int32>& WeldedElement = WeldedElements[TriangleIndex];
			auto MatchIndex = [&WeldedPatternElement, &WeldedElement](int32 WeldedIndex)
			{
				check(WeldedPatternElement[0] == WeldedElement[WeldedIndex] || WeldedPatternElement[1] == WeldedElement[WeldedIndex] || WeldedPatternElement[2] == WeldedElement[WeldedIndex]); // TODO: checkSlow
				return WeldedPatternElement[0] == WeldedElement[WeldedIndex] ? 0 : WeldedPatternElement[1] == WeldedElement[WeldedIndex] ? 1 : 2;
			};

			TVec3<int32>& FaceVertex = WeldedToPatternFaceVertexIndices[TriangleIndex];
			check(FaceVertex == TVec3<int32>(INDEX_NONE)); // TODO: checkSlow
			FaceVertex[0] = PatternElement[MatchIndex(0)];
			FaceVertex[1] = PatternElement[MatchIndex(1)];
			FaceVertex[2] = PatternElement[MatchIndex(2)];
		}
		WeldedFaceVertexPatternPositions.SetNumUninitialized(WeldedElements.Num());
		for (int32 ElemIdx = 0; ElemIdx < WeldedElements.Num(); ++ElemIdx)
		{
			const TVec3<int32>& FaceVertex = WeldedToPatternFaceVertexIndices[ElemIdx];

			// TODO: checkSlow
			const TVec3<int32>& WeldedElement = WeldedElements[ElemIdx];
			check(!(FaceVertex == TVec3<int32>(INDEX_NONE))); // This should be OK to be a check since we already know that NumTriangles match between pattern and welded and each pattern triangle corresponds with a unique welded triangle.
			check(PatternToWeldedIndices[FaceVertex[0]] == WeldedElement[0]);
			check(PatternToWeldedIndices[FaceVertex[1]] == WeldedElement[1]);
			check(PatternToWeldedIndices[FaceVertex[2]] == WeldedElement[2]);

			TVec3<FVec2f>& FaceVertexPositions = WeldedFaceVertexPatternPositions[ElemIdx];
			FaceVertexPositions[0] = PatternPositions[FaceVertex[0]];
			FaceVertexPositions[1] = PatternPositions[FaceVertex[1]];
			FaceVertexPositions[2] = PatternPositions[FaceVertex[2]];
		}
	}
	else
	{
		WeldedFaceVertexPatternPositions.Reset();
	}
}
}  // End namespace Chaos
