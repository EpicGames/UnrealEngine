// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothPrivate.h"

#include "ClothingSimulation.h"
#include "ClothingSimulationInstance.h"
#include "ClothingAsset.h"
#include "ChaosCloth/ChaosClothConfig.h"

#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationSkeletalMesh.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"

#include "PhysicsField/PhysicsFieldComponent.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"

#include "HAL/IConsoleManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "PrimitiveDrawingUtils.h"

#if !UE_BUILD_SHIPPING
#include "FramePro/FramePro.h"
#else
#define FRAMEPRO_ENABLED 0
#endif

#if INTEL_ISPC
#include "ChaosClothingSimulation.ispc.generated.h"
#endif

#if INTEL_ISPC && !UE_BUILD_SHIPPING  // Include only used for the Ispc command
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDEvolution.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/PerParticleDampVelocity.h"
#include "Chaos/PerParticlePBDCollisionConstraint.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/VelocityField.h"
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/XPBDBendingConstraints.h"
#include "Chaos/SoftsMultiResConstraints.h"
#endif

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector3f) == sizeof(FVector3f), "sizeof(ispc::FVector3f) != sizeof(FVector3f)");
static_assert(sizeof(ispc::FVector) == sizeof(Chaos::FVec3), "sizeof(ispc::FVector) != sizeof(Chaos::FVec3)");
static_assert(sizeof(ispc::FTransform) == sizeof(Chaos::FRigidTransform3), "sizeof(ispc::FTransform) != sizeof(Chaos::FRigidTransform3)");

#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_GetSimData_ISPC_Enabled = CHAOS_GET_SIM_DATA_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosGetSimDataISPCEnabled(TEXT("p.Chaos.GetSimData.ISPC"), bChaos_GetSimData_ISPC_Enabled, TEXT("Whether to use ISPC optimizations when getting simulation data"));
#endif
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Simulate"), STAT_ChaosClothSimulate, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Create Actor"), STAT_ChaosClothCreateActor, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Get Simulation Data"), STAT_ChaosClothGetSimulationData, STATGROUP_ChaosCloth);

namespace Chaos
{
bool bClothUseTimeStepSmoothing = true;
FAutoConsoleVariableRef CVarClothUseTimeStepSmoothing(TEXT("p.ChaosCloth.UseTimeStepSmoothing"), bClothUseTimeStepSmoothing, TEXT("Use time step smoothing to avoid jitter during drastic changes in time steps."));

bool bClothUseLegacyEvolution = true;
FAutoConsoleVariableRef CVarClothUseLegacyEvolution(TEXT("p.ChaosCloth.UseLegacyEvolution"), bClothUseLegacyEvolution, TEXT("Use legacy underlying evolution as opposed to the new evolution shared with the ChaosClothAsset."));

#if CHAOS_DEBUG_DRAW
namespace ClothingSimulationCVar
{
	TAutoConsoleVariable<int32> DebugDrawSingleVertex       (TEXT("p.ChaosCloth.DebugDrawSingleVertex"        ), -1,    TEXT("Filter Chaos Cloth debug draw to a single vertex. Default = -1 is no filtering."), ECVF_Cheat);
	TAutoConsoleVariable<FString> DebugDrawFilterSet        (TEXT("p.ChaosCloth.DebugDrawFilterSet"           ), "",    TEXT("Filter Chaos Cloth debug draw to a named sim vertex set. Default ='' is no filtering."), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawLocalSpace          (TEXT("p.ChaosCloth.DebugDrawLocalSpace"          ), false, TEXT("Whether to debug draw the Chaos Cloth local space"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawBounds              (TEXT("p.ChaosCloth.DebugDrawBounds"              ), false, TEXT("Whether to debug draw the Chaos Cloth bounds"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawGravity             (TEXT("p.ChaosCloth.DebugDrawGravity"             ), false, TEXT("Whether to debug draw the Chaos Cloth gravity acceleration vector"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawPhysMeshWired       (TEXT("p.ChaosCloth.DebugDrawPhysMeshWired"       ), false, TEXT("Whether to debug draw the Chaos Cloth wireframe meshes"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawAnimMeshWired       (TEXT("p.ChaosCloth.DebugDrawAnimMeshWired"       ), false, TEXT("Whether to debug draw the animated/kinematic Cloth wireframe meshes"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawAnimNormals         (TEXT("p.ChaosCloth.DebugDrawAmimNormals"         ), false, TEXT("Whether to debug draw the animated/kinematic Cloth normals"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawAnimVelocities      (TEXT("p.ChaosCloth.DebugDrawAnimVelocities"         ), false, TEXT("Whether to debug draw the animated/kinematic Cloth velocities"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawPointNormals        (TEXT("p.ChaosCloth.DebugDrawPointNormals"        ), false, TEXT("Whether to debug draw the Chaos Cloth point normals"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawPointVelocities     (TEXT("p.ChaosCloth.DebugDrawPointVelocities"     ), false, TEXT("Whether to debug draw the Chaos Cloth point velocities"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawFaceNormals         (TEXT("p.ChaosCloth.DebugDrawFaceNormals"         ), false, TEXT("Whether to debug draw the Chaos Cloth face normals"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawInversedFaceNormals (TEXT("p.ChaosCloth.DebugDrawInversedFaceNormals" ), false, TEXT("Whether to debug draw the Chaos Cloth inversed face normals"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawCollision           (TEXT("p.ChaosCloth.DebugDrawCollision"           ), false, TEXT("Whether to debug draw the Chaos Cloth collisions"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawBackstops           (TEXT("p.ChaosCloth.DebugDrawBackstops"           ), false, TEXT("Whether to debug draw the Chaos Cloth backstops"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawBackstopDistances   (TEXT("p.ChaosCloth.DebugDrawBackstopDistances"   ), false, TEXT("Whether to debug draw the Chaos Cloth backstop distances"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawMaxDistances        (TEXT("p.ChaosCloth.DebugDrawMaxDistances"        ), false, TEXT("Whether to debug draw the Chaos Cloth max distances"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawMaxDistanceValues   (TEXT("p.ChaosCloth.DebugDrawMaxDistanceValues"   ), false, TEXT("Whether to debug draw the Chaos Cloth max distances as numbers"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawAnimDrive           (TEXT("p.ChaosCloth.DebugDrawAnimDrive"           ), false, TEXT("Whether to debug draw the Chaos Cloth anim drive"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawEdgeConstraint      (TEXT("p.ChaosCloth.DebugDrawEdgeConstraint"      ), false, TEXT("Whether to debug draw the Chaos Cloth edge constraint"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawBendingConstraint   (TEXT("p.ChaosCloth.DebugDrawBendingConstraint"   ), false, TEXT("Whether to debug draw the Chaos Cloth bending constraint"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawLongRangeConstraint (TEXT("p.ChaosCloth.DebugDrawLongRangeConstraint" ), false, TEXT("Whether to debug draw the Chaos Cloth long range constraint (aka tether constraint)"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawWindForces          (TEXT("p.ChaosCloth.DebugDrawWindForces"          ), false, TEXT("Whether to debug draw the Chaos Cloth wind forces"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawWindVelocity        (TEXT("p.ChaosCloth.DebugDrawWindVelocity"        ), false, TEXT("Whether to debug draw the Chaos Cloth wind velocity"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawSelfCollision       (TEXT("p.ChaosCloth.DebugDrawSelfCollision"       ), false, TEXT("Whether to debug draw the Chaos Cloth self collision information"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawSelfIntersection    (TEXT("p.ChaosCloth.DebugDrawSelfIntersection"    ), false, TEXT("Whether to debug draw the Chaos Cloth self intersection information"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawSelfCollisionThickness(TEXT("p.ChaosCloth.DebugDrawSelfCollisionThickness"), false, TEXT("Whether to debug draw the Chaos Cloth self collision thickness"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawClothCollisionThickness(TEXT("p.ChaosCloth.DebugDrawClothCollisionThickness"), false, TEXT("Whether to debug draw the Chaos Cloth cloth collision thickness"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawInnerCollisionThickness(TEXT("p.ChaosCloth.DebugDrawInnerCollisionThickness"), false, TEXT("Whether to debug draw the Chaos Cloth inner collision thickness"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawParticleIndices     (TEXT("p.ChaosCloth.DebugDrawParticleIndices"     ), false, TEXT("Whether to debug draw the Chaos Cloth particle indices"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawElementIndices      (TEXT("p.ChaosCloth.DebugDrawElementIndices"      ), false, TEXT("Whether to debug draw the Chaos Cloth element indices"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawClothClothConstraints    (TEXT("p.ChaosCloth.DebugDrawClothClothConstraints"    ), false, TEXT("Whether to debug draw the Chaos Cloth-Cloth constraints"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawTeleportReset(TEXT("p.ChaosCloth.DebugDrawTeleportReset"), false, TEXT("Whether to debug draw the Chaos Cloth teleport/reset status"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawExtremlyDeformedEdges(TEXT("p.ChaosCloth.DebugDrawExtremlyDeformedEdges"), false, TEXT("Whether to debug draw edges that are deformed beyond EdgeDeformationThreshold"), ECVF_Cheat);
	TAutoConsoleVariable<bool> DebugDrawVelocityScale(TEXT("p.ChaosCloth.DebugDrawVelocityScale"), false, TEXT("Whether to debug draw the Chaos Cloth velocity scale"), ECVF_Cheat);
}
#endif  // #if CHAOS_DEBUG_DRAW

#if !UE_BUILD_SHIPPING
namespace ClothingSimulationConsole
{
#if INTEL_ISPC
	static FAutoConsoleCommand CommandIspc(
		TEXT("p.ChaosCloth.Ispc"),
		TEXT("Enable or disable ISPC optimizations for cloth simulation."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			bool bEnableISPC;
			switch (Args.Num())
			{
			default:
				break; // Invalid arguments
			case 1:
				if (Args[0] == TEXT("1") || Args[0] == TEXT("true") || Args[0] == TEXT("on"))
				{
					bEnableISPC = true;
				}
				else if (Args[0] == TEXT("0") || Args[0] == TEXT("false") || Args[0] == TEXT("off"))
				{
					bEnableISPC = false;
				}
				else
				{
					break; // Invalid arguments
				}
				bChaos_AxialSpring_ISPC_Enabled =
					bChaos_LongRange_ISPC_Enabled =
					bChaos_Spherical_ISPC_Enabled =
					bChaos_Spring_ISPC_Enabled =
					bChaos_DampVelocity_ISPC_Enabled =
					bChaos_PerParticleCollision_ISPC_Enabled =
					bChaos_VelocityField_ISPC_Enabled =
					bChaos_GetSimData_ISPC_Enabled =
					bChaos_SkinPhysicsMesh_ISPC_Enabled =
					bChaos_PreSimulationTransforms_ISPC_Enabled =
					bChaos_CalculateBounds_ISPC_Enabled =
					bChaos_PostIterationUpdates_ISPC_Enabled =
					bChaos_Bending_ISPC_Enabled =
					bChaos_XPBDSpring_ISPC_Enabled =
					bChaos_XPBDBending_ISPC_Enabled =
					bChaos_MultiRes_ISPC_Enabled =
					bChaos_KinematicTriangleMesh_ISPC_Enabled =
					bEnableISPC;
				return;
			}

			UE_LOGF(LogChaosCloth, Display, "Invalid arguments.");
			UE_LOGF(LogChaosCloth, Display, "Usage:");
			UE_LOGF(LogChaosCloth, Display, "  p.ChaosCloth.Ispc [0|1]|[true|false]|[on|off]");
			UE_LOGF(LogChaosCloth, Display, "Example: p.Chaos.Ispc on");
		}),
		ECVF_Cheat);
#endif // #if INTEL_ISPC

#if !UE_BUILD_TEST
	class FCommand final
	{
	public:
		FCommand()
			: StepCount(0)
			, bIsPaused(false)
			, ResetCount(0)
		{
			// Register DebugStep console command
			ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("p.ChaosCloth.DebugStep"),
				TEXT("Pause/step/resume cloth simulations."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FCommand::DebugStep),
				ECVF_Cheat));

			// Register Reset console command
			ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("p.ChaosCloth.Reset"),
				TEXT("Reset all cloth simulations."),
				FConsoleCommandDelegate::CreateRaw(this, &FCommand::Reset),
				ECVF_Cheat));
		}

		~FCommand()
		{
			for (IConsoleObject* ConsoleObject : ConsoleObjects)
			{
				IConsoleManager::Get().UnregisterConsoleObject(ConsoleObject);
			}
		}

		bool MustStep(int32& InOutStepCount) const
		{
			const bool bMustStep = !bIsPaused || (InOutStepCount != StepCount);
			InOutStepCount = StepCount;
			return bMustStep;
		}

		bool MustReset(int32& InOutResetCount) const
		{
			const bool bMustReset = (InOutResetCount != ResetCount);
			InOutResetCount = ResetCount;
			return bMustReset;
		}

	private:
		void DebugStep(const TArray<FString>& Args)
		{
			switch (Args.Num())
			{
			default:
				break;  // Invalid arguments
			case 1:
				if (Args[0].Compare(TEXT("Pause"), ESearchCase::IgnoreCase) == 0)
				{
					UE_CLOGF(bIsPaused, LogChaosCloth, Warning, "Cloth simulations are already paused!");
					UE_CLOGF(!bIsPaused, LogChaosCloth, Display, "Cloth simulations are now pausing...");
					bIsPaused = true;
				}
				else if (Args[0].Compare(TEXT("Step"), ESearchCase::IgnoreCase) == 0)
				{
					if (bIsPaused)
					{
						UE_LOGF(LogChaosCloth, Display, "Cloth simulations are now stepping...");
						++StepCount;
					}
					else
					{
						UE_LOGF(LogChaosCloth, Warning, "The Cloth simulations aren't paused yet!");
						UE_LOGF(LogChaosCloth, Display, "Cloth simulations are now pausing...");
						bIsPaused = true;
					}
				}
				else if (Args[0].Compare(TEXT("Resume"), ESearchCase::IgnoreCase) == 0)
				{
					UE_CLOGF(!bIsPaused, LogChaosCloth, Warning, "Cloth simulations haven't been paused yet!");
					UE_CLOGF(bIsPaused, LogChaosCloth, Display, "Cloth simulations are now resuming...");
					bIsPaused = false;
				}
				else
				{
					break;  // Invalid arguments
				}
				return;
			}
			UE_LOGF(LogChaosCloth, Display, "Invalid arguments.");
			UE_LOGF(LogChaosCloth, Display, "Usage:");
			UE_LOGF(LogChaosCloth, Display, "  p.ChaosCloth.DebugStep [Pause|Step|Resume]");
		}

		void Reset()
		{
			UE_LOGF(LogChaosCloth, Display, "All cloth simulations have now been asked to reset.");
			++ResetCount;
		}

	private:
		TArray<IConsoleObject*> ConsoleObjects;
		TAtomic<int32> StepCount;
		TAtomic<bool> bIsPaused;
		TAtomic<int32> ResetCount;
	};
	static TUniquePtr<FCommand> Command;
#endif // #if !UE_BUILD_TEST
}
#endif  // #if !UE_BUILD_SHIPPING

// Default parameters, will be overwritten when cloth assets are loaded
namespace ChaosClothingSimulationDefault
{
	static const FVector Gravity(0.f, 0.f, -980.665f);
	static const FReal MaxDistancesMultipliers = (FReal)1.;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For Visualization
FClothingSimulation::FClothingSimulation()
	: ClothSharedSimConfig(nullptr)
	, bUseLocalSpaceSimulation(true)
	, bUseGravityOverride(false)
	, GravityOverride(ChaosClothingSimulationDefault::Gravity)
	, MaxDistancesMultipliers(ChaosClothingSimulationDefault::MaxDistancesMultipliers)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, StepCount(0)
	, ResetCount(0)
#endif
	, bHasInvalidReferenceBoneTransforms(false)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!ClothingSimulationConsole::Command)
	{
		ClothingSimulationConsole::Command = MakeUnique<ClothingSimulationConsole::FCommand>();
	}
#endif
}

FClothingSimulation::~FClothingSimulation()
{}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FClothingSimulation::Initialize()
{
	// Create solver
	constexpr FClothingSimulationConfig* InConfig = nullptr;
	Solver = MakeUnique<FClothingSimulationSolver>(InConfig, bClothUseLegacyEvolution);

	// Assign the solver to the visualization
	Visualization.SetSolver(Solver.Get());

	ResetStats();
}

void FClothingSimulation::Shutdown()
{
	Visualization.SetSolver(nullptr);
	Solver.Reset();
	Meshes.Reset();
	Cloths.Reset();
	Colliders.Reset();
	Configs.Reset();
	ClothSharedSimConfig = nullptr;
}

void FClothingSimulation::DestroyActors()
{
	Shutdown();
	Initialize();
}

// Have to also use the pragma here due to some compiler weirdness
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
IClothingSimulationContext* FClothingSimulation::CreateContext()
{
	return new FClothingSimulationContext();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FClothingSimulation::CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* InAsset, int32 InSimDataIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulation_CreateActor);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothCreateActor);

	check(InOwnerComponent);
	check(Solver);

	if (!InAsset)
	{
		return;
	}

	// ClothSharedSimConfig should either be a nullptr, or point to an object common to the whole skeletal mesh
	const UClothingAssetCommon* const Asset = Cast<UClothingAssetCommon>(InAsset);
	if (!ClothSharedSimConfig)
	{
		ClothSharedSimConfig = Asset->GetClothConfig<UChaosClothSharedSimConfig>();

		UpdateSimulationFromSharedSimConfig();

		// Must set the local space location prior to adding any mesh/cloth, as otherwise the start poses would be in the wrong local space
		const FClothingSimulationContext* const Context = GetClothingSimulationContext(InOwnerComponent);
		check(Context);
		static const bool bReset = true;
		const FReal WorldToSolverScale = bUseLocalSpaceSimulation ? Context->SolverGeometryScale : 1.;
		const FReal LocalSpaceScale = 1. / FMath::Max(WorldToSolverScale, UE_SMALL_NUMBER);
		Solver->SetLocalSpaceScale(LocalSpaceScale, bReset);
		Solver->SetLocalSpaceLocation(bUseLocalSpaceSimulation ? (FVec3)Context->ComponentToWorld.GetLocation() : FVec3(0.), bReset);
		Solver->SetLocalSpaceRotation(bUseLocalSpaceSimulation ? (FQuat)Context->ComponentToWorld.GetRotation() : FQuat::Identity);
	}
	else
	{
		check(ClothSharedSimConfig == Asset->GetClothConfig<UChaosClothSharedSimConfig>());
	}

	// Retrieve the cloth config stored in the asset
	const UChaosClothConfig* const ClothConfig = Asset->GetClothConfig<UChaosClothConfig>();
	if (!ClothConfig)
	{
		UE_LOGF(LogChaosCloth, Warning, "Missing Chaos config Cloth LOD asset to %ls in sim slot %d", InOwnerComponent->GetOwner() ? *InOwnerComponent->GetOwner()->GetName() : TEXT("None"), InSimDataIndex);
		return;
	}

	// Create mesh runtime simulation object
	const int32 MeshIndex = Meshes.Emplace(MakeUnique<FClothingSimulationSkeletalMesh>(
		Asset,
		InOwnerComponent));

	// Create collider runtime simulation object
	const FReferenceSkeleton* const ReferenceSkeleton = &CastChecked<USkeletalMesh>(Asset->GetOuter())->GetRefSkeleton();
	const int32 ColliderIndex = Colliders.Emplace(MakeUnique<FClothingSimulationCollider>(
		Asset->PhysicsAsset,
		ReferenceSkeleton));

	// Set the external collision data to get updated at every frame
	Colliders[ColliderIndex]->SetCollisionData(&ExternalCollisionData);

	// Create cloth config runtime simulation object
	const int32 ClothConfigIndex = Configs.Emplace(MakeUnique<FClothingSimulationConfig>());
	constexpr bool bUseLegacyConfig = true;  // Make the config a legacy cloth config, so that the constraints disable themselves with missing masks, ...etc.
	Configs[ClothConfigIndex]->Initialize(ClothConfig, nullptr, bUseLegacyConfig);

	// Create cloth runtime simulation object
	const int32 ClothIndex = Cloths.Emplace(MakeUnique<FClothingSimulationCloth>(
		Configs[ClothConfigIndex].Get(),
		Meshes[MeshIndex].Get(),
		TArray<FClothingSimulationCollider*>({ Colliders[ColliderIndex].Get() }),
		InSimDataIndex));

	// Add cloth to solver
	Solver->AddCloth(Cloths[ClothIndex].Get());

	// Update stats
	UpdateStats(Cloths[ClothIndex].Get());

	UE_LOGF(LogChaosCloth, Verbose, "Added Cloth asset to %ls in sim slot %d", InOwnerComponent->GetOwner() ? *InOwnerComponent->GetOwner()->GetName() : TEXT("None"), InSimDataIndex);
}

void FClothingSimulation::EndCreateActor()
{
	LODHasAnyRenderClothMappingData.Reset();
	for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
	{
		const int32 AssetIndex = Cloth->GetGroupId();
		if (const FClothingSimulationSkeletalMesh* const SimulationMesh = static_cast<const FClothingSimulationSkeletalMesh*>(Cloth->GetMesh()))
		{
			if (const USkeletalMeshComponent* const OwnerComponent = SimulationMesh->GetSkeletalMeshComponent())
			{
				if (const USkeletalMesh* const SkeletalMesh = OwnerComponent->GetSkeletalMeshAsset())
				{
					if (const FSkeletalMeshRenderData* const SkeletalMeshRenderData = SkeletalMesh->GetResourceForRendering())
					{
						if (LODHasAnyRenderClothMappingData.Num() < SkeletalMeshRenderData->LODRenderData.Num())
						{
							LODHasAnyRenderClothMappingData.SetNum(SkeletalMeshRenderData->LODRenderData.Num(), false);
						}
						for (int32 LodIndex = 0; LodIndex < SkeletalMeshRenderData->LODRenderData.Num(); ++LodIndex)
						{
							if (LODHasAnyRenderClothMappingData[LodIndex])
							{
								continue;
							}
							for (const FSkelMeshRenderSection& SkelMeshRenderSection : SkeletalMeshRenderData->LODRenderData[LodIndex].RenderSections)
							{
								if (SkelMeshRenderSection.HasClothingData() &&
									SkelMeshRenderSection.CorrespondClothAssetIndex == AssetIndex)
								{
									LODHasAnyRenderClothMappingData[LodIndex] = true;
									continue;
								}
							}
						}
					}
				}
			}
		}
	}
}

void FClothingSimulation::ResetStats()
{
	check(Solver);
	NumCloths = 0;
	NumKinematicParticles = 0;
	NumDynamicParticles = 0;
	SimulationTime = 0.f;
	NumSubsteps = Solver->GetNumSubsteps();
	NumIterations = Solver->GetNumIterations();
	bHasInvalidReferenceBoneTransforms = false;
}

void FClothingSimulation::UpdateStats(const FClothingSimulationCloth* Cloth)
{
	NumCloths = Cloths.Num();
	NumKinematicParticles += Cloth->GetNumActiveKinematicParticles();
	NumDynamicParticles += Cloth->GetNumActiveDynamicParticles();
}

void FClothingSimulation::UpdateSimulationFromSharedSimConfig()
{
	check(Solver);
	if (ClothSharedSimConfig) // ClothSharedSimConfig will be a null pointer if all cloth instances are disabled in which case we will use default Evolution parameters
	{
		FClothingSimulationConfig* SolverConfig = Solver->GetConfig();
		if (!SolverConfig)
		{
			// Create solver config runtime simulation object
			const int32 SolverConfigIndex = Configs.Emplace(MakeUnique<FClothingSimulationConfig>());
			SolverConfig = Configs[SolverConfigIndex].Get();
			Solver->SetConfig(SolverConfig);
		}
		constexpr bool bUseLegacyConfig = true;  // Make the config a legacy cloth config, so that the constraints disable themselves with missing masks, ...etc.
		SolverConfig->Initialize(nullptr, ClothSharedSimConfig, bUseLegacyConfig);
	}
	else
	{
		// This will cause the solver to create a default config
		Solver->SetConfig(nullptr);
	}
}

void FClothingSimulation::FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly)
{
	if (bForceTeleportResetOnly)
	{
		// Nothing to do in this case.
		return;
	}

	// Update world forces
	if (InComponent)
	{
		UWorld* OwnerWorld = InComponent->GetWorld();
		if (OwnerWorld && OwnerWorld->PhysicsField && Solver)
		{
			const FBox BoundingBox = InComponent->CalcBounds(InComponent->GetComponentTransform()).GetBox();

			OwnerWorld->PhysicsField->FillTransientCommands(false, BoundingBox, Solver->GetTime(), Solver->GetPerSolverField().GetTransientCommands());
			OwnerWorld->PhysicsField->FillPersistentCommands(false, BoundingBox, Solver->GetTime(), Solver->GetPerSolverField().GetPersistentCommands());
		}
	}

	// Fill context
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	FClothingSimulationCommon::FillContextAndPrepareTick(InComponent, InDeltaTime, InOutContext, bIsInitialization, bForceTeleportResetOnly);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FClothingSimulation::ShouldSimulateLOD(int32 OwnerLODIndex) const
{
	if (!LODHasAnyRenderClothMappingData.IsValidIndex(OwnerLODIndex) || !LODHasAnyRenderClothMappingData[OwnerLODIndex])
	{
		return false;
	}

	for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
	{
		if (const FClothingSimulationMesh* const SimulationMesh = Cloth->GetMesh())
		{
			const int32 ClothLODIndex = SimulationMesh->GetLODIndexFromOwnerLODIndex(OwnerLODIndex);
			if (SimulationMesh->IsValidLODIndex(ClothLODIndex) && SimulationMesh->GetNumPoints(ClothLODIndex) > 0)
			{
				return true;
			}
		}
	}
	return false;
}

void FClothingSimulation::ForceClothNextUpdateTeleportAndReset_AnyThread()
{
	for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
	{
		Cloth->Reset();
	}
}

void FClothingSimulation::Simulate_AnyThread(const IClothingSimulationContext* InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulation_Simulate);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (ClothingSimulationConsole::Command && ClothingSimulationConsole::Command->MustStep(StepCount))
#endif
	{
		SCOPE_CYCLE_COUNTER(STAT_ChaosClothSimulate);
		const FClothingSimulationContext* const Context = static_cast<const FClothingSimulationContext*>(InContext);
		if (Context->DeltaSeconds == 0.f)
		{
			return;
		}

		// Filter delta time to smoothen time variations and prevent unwanted vibrations
		constexpr Softs::FSolverReal DeltaTimeDecay = (Softs::FSolverReal)0.1;
		const Softs::FSolverReal DeltaTime = (Softs::FSolverReal)Context->DeltaSeconds;
		const Softs::FSolverReal PrevDeltaTime = Solver->GetDeltaTime();
		const Softs::FSolverReal SmoothedDeltaTime = bClothUseTimeStepSmoothing ?
			PrevDeltaTime + (DeltaTime - PrevDeltaTime) * DeltaTimeDecay :
			DeltaTime;

		const double StartTime = FPlatformTime::Seconds();
		const float PrevSimulationTime = SimulationTime;  // Copy the atomic to prevent a re-read

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		const bool bNeedsReset = (ClothingSimulationConsole::Command && ClothingSimulationConsole::Command->MustReset(ResetCount)) ||
			Context->TeleportMode >= EClothingTeleportMode::TeleportAndReset || PrevSimulationTime == 0.f;
#else
		const bool bNeedsReset = Context->TeleportMode >= EClothingTeleportMode::TeleportAndReset || PrevSimulationTime == 0.f;
#endif
		const bool bNeedsTeleport = (Context->TeleportMode > EClothingTeleportMode::None);
		bIsTeleported = bNeedsTeleport;

		// Update Solver animatable parameters
		Solver->SetLocalSpaceLocation(bUseLocalSpaceSimulation ? (FVec3)Context->ComponentToWorld.GetLocation() : FVec3(0), bNeedsReset);
		Solver->SetLocalSpaceRotation(bUseLocalSpaceSimulation ? (FQuat)Context->ComponentToWorld.GetRotation() : FQuat::Identity);
		Solver->SetWindVelocity(Context->WindVelocity, Context->WindAdaption);
		Solver->SetGravity(bUseGravityOverride ? GravityOverride : Context->WorldGravity);
		Solver->EnableClothGravityOverride(!bUseGravityOverride);  // Disable all cloth gravity overrides when the interactor takes over
		Solver->SetVelocityScale(!bNeedsReset ? (FReal)Context->VelocityScale * (FReal)SmoothedDeltaTime / DeltaTime : 1.f);

		// Check teleport modes
		for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
		{
			// Update Cloth animatable parameters while in the cloth loop
			Cloth->SetMaxDistancesMultiplier(Context->MaxDistanceScale);

			if (bNeedsReset)
			{
				Cloth->Reset();
			}
			if (bNeedsTeleport)
			{
				Cloth->Teleport();
			}
		}

		// Step the simulation
		if (Solver->GetEnableSolver() || !Context->CacheData.HasData())
		{
			Solver->Update(SmoothedDeltaTime);

			for (TUniquePtr<FClothingSimulationConfig>& Config : Configs)
			{
				Config->GetProperties().ClearDirtyFlags();
			}
		}
		else
		{
			Solver->UpdateFromCache(Context->CacheData);
		}

		// Keep the actual used number of iterations for the stats
		NumIterations = Solver->GetNumUsedIterations();

		// Update simulation time in ms (and provide an instant average instead of the value in real-time)
		const float CurrSimulationTime = (float)((FPlatformTime::Seconds() - StartTime) * 1000.);
		static const float SimulationTimeDecay = 0.03f; // 0.03 seems to provide a good rate of update for the instant average
		SimulationTime = PrevSimulationTime ? PrevSimulationTime + (CurrSimulationTime - PrevSimulationTime) * SimulationTimeDecay : CurrSimulationTime;

#if FRAMEPRO_ENABLED
		FRAMEPRO_CUSTOM_STAT("ChaosClothSimulationTimeMs", SimulationTime, "ChaosCloth", "ms", FRAMEPRO_COLOUR(0,128,255));
		FRAMEPRO_CUSTOM_STAT("ChaosClothNumDynamicParticles", NumDynamicParticles, "ChaosCloth", "Particles", FRAMEPRO_COLOUR(0,128,128));
		FRAMEPRO_CUSTOM_STAT("ChaosClothNumKinematicParticles", NumKinematicParticles, "ChaosCloth", "Particles", FRAMEPRO_COLOUR(128, 0, 128));
#endif
	}

	// Debug draw
#if CHAOS_DEBUG_DRAW
	FClothVisualizationNoGC::FDrawContext DrawContext;
	DrawContext.Filter.SingleVertex = ClothingSimulationCVar::DebugDrawSingleVertex.GetValueOnAnyThread();
	DrawContext.Filter.VertexSet = ClothingSimulationCVar::DebugDrawFilterSet.GetValueOnAnyThread();
	FClothVisualizationNoGC::FDrawTextsContext DrawTextsContext;
	DrawTextsContext.Filter = DrawContext.Filter;
	
	if (ClothingSimulationCVar::DebugDrawLocalSpace          .GetValueOnAnyThread()) { Visualization.DrawLocalSpace          (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawBounds              .GetValueOnAnyThread()) { Visualization.DrawBounds              (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawGravity             .GetValueOnAnyThread()) { Visualization.DrawGravity             (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawPhysMeshWired       .GetValueOnAnyThread()) { Visualization.DrawPhysMeshWired       (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawAnimMeshWired       .GetValueOnAnyThread()) { Visualization.DrawAnimMeshWired       (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawPointVelocities     .GetValueOnAnyThread()) { Visualization.DrawPointVelocities     (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawAnimNormals         .GetValueOnAnyThread()) { Visualization.DrawAnimNormals         (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawAnimVelocities      .GetValueOnAnyThread()) { Visualization.DrawAnimVelocities      (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawPointNormals        .GetValueOnAnyThread()) { Visualization.DrawPointNormals        (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawCollision           .GetValueOnAnyThread()) { Visualization.DrawCollision           (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawBackstops           .GetValueOnAnyThread()) { Visualization.DrawBackstops           (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawBackstopDistances   .GetValueOnAnyThread()) { Visualization.DrawBackstopDistances   (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawMaxDistances        .GetValueOnAnyThread()) { Visualization.DrawMaxDistances        (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawMaxDistanceValues   .GetValueOnAnyThread()) { Visualization.DrawMaxDistanceValues   (DrawTextsContext); }
	if (ClothingSimulationCVar::DebugDrawAnimDrive           .GetValueOnAnyThread()) { Visualization.DrawAnimDrive           (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawEdgeConstraint      .GetValueOnAnyThread()) { Visualization.DrawEdgeConstraint      (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawBendingConstraint   .GetValueOnAnyThread()) { Visualization.DrawBendingConstraint   (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawLongRangeConstraint .GetValueOnAnyThread()) { Visualization.DrawLongRangeConstraint (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawWindForces          .GetValueOnAnyThread()) { Visualization.DrawWindAndPressureForces(DrawContext); }
	if (ClothingSimulationCVar::DebugDrawWindVelocity        .GetValueOnAnyThread()) { Visualization.DrawWindVelocity        (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawSelfCollision       .GetValueOnAnyThread()) { Visualization.DrawSelfCollision       (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawSelfIntersection    .GetValueOnAnyThread()) { Visualization.DrawSelfIntersection    (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawParticleIndices     .GetValueOnAnyThread()) { Visualization.DrawParticleIndices     (DrawTextsContext); }
	if (ClothingSimulationCVar::DebugDrawElementIndices      .GetValueOnAnyThread()) { Visualization.DrawElementIndices      (DrawTextsContext); }
	if (ClothingSimulationCVar::DebugDrawTeleportReset       .GetValueOnAnyThread()) { Visualization.DrawTeleportReset       (DrawContext); }
	if (ClothingSimulationCVar::DebugDrawExtremlyDeformedEdges.GetValueOnAnyThread()) { Visualization.DrawExtremlyDeformedEdges(DrawContext); }
	if (ClothingSimulationCVar::DebugDrawVelocityScale       .GetValueOnAnyThread()) { Visualization.DrawVelocityScale       (DrawContext); }
#endif  // #if CHAOS_DEBUG_DRAW
}

void FClothingSimulation::AppendSimulationData(
	TMap<int32, FClothSimulData>& OutData,
	const USkeletalMeshComponent* InOwnerComponent,
	const USkinnedMeshComponent* InOverrideComponent) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulation_GetSimulationData);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothGetSimulationData);


	// Get the solver's local space
	const FVec3& LocalSpaceLocation = Solver->GetLocalSpaceLocation(); // Note: Since the ReferenceSpaceTransform can be suspended with the simulation, it is important that the suspended local space location is used too in order to get the simulation data back into reference space
	const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();

	// Retrieve the component transforms
	const FClothingSimulationContext* const Context = GetClothingSimulationContext(InOwnerComponent);
	check(Context);  // A simulation can't be created without context
	const FTransform& OwnerTransform = Context->ComponentToWorld;

	const TArray<FTransform>& ComponentSpaceTransforms = InOverrideComponent ? InOverrideComponent->GetComponentSpaceTransforms() : InOwnerComponent->GetComponentSpaceTransforms();

	// Set the simulation data for each of the cloths
	for (const TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
	{
		const int32 AssetIndex = Cloth->GetGroupId();

		if (!Cloth->GetMesh())
		{
			OutData.Remove(AssetIndex);  // Ensures that the cloth vertex factory won't run unnecessarily
			continue;  // Invalid or empty cloth
		}

		// If the LOD has changed while the simulation is suspended, the cloth still needs to be updated with the correct LOD data
		const int32 LODIndex = Cloth->GetMesh()->GetLODIndex();
		if (LODIndex != Cloth->GetLODIndex(Solver.Get()))
		{
			Solver->Update(FSolverReal(0.));  // Update for LOD switching, but do not simulate
		}

		if (Cloth->GetParticleRangeId(Solver.Get()) == INDEX_NONE || Cloth->GetLODIndex(Solver.Get()) == INDEX_NONE)
		{
			OutData.Remove(AssetIndex);  // Ensures that the cloth vertex factory won't run unnecessarily
			continue;  // No valid LOD, there's nothing to write out
		}

		// Get the reference bone index for this cloth
		const int32 ReferenceBoneIndex = InOverrideComponent ? InOwnerComponent->GetLeaderBoneMap()[Cloth->GetReferenceBoneIndex()] : Cloth->GetReferenceBoneIndex();
		if (!ComponentSpaceTransforms.IsValidIndex(ReferenceBoneIndex))
		{
			UE_CLOGF(!bHasInvalidReferenceBoneTransforms, LogSkeletalMesh, Warning, "Failed to write back clothing simulation data for component %ls as bone transforms are invalid.", *InOwnerComponent->GetName());
			bHasInvalidReferenceBoneTransforms = true;
			return;
		}

		// Get the reference transform used in the current animation pose
		FTransform ReferenceBoneTransform = ComponentSpaceTransforms[ReferenceBoneIndex];
		ReferenceBoneTransform *= OwnerTransform;
		const bool bIsMirrored = (ReferenceBoneTransform.GetDeterminant() < 0);
		ReferenceBoneTransform.SetScale3D(FVector(1.0f));  // Scale is already baked in the cloth mesh

		// Set the world space transform to be this cloth's reference bone
		FClothSimulData& Data = OutData.FindOrAdd(AssetIndex);
		Data.Transform = ReferenceBoneTransform;
		Data.ComponentRelativeTransform = ReferenceBoneTransform.GetRelativeTransform(OwnerTransform);

		// Retrieve the last reference space transform used for this cloth
		// Note: This won't necessary match the current bone reference transform when the simulation is paused,
		//       and still allows for the correct positioning of the sim data while the component is animated.
		const FRigidTransform3& ReferenceSpaceTransform = Cloth->GetReferenceSpaceTransform();

		// Copy positions and normals
		Data.Positions = Cloth->GetParticlePositions(Solver.Get());
		Data.Normals = Cloth->GetParticleNormals(Solver.Get());

		// Transform into the cloth reference simulation space used at the time of simulation
#if INTEL_ISPC
		if (bChaos_GetSimData_ISPC_Enabled)
		{
			// ISPC is assuming float input here
			check(sizeof(ispc::FVector3f) == Data.Positions.GetTypeSize());
			check(sizeof(ispc::FVector3f) == Data.Normals.GetTypeSize());

			ispc::GetClothingSimulationData(
				(ispc::FVector3f*)Data.Positions.GetData(),
				(ispc::FVector3f*)Data.Normals.GetData(),
				(ispc::FTransform&)ReferenceSpaceTransform,
				(ispc::FVector&)LocalSpaceLocation,
				LocalSpaceScale,
				Data.Positions.Num());
		}
		else
#endif
		{
			for (int32 Index = 0; Index < Data.Positions.Num(); ++Index)
			{
				using FPositionsType = decltype(Data.Positions)::ElementType;
				using FNormalsType = decltype(Data.Normals)::ElementType;
				Data.Positions[Index] = FPositionsType(ReferenceSpaceTransform.InverseTransformPosition(LocalSpaceScale * FVec3(Data.Positions[Index]) + LocalSpaceLocation));  // Move into world space first
				Data.Normals[Index] = FNormalsType(ReferenceSpaceTransform.InverseTransformVector(FVec3(-Data.Normals[Index])));  // Normals are inverted due to how barycentric coordinates are calculated (see GetPointBaryAndDist in ClothingMeshUtils.cpp)
			}
		}

		// Invert normals in mirrored setups
		if (bIsMirrored)
		{
			using FNormalsType = decltype(Data.Normals)::ElementType;
			for (FNormalsType& Normal : Data.Normals)
			{
				Normal = -Normal;
			}
		}

		// Set the current LOD these data apply to, so that the correct deformer mappings can be applied
		Data.LODIndex = Cloth->GetMesh()->GetOwnerLODIndex(LODIndex);  // The owner component LOD index can be different to the cloth mesh LOD index
	}
}

FBoxSphereBounds FClothingSimulation::GetBounds(const USkeletalMeshComponent* InOwnerComponent) const
{
	check(Solver);
	FBoxSphereBounds Bounds = Solver->CalculateBounds();

	if (bUseLocalSpaceSimulation)
	{
		// The component could be moving while the simulation is suspended so getting the bounds
		// in world space isn't good enough and the bounds origin needs to be continuously updated
		// Intentionally not applying LocalSpaceScale here because LocalSpaceScale does not match component scale.
		Bounds = Bounds.TransformBy(FTransform((FQuat)Solver->GetLocalSpaceRotation(), (FVector)Solver->GetLocalSpaceLocation()).Inverse());
	}
	else if (InOwnerComponent)
	{
		const FClothingSimulationContext* const Context = GetClothingSimulationContext(InOwnerComponent);
		check(Context);  // A simulation can't be created without context
		const FTransform& OwnerTransform = Context->ComponentToWorld;
		// Return local bounds
		Bounds = Bounds.TransformBy(OwnerTransform.Inverse());
	}
	// Else return bounds in world space
	return Bounds;
}

void FClothingSimulation::AddExternalCollisions(const FClothCollisionData& InData)
{
	ExternalCollisionData.Append(InData);
}

void FClothingSimulation::ClearExternalCollisions()
{
	ExternalCollisionData.Reset();
}

void FClothingSimulation::RefreshClothConfig(const IClothingSimulationContext* InContext)
{
	UpdateSimulationFromSharedSimConfig();

	// Update new space location
	const FClothingSimulationContext* const Context = static_cast<const FClothingSimulationContext*>(InContext);
	static const bool bReset = true;
	const FReal WorldToSolverScale = bUseLocalSpaceSimulation ? Context->SolverGeometryScale : 1.;
	const FReal LocalSpaceScale = 1. / FMath::Max(WorldToSolverScale, UE_SMALL_NUMBER);
	Solver->SetLocalSpaceScale(LocalSpaceScale, bReset);
	Solver->SetLocalSpaceLocation(bUseLocalSpaceSimulation ? (FVec3)Context->ComponentToWorld.GetLocation() : FVec3(0), bReset);
	Solver->SetLocalSpaceRotation(bUseLocalSpaceSimulation ? (FQuat)Context->ComponentToWorld.GetRotation() : FQuat::Identity);

	// Reset stats
	ResetStats();

	// Clear all cloths from the solver
	Solver->RemoveCloths();

	// Recreate all cloths
	for (TUniquePtr<FClothingSimulationCloth>& Cloth : Cloths)
	{
		FClothingSimulationSkeletalMesh* const Mesh = static_cast<FClothingSimulationSkeletalMesh*>(Cloth->GetMesh());
		TArray<FClothingSimulationCollider*> ClothColliders = Cloth->GetColliders();
		const uint32 GroupId = Cloth->GetGroupId();
		const UChaosClothConfig* const ClothConfig = Mesh->GetAsset()->GetClothConfig<UChaosClothConfig>();

		// Update cloth config runtime simulation object
		FClothingSimulationConfig* const Config = Cloth->GetConfig();
		check(Config);
		constexpr bool bUseLegacyConfig = true;  // Make the config a legacy cloth config, so that the constraints disable themselves with missing masks, ...etc.
		Config->Initialize(ClothConfig, nullptr, bUseLegacyConfig);

		// Recreate cloth runtime simulation object
		Cloth = MakeUnique<FClothingSimulationCloth>(
			Config,
			Mesh,
			MoveTemp(ClothColliders),
			GroupId);

		// Re-add cloth to the solver
		Solver->AddCloth(Cloth.Get());

		// Update stats
		UpdateStats(Cloth.Get());
	}
	UE_LOGF(LogChaosCloth, VeryVerbose, "RefreshClothConfig, all constraints and self-collisions have been updated for all clothing assets and LODs.");
}

void FClothingSimulation::RefreshPhysicsAsset()
{
	// A collider update cannot be re-triggered for now, refresh all cloths from the solver instead
	Solver->RefreshCloths();

	UE_LOGF(LogChaosCloth, VeryVerbose, "RefreshPhysicsAsset, all collisions have been re-added for all clothing assets");
}

void FClothingSimulation::SetGravityOverride(const FVector& InGravityOverride)
{
	bUseGravityOverride = true;
	GravityOverride = InGravityOverride;
}

void FClothingSimulation::DisableGravityOverride()
{
	bUseGravityOverride = false;
}

// Have to also use the pragma here due to some compiler weirdness
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
FClothingSimulationCloth* FClothingSimulation::GetCloth(int32 ClothId)
{
	TUniquePtr<FClothingSimulationCloth>* const Cloth = Cloths.FindByPredicate(
		[ClothId](TUniquePtr<FClothingSimulationCloth>& InCloth)
		{
			return InCloth->GetGroupId() == ClothId;
		});

	return Cloth ? Cloth->Get(): nullptr;
}

const FClothingSimulationContext* FClothingSimulation::GetClothingSimulationContext(const USkeletalMeshComponent* SkeletalMeshComponent) const
{
	for (const FClothingSimulationInstance& ClothingSimulationInstance : SkeletalMeshComponent->GetClothingSimulationInstances())
	{
		if (ClothingSimulationInstance.GetClothingSimulation() == this)
		{
			return static_cast<const FClothingSimulationContext*>(ClothingSimulationInstance.GetClothingSimulationContext());
		}
	}
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

}  // End namespace Chaos
