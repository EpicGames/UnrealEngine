// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationAccessoryMesh.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "ClothingSimulation.h"
#include "ClothVertBoneData.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/ArrayView.h"
#include "Async/ParallelFor.h"
#if INTEL_ISPC
#include "ChaosClothingSimulationMesh.ispc.generated.h"
#endif

#if INTEL_ISPC
#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_SkinPhysicsMesh_ISPC_Enabled = CHAOS_SKIN_PHYSICS_MESH_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosSkinPhysicsMeshISPCEnabled(TEXT("p.Chaos.SkinPhysicsMesh.ISPC"), bChaos_SkinPhysicsMesh_ISPC_Enabled, TEXT("Whether to use ISPC optimizations on skinned physics meshes"));
#endif

static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FVector3f) == sizeof(FVector3f), "sizeof(ispc::FVector3f) != sizeof(FVector3f)");
static_assert(sizeof(ispc::FMatrix44f) == sizeof(FMatrix44f), "sizeof(ispc::FMatrix44f) != sizeof(FMatrix44f)");
static_assert(sizeof(ispc::FTransform3f) == sizeof(FTransform3f), "sizeof(ispc::FTransform3f) != sizeof(FTransform3f)");
static_assert(sizeof(ispc::FClothVertBoneData) == sizeof(FClothVertBoneData), "sizeof(ispc::FClothVertBoneData) != sizeof(Chaos::FClothVertBoneData)");
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Skin Physics Mesh"), STAT_ChaosClothSkinPhysicsMesh, STATGROUP_ChaosCloth);

namespace Chaos
{

namespace Private
{
	template<typename InOutVectorType>
	void ApplyMorphTargetInternal(
		float ActiveMorphTargetWeight,
		const TConstArrayView<FVector3f>& MorphTargetPositionDeltas,
		const TConstArrayView<FVector3f>& MorphTargetTangentZDeltas,
		const TConstArrayView<int32>& MorphTargetIndices,
		const TArrayView<InOutVectorType>& InOutPositions,
		const TArrayView<InOutVectorType>& InOutNormals)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_ApplyMorphTargetInternal);

		for (int32 Index = 0; Index < MorphTargetIndices.Num(); ++Index)
		{
			const int32 VertexIndex = MorphTargetIndices[Index];
			InOutPositions[VertexIndex] += ActiveMorphTargetWeight * MorphTargetPositionDeltas[Index];
			InOutNormals[VertexIndex] = (InOutNormals[VertexIndex] + ActiveMorphTargetWeight * MorphTargetTangentZDeltas[Index]).GetSafeNormal();
		}
	}

	// Inline function used to force the unrolling of the skinning loop, LWC: note skinning is all done in float to match the asset data type
	FORCEINLINE static void AddInfluence(FVector3f& OutPosition, FVector3f& OutNormal, const FVector3f& RefParticle, const FVector3f& RefNormal, const FMatrix44f& BoneMatrix, const float Weight)
	{
		OutPosition += BoneMatrix.TransformPosition(RefParticle) * Weight;
		OutNormal += BoneMatrix.TransformVector(RefNormal) * Weight;
	}
}

FClothingSimulationAccessoryMesh::FClothingSimulationAccessoryMesh(const FClothingSimulationMesh& InMesh, const FName& InAccessoryMeshName)
	: Mesh(InMesh)
	, AccessoryMeshName(InAccessoryMeshName)
{
}

FClothingSimulationAccessoryMesh::~FClothingSimulationAccessoryMesh() = default;

void FClothingSimulationAccessoryMesh::ApplyMorphTarget(
	int32 ActiveMorphTargetIndex,
	float ActiveMorphTargetWeight,
	const TArrayView<Softs::FSolverVec3>& InOutPositions,
	const TArrayView<Softs::FSolverVec3>& InOutNormals) const
{
	const TConstArrayView<FVector3f> MorphTargetPositionDeltas = GetMorphTargetPositionDeltas(ActiveMorphTargetIndex);
	const TConstArrayView<FVector3f> MorphTargetTangentZDeltas = GetMorphTargetTangentZDeltas(ActiveMorphTargetIndex);
	const TConstArrayView<int32> MorphTargetIndices = GetMorphTargetIndices(ActiveMorphTargetIndex);

	if (!FMath::IsNearlyZero(ActiveMorphTargetWeight) && !MorphTargetPositionDeltas.IsEmpty() &&
		MorphTargetPositionDeltas.Num() == MorphTargetTangentZDeltas.Num() &&
		MorphTargetPositionDeltas.Num() == MorphTargetIndices.Num())
	{
		Private::ApplyMorphTargetInternal(ActiveMorphTargetWeight, MorphTargetPositionDeltas, MorphTargetTangentZDeltas, MorphTargetIndices, InOutPositions, InOutNormals);
	}
}

void FClothingSimulationAccessoryMesh::SkinPhysicsMesh(int32 ActiveMorphTargetIndex, float ActiveMorphTargetWeight, const FReal LocalSpaceScale, const FVec3& LocalSpaceLocation, const TArrayView<Softs::FSolverVec3>& OutPositions, const TArrayView<Softs::FSolverVec3>& OutNormals) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_SkinPhysicsMesh);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothSkinPhysicsMesh);
	SCOPE_CYCLE_COUNTER(STAT_ClothSkinPhysMesh);

	FTransform ComponentToLocalSpaceReal = Mesh.GetComponentToWorldTransform();
	ComponentToLocalSpaceReal.AddToTranslation(-LocalSpaceLocation);
	check(LocalSpaceScale > UE_SMALL_NUMBER);
	const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
	ComponentToLocalSpaceReal.MultiplyScale3D(FVec3(LocalSpaceScaleInv));
	ComponentToLocalSpaceReal.ScaleTranslation(LocalSpaceScaleInv);
	const FTransform3f ComponentToLocalSpace(ComponentToLocalSpaceReal);  // LWC: Now in local space, therefore it is safe to use single precision which is the asset data format

	const int32* const RESTRICT BoneMap = Mesh.GetBoneMap().GetData();
	const FMatrix44f* const RESTRICT RefToLocalMatrices = Mesh.GetRefToLocalMatrices().GetData();

	const uint32 NumPoints = GetNumPoints();
	check(NumPoints == OutPositions.Num());
	check(NumPoints == OutNormals.Num());
	const TConstArrayView<FClothVertBoneData> BoneData = GetBoneData();
	TConstArrayView<FVector3f> Positions = GetPositions();
	TConstArrayView<FVector3f> Normals = GetNormals();
	TArray<FVector3f> WritablePositions;
	TArray<FVector3f> WritableNormals;

	const TConstArrayView<FVector3f> MorphTargetPositionDeltas = GetMorphTargetPositionDeltas(ActiveMorphTargetIndex);
	const TConstArrayView<FVector3f> MorphTargetTangentZDeltas = GetMorphTargetTangentZDeltas(ActiveMorphTargetIndex);
	const TConstArrayView<int32> MorphTargetIndices = GetMorphTargetIndices(ActiveMorphTargetIndex);

	if (!FMath::IsNearlyZero(ActiveMorphTargetWeight) && !MorphTargetPositionDeltas.IsEmpty() &&
		MorphTargetPositionDeltas.Num() == MorphTargetTangentZDeltas.Num() &&
		MorphTargetPositionDeltas.Num() == MorphTargetIndices.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_SkinPhysicsMesh_MorphTargets);
		// TODO optimize this
		WritablePositions = Positions;
		WritableNormals = Normals;
		Private::ApplyMorphTargetInternal(ActiveMorphTargetWeight, MorphTargetPositionDeltas, MorphTargetTangentZDeltas, MorphTargetIndices, TArrayView<FVector3f>(WritablePositions), TArrayView<FVector3f>(WritableNormals));
		Positions = WritablePositions;
		Normals = WritableNormals;
	}

#if INTEL_ISPC
	if (bChaos_SkinPhysicsMesh_ISPC_Enabled)
	{
		ispc::SkinPhysicsMesh(
			(ispc::FVector3f*)OutPositions.GetData(),
			(ispc::FVector3f*)OutNormals.GetData(),
			(ispc::FVector3f*)Positions.GetData(),
			(ispc::FVector3f*)Normals.GetData(),
			(ispc::FClothVertBoneData*)BoneData.GetData(),
			BoneMap,
			(ispc::FMatrix44f*)RefToLocalMatrices,
			(ispc::FTransform3f&)ComponentToLocalSpace,
			NumPoints);
	}
	else
#endif
	{
		static const uint32 MinParallelVertices = 500;  // 500 seems to be the lowest threshold still giving gains even on profiled assets that are only using a small number of influences

		ParallelFor(NumPoints, [&BoneData, &Positions, &Normals, &ComponentToLocalSpace, BoneMap, RefToLocalMatrices, &OutPositions, &OutNormals](uint32 VertIndex)
		{
			const uint16* const RESTRICT BoneIndices = BoneData[VertIndex].BoneIndices;
			const float* const RESTRICT BoneWeights = BoneData[VertIndex].BoneWeights;
	
			// WARNING - HORRIBLE UNROLLED LOOP + JUMP TABLE BELOW
			// done this way because this is a pretty tight and performance critical loop. essentially
			// rather than checking each influence we can just jump into this switch and fall through
			// everything to compose the final skinned data
			const FVector3f& RefParticle = Positions[VertIndex];
			const FVector3f& RefNormal = Normals[VertIndex];

			FVector3f Position(ForceInitToZero);
			FVector3f Normal(ForceInitToZero);
			switch (BoneData[VertIndex].NumInfluences)
			{
			default:  // Intentional fall through
			case 12: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[11]]], BoneWeights[11]);  // Intentional fall through
			case 11: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[10]]], BoneWeights[10]);  // Intentional fall through
			case 10: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 9]]], BoneWeights[ 9]);  // Intentional fall through
			case  9: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 8]]], BoneWeights[ 8]);  // Intentional fall through
			case  8: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 7]]], BoneWeights[ 7]);  // Intentional fall through
			case  7: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 6]]], BoneWeights[ 6]);  // Intentional fall through
			case  6: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 5]]], BoneWeights[ 5]);  // Intentional fall through
			case  5: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 4]]], BoneWeights[ 4]);  // Intentional fall through
			case  4: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 3]]], BoneWeights[ 3]);  // Intentional fall through
			case  3: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 2]]], BoneWeights[ 2]);  // Intentional fall through
			case  2: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 1]]], BoneWeights[ 1]);  // Intentional fall through
			case  1: Private::AddInfluence(Position, Normal, RefParticle, RefNormal, RefToLocalMatrices[BoneMap[BoneIndices[ 0]]], BoneWeights[ 0]);  // Intentional fall through
			case  0: break;
			}

			OutPositions[VertIndex] = FSolverVec3(ComponentToLocalSpace.TransformPosition(Position));
			OutNormals[VertIndex] = FSolverVec3(ComponentToLocalSpace.TransformVector(Normal).GetSafeNormal());

		}, NumPoints > MinParallelVertices ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}
}


void FClothingSimulationAccessoryMesh::Update(
	const FClothingSimulationSolver* Solver,
	const TArrayView<Softs::FSolverVec3>& OutPositions,
	const TArrayView<Softs::FSolverVec3>& OutNormals,
	int32 ActiveMorphTargetIndex,
	float ActiveMorphTargetWeight) const
{
	check(Solver);

	// Skin current LOD positions
	const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
	const FVec3& LocalSpaceLocation = Solver->GetLocalSpaceLocation();

	SkinPhysicsMesh(ActiveMorphTargetIndex, ActiveMorphTargetWeight, LocalSpaceScale, LocalSpaceLocation, OutPositions, OutNormals);
}
}  // End namespace Chaos
