// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingAccessoryMeshData.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosClothingPatternData.h"
#include "ChaosCloth/ChaosWeightMapTarget.h"
#include "ChaosCloth/ChaosClothComplexColliders.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftBodyCollisionConstraint.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/SoftsExternalForces.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "Chaos/VelocityField.h"
#include "Containers/ArrayView.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "HAL/IConsoleManager.h"
#include "ClothingSimulation.h"

namespace Chaos
{

namespace ClothingSimulationClothDefault
{
	constexpr int32 MassMode = (int32)FClothingSimulationCloth::EMassMode::Density;
	constexpr bool bUseGeodesicTethers = true;
	constexpr float MassValue = 0.35f;
	constexpr float MinPerParticleMass = 0.0001f;
	constexpr float CollisionThickness = Softs::FPBDSoftBodyCollisionConstraint::DefaultCollisionThickness;
	constexpr float FrictionCoefficient = Softs::FPBDSoftBodyCollisionConstraint::DefaultFrictionCoefficient;
	constexpr float DampingCoefficient = 0.01f;
	constexpr float LocalDampingCoefficient = 0.0f;
	constexpr float Drag = 0.035f;
	constexpr float Lift = 0.035f;
	constexpr float Pressure = 0.f;
	constexpr float AirDensity = 1.225f;  // Air density in kg/m^3
	constexpr float GravityScale = Softs::FExternalForces::DefaultGravityScale; // 1.f;
	constexpr float GravityZOverride = Softs::FExternalForces::DefaultGravityZOverride; // -980.665f;
	constexpr EChaosSoftsSimulationSpace VelocityScaleSpace = EChaosSoftsSimulationSpace::ReferenceBoneSpace;
	constexpr float VelocityScale = 0.75f;
	constexpr float MaxVelocityScale = 1.f;
	constexpr float MaxVelocity = TNumericLimits<float>::Max();
	constexpr float MaxAcceleration = TNumericLimits<float>::Max();
	constexpr float FictitiousAngularScale = Softs::FExternalForces::DefaultFictitiousAngularScale; // 1.f;
}

namespace ClothingSimulationClothConsoleVariables
{
	TAutoConsoleVariable<bool> CVarLegacyDisablesAccurateWind(
		TEXT("p.ChaosCloth.LegacyDisablesAccurateWind"),
		true,
		TEXT("Whether using the Legacy wind model switches off the accurate wind model, or adds up to it"));

	TAutoConsoleVariable<float> CVarGravityMultiplier(
		TEXT("p.ChaosCloth.GravityMultiplier"),
		1.f,
		TEXT("Scalar multiplier applied at the final stage of the cloth's gravity formulation."));

	static float GlobalMaxLinearVelocity = 0.f;
	FAutoConsoleVariableRef CVarMaxLinearVelocity(
		TEXT("p.ChaosCloth.MaxLinearVelocity"),
		GlobalMaxLinearVelocity,
		TEXT("The maximum amount of linear velocity sent to the local cloth space from the reference bone. "
			 "The console variable value will take precedence when set smaller to the equivalent Velocity Scale config properties."
			 "Use 0 to disable."));

	static float GlobalMaxAngularVelocity = 0.f;
	FAutoConsoleVariableRef CVarMaxAngularVelocity(
		TEXT("p.ChaosCloth.MaxAngularVelocity"),
		GlobalMaxAngularVelocity,
		TEXT("The maximum amount of angular velocity sent to the local cloth space from the reference bone. "
			 "The console variable value will take precedence when set smaller to the equivalent Velocity Scale config properties."
			 "Use 0 to disable."));

	static float GlobalMaxLinearAcceleration = 0.f;
	FAutoConsoleVariableRef CVarMaxLinearAcceleration(
		TEXT("p.ChaosCloth.MaxLinearAcceleration"),
		GlobalMaxLinearAcceleration,
		TEXT("The maximum amount of linear acceleration sent to the local cloth space from the reference bone. "
			 "The console variable value will take precedence when set smaller to the equivalent Velocity Scale config properties."
			 "Use 0 to disable."));

	static float GlobalMaxAngularAcceleration = 0.f;
	FAutoConsoleVariableRef CVarMaxAngularAcceleration(
		TEXT("p.ChaosCloth.MaxAngularAcceleration"),
		GlobalMaxAngularAcceleration,
		TEXT("The maximum amount of angular acceleration sent to the local cloth space from the reference bone. "
			 "The console variable value will take precedence when set smaller to the equivalent Velocity Scale config properties."
			 "Use 0 to disable."));
}

using namespace Softs;
struct FClothingSimulationCloth::FLODData
{
	// Input mesh
	const int32 NumParticles;
	const TMap<FString, TConstArrayView<FRealSingle>> WeightMaps;
	const TMap<FString, const TSet<int32>*> VertexSets;
	const TMap<FString, const TSet<int32>*> FaceSets;
	const TMap<FString, TConstArrayView<int32>> FaceIntMaps;
	const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>> Tethers;
	const TSharedPtr<const FManagedArrayCollection> ManagedArrayCollection;

	const FClothingPatternData PatternData;
	const FTriangleMesh NoOffsetTriangleMesh;

	// Per Solver data
	struct FSolverData
	{
		int32 LODIndex;
		int32 ParticleRangeId;
		FTriangleMesh OffsetTriangleMesh; // Only used if using PBD solver

		TMap<FName, FClothingAccessoryMeshData> AccessoryMeshData;
	};
	TMap<FClothingSimulationSolver*, FSolverData> SolverData;

	// Cached property facade data
	int32 ActiveMorphTarget = INDEX_NONE;
	float ActiveMorphTargetWeight = 0.f;

	// Stats
	int32 NumKinematicParticles;
	int32 NumDynamicParticles;

	FLODData(
		const FClothingSimulationMesh& Mesh,
		const int32 LODIndex,
		bool bUseGeodesicTethers,
		TMap<FString, TConstArrayView<FRealSingle>>&& InWeightMaps,
		TMap<FString, const TSet<int32>*>&& InVertexSets,
		TMap<FString, const TSet<int32>*>&& InFaceSets,
		TMap<FString, TConstArrayView<int32>>&& InFaceIntMaps,
		const Softs::FCollectionPropertyFacade& ConfigProperties);

	static FTriangleMesh BuildTriangleMesh(const TConstArrayView<uint32>& Indices, const int32 NumParticles);

	void UpdateCachedProperties(const FClothingSimulationMesh& Mesh,
		const int32 LODIndex, const Softs::FCollectionPropertyFacade& PropertyCollection, bool bForce = false);

	void AddParticles(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, int32 LODIndex);
	void Add(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, int32 LODIndex);
	void Remove(FClothingSimulationSolver* Solver);

	void Update(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);

	void Enable(FClothingSimulationSolver* Solver, bool bEnable) const;

	void UpdateAccessoryMeshes(FClothingSimulationSolver* Solver);
	void ResetAccessoryMeshes(FClothingSimulationSolver* Solver);
	void ResetStartPose(FClothingSimulationSolver* Solver);
	void ApplyPreSimulationTransformsToAccessoryMeshes(FClothingSimulationSolver* Solver, const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime);
	void PreSubstepAccessoryMeshes(FClothingSimulationSolver* Solver, const Softs::FSolverReal InterpolationAlpha);

	void UpdateNormals(FClothingSimulationSolver* Solver) const;

	// This can update every frame, so should be cached
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(ActiveMorphTarget, float);

};

FClothingSimulationCloth::FLODData::FLODData(
	const FClothingSimulationMesh& InMesh,
	const int32 LODIndex,
	bool bUseGeodesicTethers,
	TMap<FString, TConstArrayView<FRealSingle>>&& InWeightMaps,
	TMap<FString, const TSet<int32>*>&& InVertexSets,
	TMap<FString, const TSet<int32>*>&& InFaceSets,
	TMap<FString, TConstArrayView<int32>>&& InFaceIntMaps,
	const Softs::FCollectionPropertyFacade& PropertyCollection)
	: NumParticles(InMesh.GetNumPoints(LODIndex))
	, WeightMaps(MoveTemp(InWeightMaps))
	, VertexSets(MoveTemp(InVertexSets))
	, FaceSets(MoveTemp(InFaceSets))
	, FaceIntMaps(MoveTemp(InFaceIntMaps))
	, Tethers(InMesh.GetTethers(LODIndex, bUseGeodesicTethers))
	, ManagedArrayCollection(InMesh.GetManagedArrayCollection(LODIndex))
	, PatternData(NumParticles, InMesh.GetIndices(LODIndex), InMesh.GetPatternPositions(LODIndex), InMesh.GetPatternIndices(LODIndex), InMesh.GetPatternToWeldedIndices(LODIndex))
	, NoOffsetTriangleMesh(BuildTriangleMesh(InMesh.GetIndices(LODIndex), NumParticles))
	, ActiveMorphTargetIndex(PropertyCollection)
{
	UpdateCachedProperties(InMesh, LODIndex, PropertyCollection, true);
}

FTriangleMesh FClothingSimulationCloth::FLODData::BuildTriangleMesh(const TConstArrayView<uint32>& Indices, const int32 InNumParticles)
{
	FTriangleMesh OutTriangleMesh;
	// Build a sim friendly triangle mesh including the solver particle's Offset
	const int32 NumElements = Indices.Num() / 3;
	TArray<TVec3<int32>> Elements;
	Elements.Reserve(NumElements);

	for (int32 i = 0; i < NumElements; ++i)
	{
		const int32 Index = 3 * i;
		Elements.Add(
			{ static_cast<int32>(Indices[Index]),
			 static_cast<int32>(Indices[Index + 1]),
			 static_cast<int32>(Indices[Index + 2]) });
	}

	OutTriangleMesh.Init(MoveTemp(Elements), 0, InNumParticles - 1);
	OutTriangleMesh.GetPointToTriangleMap(); // Builds map for later use by GetPointNormals(), and the velocity fields
	return OutTriangleMesh;
}

void FClothingSimulationCloth::FLODData::UpdateCachedProperties(const FClothingSimulationMesh& InMesh,
	const int32 LODIndex, const Softs::FCollectionPropertyFacade& InPropertyCollection, bool bForce)
{
	if (ActiveMorphTargetIndex != INDEX_NONE && (bForce || IsActiveMorphTargetMutable(InPropertyCollection)))
	{
		ActiveMorphTargetWeight = FMath::Clamp(GetActiveMorphTarget(InPropertyCollection), -1.f, 1.f);

		if (bForce || IsActiveMorphTargetStringDirty(InPropertyCollection))
		{
			ActiveMorphTarget = InMesh.FindMorphTargetByName(LODIndex, GetActiveMorphTargetString(InPropertyCollection));
		}
	}
}

void FClothingSimulationCloth::FLODData::AddParticles(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, int32 InLODIndex)
{
	check(Solver);
	check(Cloth);
	check(Cloth->Mesh);

	// Add a new solver data chunk
	check(!SolverData.Find(Solver));
	FSolverData& SolverDatum = SolverData.Add(Solver);
	SolverDatum.LODIndex = InLODIndex;
	// Add particles
	SolverDatum.ParticleRangeId = Solver->AddParticles(NumParticles, Cloth->GroupId);
	if (!NumParticles)
	{
		return;
	}

	if (Solver->IsLegacySolver())
	{
		const TArray<TVec3<int32>>& Elements = NoOffsetTriangleMesh.GetElements();
		TArray<TVec3<int32>> OffsetElements;
		OffsetElements.Reserve(Elements.Num());
		for (const TVec3<int32>& Element : Elements)
		{
			OffsetElements.Add({
				Element[0] + SolverDatum.ParticleRangeId,
				Element[1] + SolverDatum.ParticleRangeId,
				Element[2] + SolverDatum.ParticleRangeId });
		}

		SolverDatum.OffsetTriangleMesh.Init(MoveTemp(OffsetElements), SolverDatum.ParticleRangeId, SolverDatum.ParticleRangeId + NumParticles - 1);
		SolverDatum.OffsetTriangleMesh.GetPointToTriangleMap(); // Builds map for later use by GetPointNormals(), and the velocity fields
	}

	TConstArrayView<FName> AccessoryMeshes = Cloth->Mesh->GetAllAccessoryMeshNames(InLODIndex);
	SolverDatum.AccessoryMeshData.Reserve(AccessoryMeshes.Num());
	for (const FName& AccessoryMeshName : AccessoryMeshes)
	{
		if (const FClothingSimulationAccessoryMesh* const AccessoryMesh = Cloth->Mesh->GetAccessoryMesh(InLODIndex, AccessoryMeshName))
		{
			FClothingAccessoryMeshData& MeshData = SolverDatum.AccessoryMeshData.Add(AccessoryMeshName, FClothingAccessoryMeshData(NumParticles, *AccessoryMesh));
			MeshData.AllocateParticles();
		}
	}

	const FTriangleMesh& TriangleMesh = Solver->IsLegacySolver() ? SolverDatum.OffsetTriangleMesh : NoOffsetTriangleMesh;

	// Update source mesh for this LOD, this is required prior to reset the start pose
	Cloth->Mesh->Update(Solver, INDEX_NONE, InLODIndex, 0, SolverDatum.ParticleRangeId);
	UpdateAccessoryMeshes(Solver);

	// Reset the particles start pose before setting up mass and constraints
	ResetStartPose(Solver);

	// Initialize the normals, in case the sim data is queried before the simulation steps
	UpdateNormals(Solver);

	// Retrieve config properties
	check(Cloth->Config);
	const Softs::FCollectionPropertyFacade& ConfigProperties = Cloth->Config->GetProperties(InLODIndex);

	// Retrieve MaxDistance information (weight map and Low/High values)
	const Softs::FPBDFlatWeightMapView MaxDistances(
		Softs::FPBDSphericalConstraint::GetWeightedFloatMaxDistance(ConfigProperties, FVector2f(0.f, 1.f)),
		WeightMaps.FindRef(Softs::FPBDSphericalConstraint::GetMaxDistanceString(ConfigProperties, Softs::FPBDSphericalConstraint::MaxDistanceName.ToString())),
		NumParticles);

	const TSet<int32>* const KinematicVertices3DSet = VertexSets.FindRef(FClothingSimulationCloth::GetKinematicVertices3DString(ConfigProperties, KinematicVertices3DName.ToString()), nullptr);

	// Set the particle masses
	static const FRealSingle KinematicDistanceThreshold = 0.1f;  // TODO: This is not the same value as set in the painting UI but we might want to expose this value as parameter
	auto KinematicPredicate =
		[&MaxDistances, KinematicVertices3DSet](int32 Index)
	{
		return MaxDistances.GetValue(Index) < KinematicDistanceThreshold || (KinematicVertices3DSet && KinematicVertices3DSet->Contains(Index));
	};

	const int32 MassMode = FClothingSimulationCloth::GetMassMode(ConfigProperties, ClothingSimulationClothDefault::MassMode);

	constexpr FRealSingle MinPerParticleMassClampMin = UE_SMALL_NUMBER;
	const FRealSingle MinPerParticleMass = FMath::Max((FRealSingle)FClothingSimulationCloth::GetMinPerParticleMass(ConfigProperties, ClothingSimulationClothDefault::MinPerParticleMass), MinPerParticleMassClampMin);

	switch (MassMode)
	{
	default:
		check(false);
	case EMassMode::UniformMass:
	{
		const FVector2f MassValue = FClothingSimulationCloth::GetWeightedFloatMassValue(ConfigProperties, ClothingSimulationClothDefault::MassValue);
		const TConstArrayView<float> MassValueMultipliers = WeightMaps.FindRef(FClothingSimulationCloth::GetMassValueString(ConfigProperties, MassValueName.ToString()));
		Solver->SetParticleMassUniform(SolverDatum.ParticleRangeId, MassValue, MassValueMultipliers, MinPerParticleMass, TriangleMesh, KinematicPredicate);
	}
	break;
	case EMassMode::TotalMass:
	{
		const FRealSingle MassValue = FClothingSimulationCloth::GetMassValue(ConfigProperties, ClothingSimulationClothDefault::MassValue);
		Solver->SetParticleMassFromTotalMass(SolverDatum.ParticleRangeId, MassValue, MinPerParticleMass, TriangleMesh, KinematicPredicate);
	}
	break;
	case EMassMode::Density:
	{
		const FVector2f MassValue = FClothingSimulationCloth::GetWeightedFloatMassValue(ConfigProperties, ClothingSimulationClothDefault::MassValue);
		const TConstArrayView<float> MassValueMultipliers = WeightMaps.FindRef(FClothingSimulationCloth::GetMassValueString(ConfigProperties, MassValueName.ToString()));
		Solver->SetParticleMassFromDensity(SolverDatum.ParticleRangeId, MassValue, MassValueMultipliers, MinPerParticleMass, TriangleMesh, KinematicPredicate);
	}
	break;
	}
}

void FClothingSimulationCloth::FLODData::Add(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, int32 InLODIndex)
{
	check(Solver);
	check(Cloth);
	check(Cloth->Mesh);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	check(SolverDatum.LODIndex == InLODIndex);
	if (!NumParticles)
	{
		return;
	}

	check(SolverDatum.ParticleRangeId != INDEX_NONE);
	const int32 ParticleRangeId = SolverDatum.ParticleRangeId;

	// Retrieve the component's scale
	const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
	check(LocalSpaceScale > UE_SMALL_NUMBER);
	const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
	const Softs::FSolverReal MeshScale = Cloth->Mesh->GetScale() * LocalSpaceScaleInv;

	const FTriangleMesh& TriangleMesh = Solver->IsLegacySolver() ? SolverDatum.OffsetTriangleMesh : NoOffsetTriangleMesh;

	// Retrieve config properties
	check(Cloth->Config);
	const Softs::FCollectionPropertyFacade& ConfigProperties = Cloth->Config->GetProperties(InLODIndex);

	// Setup solver constraints
	FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

	TArray<const FClothComplexColliders*> ComplexColliders;
	for (const FClothingSimulationCollider* const Collider : Cloth->GetColliders())
	{
		check(Collider);
		for (FClothingSimulationCollider::ECollisionDataType CollisionType : TEnumRange<FClothingSimulationCollider::ECollisionDataType>())
		{
			const int32 CollisionRangeId = Collider->GetCollisionRangeId(Solver, Cloth, CollisionType);
			if (const FClothComplexColliders* const ComplexCollider = Solver->GetComplexColliders(CollisionRangeId))
			{
				ComplexColliders.Add(ComplexCollider);
			}
		}
	}

	// Create constraints
	const bool bEnabled = false;  // Set constraint disabled by default
	ClothConstraints.AddRules(ConfigProperties, TriangleMesh, &PatternData, WeightMaps, VertexSets, FaceSets, FaceIntMaps, Tethers, MeshScale, bEnabled, ComplexColliders, ManagedArrayCollection, &SolverDatum.AccessoryMeshData);

	// Update LOD stats
	const TConstArrayView<Softs::FSolverReal> InvMasses(Solver->GetParticleInvMasses(ParticleRangeId), NumParticles);
	NumKinematicParticles = 0;
	NumDynamicParticles = 0;
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		if (InvMasses[Index] == (Softs::FSolverReal)0.)
		{
			++NumKinematicParticles;
		}
		else
		{
			++NumDynamicParticles;
		}
	}
}

void FClothingSimulationCloth::FLODData::Remove(FClothingSimulationSolver* Solver)
{
	SolverData.Remove(Solver);
}

void FClothingSimulationCloth::FLODData::Update(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	check(Solver);
	check(Cloth); 
	const FSolverData& SolverDatum = SolverData.FindChecked(Solver);

	const int32 ParticleRangeId = SolverDatum.ParticleRangeId;

	if (ParticleRangeId != INDEX_NONE)
	{
		// Update the animatable constraint parameters
		FClothConstraints& ClothConstraints = Solver->GetClothConstraints(ParticleRangeId);

		check(Cloth->Config);
		const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
		check(LocalSpaceScale > UE_SMALL_NUMBER);
		const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
		const Softs::FSolverReal MeshScale = Cloth->Mesh->GetScale() * LocalSpaceScaleInv;
		const Softs::FSolverReal MaxDistancesScale = (Softs::FSolverReal)Cloth->MaxDistancesMultiplier;
		ClothConstraints.Update(
			Cloth->Config->GetProperties(SolverDatum.LODIndex),
			WeightMaps,
			VertexSets,
			FaceSets,
			FaceIntMaps,
			MeshScale,
			MaxDistancesScale,
			Solver->GetLocalSpaceRotation(),
			Cloth->ReferenceSpaceTransform.GetRotation(),
			LocalSpaceScale,
			Solver->GetLocalSpaceLocation());

		if (const TSharedPtr<Softs::FPBDSkinnedTriangleMeshCollisionConstraints>& SkinnedTriangleCollisionConstraint = ClothConstraints.GetSkinnedTriangleCollisionsConstraint())
		{
			const Softs::FSolverReal MinLODSize = SkinnedTriangleCollisionConstraint->GetCollisionThickness() + SkinnedTriangleCollisionConstraint->GetMaxClothCollisionThickness();

			for (FClothingSimulationCollider* const Collider : Cloth->GetColliders())
			{
				check(Collider);
				for (FClothingSimulationCollider::ECollisionDataType CollisionType : TEnumRange<FClothingSimulationCollider::ECollisionDataType>())
				{
					const int32 CollisionRangeId = Collider->GetCollisionRangeId(Solver, Cloth, CollisionType);
					if (FClothComplexColliders* const ComplexCollider = Solver->GetComplexColliders(CollisionRangeId))
					{
						ComplexCollider->SetMinLODSize(MinLODSize);
					}
				}
			}
		}
	}
}

void FClothingSimulationCloth::FLODData::Enable(FClothingSimulationSolver* Solver, bool bEnable) const
{
	check(Solver);
	const int32 ParticleRangeId = SolverData.FindChecked(Solver).ParticleRangeId;

	if (ParticleRangeId != INDEX_NONE)
	{
		// Enable particles (and related constraints)
		Solver->EnableParticles(ParticleRangeId, bEnable);
	}
}

void FClothingSimulationCloth::FLODData::ResetStartPose(FClothingSimulationSolver* Solver)
{
	check(Solver);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	const int32 ParticleRangeId = SolverDatum.ParticleRangeId;
	if (ParticleRangeId != INDEX_NONE)
	{
		Solver->ResetStartPose(ParticleRangeId, NumParticles);
		ResetAccessoryMeshes(Solver);
	}
}

void FClothingSimulationCloth::FLODData::UpdateAccessoryMeshes(FClothingSimulationSolver* Solver)
{
	check(Solver);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	for (TPair<FName, FClothingAccessoryMeshData>& AccessoryMesh : SolverDatum.AccessoryMeshData)
	{
		AccessoryMesh.Get<1>().Update(Solver);
	}
}

void FClothingSimulationCloth::FLODData::ResetAccessoryMeshes(FClothingSimulationSolver* Solver)
{
	check(Solver);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	for (TPair<FName, FClothingAccessoryMeshData>& AccessoryMesh : SolverDatum.AccessoryMeshData)
	{
		AccessoryMesh.Get<1>().ResetStartPose();
	}
}

void FClothingSimulationCloth::FLODData::ApplyPreSimulationTransformsToAccessoryMeshes(FClothingSimulationSolver* Solver, const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime)
{
	check(Solver);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	for (TPair<FName, FClothingAccessoryMeshData>& AccessoryMesh : SolverDatum.AccessoryMeshData)
	{
		AccessoryMesh.Get<1>().ApplyPreSimulationTransforms(DeltaLocalSpaceLocation, GroupSpaceTransform, DeltaTime);
	}
}

void FClothingSimulationCloth::FLODData::PreSubstepAccessoryMeshes(FClothingSimulationSolver* Solver, const Softs::FSolverReal InterpolationAlpha)
{
	check(Solver);

	FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	for (TPair<FName, FClothingAccessoryMeshData>& AccessoryMesh : SolverDatum.AccessoryMeshData)
	{
		AccessoryMesh.Get<1>().PreSubstep(InterpolationAlpha);
	}
}

void FClothingSimulationCloth::FLODData::UpdateNormals(FClothingSimulationSolver* Solver) const
{
	check(Solver);

	const FSolverData& SolverDatum = SolverData.FindChecked(Solver);
	const int32 ParticleRangeId = SolverDatum.ParticleRangeId;

	if (ParticleRangeId != INDEX_NONE)
	{
		TConstArrayView<Softs::FSolverVec3> Points = Solver->GetParticleXsView(ParticleRangeId);
		TArrayView<Softs::FSolverVec3> Normals = Solver->GetNormalsView(ParticleRangeId);
		TArray<Softs::FSolverVec3> FaceNormals;
		NoOffsetTriangleMesh.GetFaceNormals(FaceNormals, Points, /*ReturnEmptyOnError =*/ false);
		NoOffsetTriangleMesh.GetPointNormals(Normals, TConstArrayView<Softs::FSolverVec3>(FaceNormals), /*bUseGlobalArray =*/ false);
	}
}

FClothingSimulationCloth::FClothingSimulationCloth(
	FClothingSimulationConfig* InConfig,
	FClothingSimulationMesh* InMesh,
	TArray<FClothingSimulationCollider*>&& InColliders,
	uint32 InGroupId)
	: GroupId(InGroupId)
{
	SetConfig(InConfig);
	SetMesh(InMesh);
	SetColliders(MoveTemp(InColliders));
}

FClothingSimulationCloth::~FClothingSimulationCloth()
{
	// If the PropertyCollection is owned by this object, so does the current config object
	if (PropertyCollection.IsValid())
	{
		delete Config;
	}
}

void FClothingSimulationCloth::SetMesh(FClothingSimulationMesh* InMesh)
{
	Mesh = InMesh;
	
	// Reset LODs
	const int32 NumLODs = Mesh ? Mesh->GetNumLODs() : 0;
	LODData.Reset(NumLODs);
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		// Regenerate LOD weight maps lookup map
		const TArray<FName> WeightMapNames = Mesh->GetWeightMapNames(LODIndex);
		TMap<FString, TConstArrayView<FRealSingle>> WeightMaps;
		WeightMaps.Reserve(WeightMapNames.Num());

		const TArray<TConstArrayView<FRealSingle>> WeightMapArray = Mesh->GetWeightMaps(LODIndex);
		ensure(WeightMapArray.Num() == WeightMapNames.Num());

		for (int32 WeightMapIndex = 0; WeightMapIndex < WeightMapNames.Num(); ++WeightMapIndex)
		{
			WeightMaps.Add(WeightMapNames[WeightMapIndex].ToString(),
				WeightMapArray.IsValidIndex(WeightMapIndex) ?
					WeightMapArray[WeightMapIndex] :
					TConstArrayView<FRealSingle>());
		}

		TMap<FString, const TSet<int32>*> VertexSets = Mesh->GetVertexSets(LODIndex);
		TMap<FString, const TSet<int32>*> FaceSets = Mesh->GetFaceSets(LODIndex);
		TMap<FString, TConstArrayView<int32>> FaceIntMaps = Mesh->GetFaceIntMaps(LODIndex);

		const bool bUseGeodesicTethers = GetUseGeodesicTethers(Config->GetProperties(LODIndex), ClothingSimulationClothDefault::bUseGeodesicTethers);

		// Add LOD data
		LODData.Add(MakeUnique<FLODData>(
			*Mesh,
			LODIndex,
			bUseGeodesicTethers,
			MoveTemp(WeightMaps),
			MoveTemp(VertexSets),
			MoveTemp(FaceSets),
			MoveTemp(FaceIntMaps),
			Config->GetProperties(LODIndex)
		));
	}

	// Iterate all known solvers
	TArray<FClothingSimulationSolver*> Solvers;
	LODIndices.GetKeys(Solvers);
	for (FClothingSimulationSolver* const Solver : Solvers)
	{
		// Refresh this cloth to recreate particles
		Solver->RefreshCloth(this);
	}
}

void FClothingSimulationCloth::SetConfig(FClothingSimulationConfig* InConfig)
{
	// If the PropertyCollection is owned by this object, so does the current config object
	if (PropertyCollection.IsValid())
	{
		delete Config;
		PropertyCollection.Reset();
	}

	if (InConfig)
	{
		Config = InConfig;
	}
	else
	{
		// Create a default empty config object for coherence
		PropertyCollection = MakeShared<FManagedArrayCollection>();
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // TODO Remove these pragmas when the method is removed and the TArray can be correctly deducted again
		Config = new FClothingSimulationConfig({ PropertyCollection });
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FClothingSimulationCloth::SetColliders(TArray<FClothingSimulationCollider*>&& InColliders)
{
	// Empty the collider list, but keep the pointers around for the removal operation below
	const TArray<FClothingSimulationCollider*> TempColliders = MoveTemp(Colliders);

	// Replace with the new colliders
	Colliders = InColliders;

	// Iterate all known solvers
	TArray<FClothingSimulationSolver*> Solvers;
	LODIndices.GetKeys(Solvers);
	for (FClothingSimulationSolver* const Solver : Solvers)
	{
		// Remove any held collider data related to this cloth simulation
		for (FClothingSimulationCollider* const Collider : TempColliders)
		{
			Collider->Remove(Solver, this);
		}

		// Refresh this cloth to recreate collision particles
		Solver->RefreshCloth(this);
	}
}

void FClothingSimulationCloth::AddCollider(FClothingSimulationCollider* InCollider)
{
	check(InCollider);

	if (Colliders.Find(InCollider) != INDEX_NONE)
	{
		return;
	}

	// Add the collider to the solver update array
	Colliders.Emplace(InCollider);

	// Iterate all known solvers
	TArray<FClothingSimulationSolver*> Solvers;
	LODIndices.GetKeys(Solvers);
	for (FClothingSimulationSolver* const Solver : Solvers)
	{
		// Refresh this cloth to recreate collision particles
		Solver->RefreshCloth(this);
	}
}

void FClothingSimulationCloth::RemoveCollider(FClothingSimulationCollider* InCollider)
{
	if (Colliders.Find(InCollider) == INDEX_NONE)
	{
		return;
	}

	// Remove collider from array
	Colliders.RemoveSwap(InCollider);

	// Iterate all known solvers
	TArray<FClothingSimulationSolver*> Solvers;
	LODIndices.GetKeys(Solvers);
	for (FClothingSimulationSolver* const Solver : Solvers)
	{
		// Remove any held collider data related to this cloth simulation
		InCollider->Remove(Solver, this);

		// Refresh this cloth to recreate collision particles
		Solver->RefreshCloth(this);
	}
}

void FClothingSimulationCloth::RemoveColliders()
{
	// Empty the collider list, but keep the pointers around for the removal operation below
	const TArray<FClothingSimulationCollider*> TempColliders = MoveTemp(Colliders);

	// Iterate all known solvers
	TArray<FClothingSimulationSolver*> Solvers;
	LODIndices.GetKeys(Solvers);
	for (FClothingSimulationSolver* const Solver : Solvers)
	{
		// Remove any held collider data related to this cloth simulation
		for (FClothingSimulationCollider* const Collider : TempColliders)
		{
			Collider->Remove(Solver, this);
		}

		// Refresh this cloth to recreate collision particles
		Solver->RefreshCloth(this);
	}
}

void FClothingSimulationCloth::Add(FClothingSimulationSolver* Solver)
{
	check(Solver);

	// Add Colliders. Do this first because the colliders need to be there to create constraints inside LODData::Add
	for (FClothingSimulationCollider* Collider : Colliders)
	{
		Collider->Add(Solver, this);
	}

	// Can't add a cloth twice to the same solver
	check(!LODIndices.Find(Solver));

	// Initialize LODIndex
	int32& LODIndex = LODIndices.Add(Solver);
	LODIndex = INDEX_NONE;

	// Add all particles first and in reverse order. (was doing this for multi-res constraints, but now just leaving it)
	for (int32 Index = LODData.Num() - 1; Index >= 0; --Index)
	{
		LODData[Index]->AddParticles(Solver, this, Index);
	}
	// Now add the LODs themselves.
	for (int32 Index = 0; Index < LODData.Num(); ++Index)
	{
		LODData[Index]->Add(Solver, this, Index);
	}
}

void FClothingSimulationCloth::Remove(FClothingSimulationSolver* Solver)
{
	// Remove Colliders
	for (FClothingSimulationCollider* Collider : Colliders)
	{
		Collider->Remove(Solver, this);
	}

	// Remove solver from maps
	LODIndices.Remove(Solver);
	for (TUniquePtr<FLODData>& LODDatum: LODData)
	{
		LODDatum->Remove(Solver);
	}
}

int32 FClothingSimulationCloth::GetNumParticles(int32 InLODIndex) const
{
	return LODData.IsValidIndex(InLODIndex) ? LODData[InLODIndex]->NumParticles : 0;
}

int32 FClothingSimulationCloth::GetParticleRangeId(const FClothingSimulationSolver* Solver, int32 InLODIndex) const
{
	return LODData.IsValidIndex(InLODIndex) ? LODData[InLODIndex]->SolverData.FindChecked(Solver).ParticleRangeId : INDEX_NONE;
}

TVec3<FRealSingle> FClothingSimulationCloth::GetGravity(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	if (!Solver->IsLegacySolver())
	{
		const int32 ParticleRangeId = GetParticleRangeId(Solver);
		if (const Softs::FExternalForces* const ExternalForces = Solver->GetClothConstraints(ParticleRangeId).GetExternalForces().Get())
		{
			return ExternalForces->GetGravity();
		}
		else
		{
			return TVec3<FRealSingle>(0.f, 0.f, Softs::FExternalForces::DefaultGravityZOverride);
		}
	}
	else
	{
		check(Config);
		const Softs::FCollectionPropertyFacade& ConfigProperties = Config->GetProperties(GetLODIndex(Solver));

		const bool bUseGravityOverride = GetUseGravityOverride(ConfigProperties, false);
		const TVec3<FRealSingle> GravityOverride = (TVec3<FRealSingle>)GetGravityOverride(ConfigProperties, FVector3f(0.f, 0.f, ClothingSimulationClothDefault::GravityZOverride));
		const FRealSingle GravityScale = (FRealSingle)GetGravityScale(ConfigProperties, 1.f);
		const FRealSingle GravityMultiplier = (FRealSingle)ClothingSimulationClothConsoleVariables::CVarGravityMultiplier.GetValueOnAnyThread();

		return (Solver->IsClothGravityOverrideEnabled() && bUseGravityOverride ? GravityOverride : Solver->GetGravity() * GravityScale) * GravityMultiplier;
	}
}

FAABB3 FClothingSimulationCloth::CalculateBoundingBox(const FClothingSimulationSolver* Solver) const
{
	check(Solver);

	// Calculate local space bounding box
	Softs::FSolverAABB3 BoundingBox = Softs::FSolverAABB3::EmptyAABB();

	const TConstArrayView<Softs::FSolverVec3> ParticlePositions = GetParticlePositions(Solver);
	for (const Softs::FSolverVec3& ParticlePosition : ParticlePositions)
	{
		BoundingBox.GrowToInclude(ParticlePosition);
	}

	// Return world space bounding box
	return FAABB3(BoundingBox).TransformedAABB(FTransform(FRotation3::Identity, Solver->GetLocalSpaceLocation(), FVector(Solver->GetLocalSpaceScale())));
}

int32 FClothingSimulationCloth::GetParticleRangeId(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? GetParticleRangeId(Solver, LODIndex) : INDEX_NONE;
}

int32 FClothingSimulationCloth::GetNumParticles(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? GetNumParticles(LODIndex) : 0;
}

const FTriangleMesh& FClothingSimulationCloth::GetTriangleMesh(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	static const FTriangleMesh EmptyTriangleMesh;

	return LODData.IsValidIndex(LODIndex) ? (Solver->IsLegacySolver() ? LODData[LODIndex]->SolverData.FindChecked(Solver).OffsetTriangleMesh: LODData[LODIndex]->NoOffsetTriangleMesh) : EmptyTriangleMesh;
}

TConstArrayView<FRealSingle> FClothingSimulationCloth::GetWeightMapByName(const FClothingSimulationSolver* Solver, const FString& Name) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->WeightMaps.FindRef(Name) : TConstArrayView<FRealSingle>();
}

FString FClothingSimulationCloth::GetPropertyString(const FClothingSimulationSolver* Solver, const FName& Property) const
{
	check(Config);
	return Config->GetProperties(GetLODIndex(Solver)).GetStringValue(Property);
}

TConstArrayView<FRealSingle> FClothingSimulationCloth::GetWeightMapByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const
{
	check(Config);
	const FString PropertyString = Config->GetProperties(GetLODIndex(Solver)).GetStringValue(Property);
	return GetWeightMapByName(Solver, PropertyString);
}

TSet<FString> FClothingSimulationCloth::GetAllWeightMapNames() const
{
	TSet<FString> Names;
	for (const TUniquePtr<FLODData>& Data : LODData)
	{
		TSet<FString> LODNames;
		Data->WeightMaps.GetKeys(LODNames);
		Names.Append(MoveTemp(LODNames));
	}
	return Names;
}

TConstArrayView<int32> FClothingSimulationCloth::GetFaceIntMapByName(const FClothingSimulationSolver* Solver, const FString& Name) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->FaceIntMaps.FindRef(Name) : TConstArrayView<int32>();
}

TConstArrayView<int32> FClothingSimulationCloth::GetFaceIntMapByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const
{
	check(Config);
	const FString PropertyString = Config->GetProperties(GetLODIndex(Solver)).GetStringValue(Property);
	return GetFaceIntMapByName(Solver, PropertyString);
}

TSet<FString> FClothingSimulationCloth::GetAllVertexSetNames() const
{
	TSet<FString> Names;
	for (const TUniquePtr<FLODData>& Data : LODData)
	{
		TSet<FString> LODNames;
		Data->VertexSets.GetKeys(LODNames);
		Names.Append(MoveTemp(LODNames));
	}
	return Names;
}

const TSet<int32>* FClothingSimulationCloth::GetVertexSetByName(const FClothingSimulationSolver* Solver, const FString& Name) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->VertexSets.FindRef(Name) : nullptr;
}

const TSet<int32>* FClothingSimulationCloth::GetVertexSetByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const
{
	check(Config);
	const FString PropertyString = Config->GetProperties(GetLODIndex(Solver)).GetStringValue(Property);
	return GetVertexSetByName(Solver, PropertyString);
}

const TArray<TConstArrayView<TTuple<int32, int32, float>>>& FClothingSimulationCloth::GetTethers(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	static const TArray<TConstArrayView<TTuple<int32, int32, float>>> EmptyTethers;
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->Tethers : EmptyTethers;
}

int32 FClothingSimulationCloth::GetReferenceBoneIndex() const
{
	return Mesh ? Mesh->GetReferenceBoneIndex() : INDEX_NONE;
}

int32 FClothingSimulationCloth::GetCurrentMorphTargetIndex(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->ActiveMorphTarget : INDEX_NONE;
}

FRealSingle FClothingSimulationCloth::GetCurrentMorphTargetWeight(const FClothingSimulationSolver* Solver) const
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	return LODData.IsValidIndex(LODIndex) ? LODData[LODIndex]->ActiveMorphTargetWeight : 0.f;
}

TSet<FString> FClothingSimulationCloth::GetAllMorphTargetNames() const
{
	TSet<FString> Names;
	for (int32 LODIndex = 0; LODIndex < LODData.Num(); ++LODIndex)
	{
		Names.Append(Mesh->GetAllMorphTargetNames(LODIndex));
	}
	return Names;
}

TSet<FName> FClothingSimulationCloth::GetAllAccessoryMeshNames() const
{
	TSet<FName> Names;
	for (int32 LODIndex = 0; LODIndex < LODData.Num(); ++LODIndex)
	{
		Names.Append(Mesh->GetAllAccessoryMeshNames(LODIndex));
	}
	return Names;
}

void FClothingSimulationCloth::PreUpdate(FClothingSimulationSolver* Solver)
{
	check(Solver);

	// Exit if the input mesh is missing
	if (!Mesh)
	{
		return;
	}

	// Update Cloth Colliders
	{
		SCOPE_CYCLE_COUNTER(STAT_ClothUpdateCollisions);

		for (FClothingSimulationCollider* Collider : Colliders)
		{
			Collider->PreUpdate(Solver, this);
		}
	}
}

void FClothingSimulationCloth::Update(FClothingSimulationSolver* Solver)
{
	check(Solver);

	// Exit if the input mesh is missing
	if (!Mesh)
	{
		return;
	}

	// Retrieve LOD Index, either from the override, or from the mesh input
	int32& LODIndex = LODIndices.FindChecked(Solver);  // Must be added to solver first

	const int32 PrevLODIndex = LODIndex;
	LODIndex = bUseLODIndexOverride && LODData.IsValidIndex(LODIndexOverride) ? LODIndexOverride : Mesh->GetLODIndex();

	// Update reference space transform from the mesh's reference bone transform  TODO: Add override in the style of LODIndexOverride
	FRigidTransform3 OldReferenceSpaceTransform = ReferenceSpaceTransform; 
	ReferenceSpaceTransform = Mesh->GetReferenceBoneTransform();
	ReferenceSpaceTransform.SetScale3D(FVec3(1.f));

	// Update Cloth Colliders
	{
		SCOPE_CYCLE_COUNTER(STAT_ClothUpdateCollisions);

		for (FClothingSimulationCollider* Collider : Colliders)
		{
			Collider->Update(Solver, this);
		}
	}

	// Update the source mesh skinned positions
	const int32 PrevParticleRangeId = GetParticleRangeId(Solver, PrevLODIndex);
	const int32 ParticleRangeId = GetParticleRangeId(Solver, LODIndex);
	if (PrevParticleRangeId == INDEX_NONE && ParticleRangeId == INDEX_NONE)
	{
		return;
	}

	// Retrieve config
	check(Config);
	const Softs::FCollectionPropertyFacade* const ConfigProperties = LODIndex == INDEX_NONE ? nullptr : &Config->GetProperties(LODIndex);
	int32 CurrentLODMorphTargetIndex = INDEX_NONE;
	float CurrentLODMorphTargetWeight = 0.f;
	if (LODIndex != INDEX_NONE)
	{
		LODData[LODIndex]->UpdateCachedProperties(*Mesh, LODIndex, *ConfigProperties);
		CurrentLODMorphTargetIndex = LODData[LODIndex]->ActiveMorphTarget;
		CurrentLODMorphTargetWeight = LODData[LODIndex]->ActiveMorphTargetWeight;
		LODData[LODIndex]->UpdateAccessoryMeshes(Solver);
	}

	Mesh->Update(Solver, PrevLODIndex, LODIndex, PrevParticleRangeId, ParticleRangeId, CurrentLODMorphTargetIndex, CurrentLODMorphTargetWeight);

	// LOD Switching
	if (LODIndex != PrevLODIndex)
	{
		if (PrevLODIndex != INDEX_NONE)
		{
			// Disable previous LOD's particles
			LODData[PrevLODIndex]->Enable(Solver, false);
		}
		if (LODIndex != INDEX_NONE)
		{
			// Enable new LOD's particles
			LODData[LODIndex]->Enable(Solver, true);
			NumActiveKinematicParticles = LODData[LODIndex]->NumKinematicParticles;
			NumActiveDynamicParticles = LODData[LODIndex]->NumDynamicParticles;

			// Wrap new LOD based on previous LOD if possible (can only do 1 level LOD at a time, and if previous LOD exists)
			bNeedsReset = bNeedsReset || ParticleRangeId == INDEX_NONE || PrevParticleRangeId == INDEX_NONE ||
				!Mesh->WrapDeformLOD(
					PrevLODIndex,
					LODIndex,
					Solver->GetNormals(PrevParticleRangeId),
					Solver->GetParticlePandInvMs(PrevParticleRangeId),
					Solver->GetParticleVs(PrevParticleRangeId),
					Solver->GetParticlePandInvMs(ParticleRangeId),
					Solver->GetParticleXs(ParticleRangeId),
					Solver->GetParticleVs(ParticleRangeId));

			if (!bNeedsReset)
			{
				LODData[LODIndex]->ResetAccessoryMeshes(Solver);
			}

			if (Solver->IsLegacySolver())
			{
				// Update the wind velocity field for the new LOD mesh
				Solver->SetWindAndPressureGeometry(GroupId, GetTriangleMesh(Solver), *ConfigProperties, LODData[LODIndex]->WeightMaps);
			}
		}
		else
		{
			NumActiveKinematicParticles = 0;
			NumActiveDynamicParticles = 0;
		}
	}

	// Update Cloth group parameters  TODO: Cloth groups should exist as their own node object so that they can be used by several cloth objects
	if (LODIndex != INDEX_NONE && ParticleRangeId != INDEX_NONE)
	{
		// TODO: Move all groupID updates out of the cloth update to allow to use of the same GroupId with different cloths

		// Set the reference input velocity and deal with teleport & reset; external forces depends on these values, so they must be initialized before then
		EChaosSoftsSimulationSpace VelocityScaleSpace = ClothingSimulationClothDefault::VelocityScaleSpace;
		FVec3f OutLinearVelocityScale;
		FRealSingle OutAngularVelocityScale;
		FRealSingle OutMaxVelocityScale;
		bool bDisableFictitiousForces = false;
		FVec3f MaxLinearVelocity(ClothingSimulationClothDefault::MaxVelocity);
		FVec3f MaxLinearAcceleration(ClothingSimulationClothDefault::MaxAcceleration);
		FRealSingle MaxAngularVelocity = ClothingSimulationClothDefault::MaxVelocity;
		FRealSingle MaxAngularAcceleration = ClothingSimulationClothDefault::MaxAcceleration;
		if (bNeedsReset)
		{
			// Make sure not to do any pre-sim transform just after a reset
			OldReferenceSpaceTransform = ReferenceSpaceTransform;
			OutLinearVelocityScale = FVec3f(1.f);
			OutAngularVelocityScale = 1.f;
			OutMaxVelocityScale = 1.f;
			AppliedReferenceSpaceAngularVelocity = FVec3(0.);
			AppliedReferenceSpaceVelocity = FVec3(0.);
			bDisableFictitiousForces = true;

			// Reset to start pose
			LODData[LODIndex]->ResetStartPose(Solver);
			for (FClothingSimulationCollider* Collider : Colliders)
			{
				Collider->ResetStartPose(Solver, this);
			}

			UE_LOGF(LogChaosCloth, Verbose, "Cloth in group Id %d Needs reset.", GroupId);
		}
		else if (bNeedsTeleport)
		{
			// Remove all impulse velocity from the last frame
			OutLinearVelocityScale = FVec3f(0.f);
			OutAngularVelocityScale = 0.f;
			OutMaxVelocityScale = 1.f;
			AppliedReferenceSpaceAngularVelocity = FVec3(0.);
			AppliedReferenceSpaceVelocity = FVec3(0.);
			bDisableFictitiousForces = true;
			UE_LOGF(LogChaosCloth, Verbose, "Cloth in group Id %d Needs teleport.", GroupId);
		}
		else
		{
			// Use the cloth config parameters
			VelocityScaleSpace = (EChaosSoftsSimulationSpace)GetVelocityScaleSpace(*ConfigProperties, (int32)ClothingSimulationClothDefault::VelocityScaleSpace);
			OutLinearVelocityScale = GetLinearVelocityScale(*ConfigProperties, FVector3f(ClothingSimulationClothDefault::VelocityScale));
			OutAngularVelocityScale = GetAngularVelocityScale(*ConfigProperties, ClothingSimulationClothDefault::VelocityScale);
			OutMaxVelocityScale = GetMaxVelocityScale(*ConfigProperties, ClothingSimulationClothDefault::MaxVelocityScale);
			MaxLinearVelocity = GetMaxLinearVelocity(*ConfigProperties, MaxLinearVelocity);
			MaxLinearAcceleration = GetMaxLinearAcceleration(*ConfigProperties, MaxLinearAcceleration);
			MaxAngularVelocity = GetMaxAngularVelocity(*ConfigProperties, MaxAngularVelocity);
			MaxAngularAcceleration = GetMaxAngularAcceleration(*ConfigProperties, MaxAngularAcceleration);

			// Apply the cvars clamp, if any are active
			using namespace ClothingSimulationClothConsoleVariables;
			MaxLinearVelocity = (GlobalMaxLinearVelocity > 0.f) ? MaxLinearVelocity.ComponentwiseMin(FVec3f(GlobalMaxLinearVelocity)) : MaxLinearVelocity;
			MaxLinearAcceleration = (GlobalMaxLinearAcceleration > 0.f) ? MaxLinearAcceleration.ComponentwiseMin(FVec3f(GlobalMaxLinearAcceleration)) : MaxLinearAcceleration;
			MaxAngularVelocity = (GlobalMaxAngularVelocity > 0.f) ? FMath::Min(MaxAngularVelocity, (FRealSingle)GlobalMaxAngularVelocity) : MaxAngularVelocity;
			MaxAngularAcceleration = (GlobalMaxAngularAcceleration > 0.f) ? FMath::Min(MaxAngularAcceleration, (FRealSingle)GlobalMaxAngularAcceleration) : MaxAngularAcceleration;
		}
		using EFictitiousForcesModel = FClothingSimulationSolver::EFictitiousForcesModel;
		const EFictitiousForcesModel FictitiousForcesModel = bDisableFictitiousForces ?
			EFictitiousForcesModel::None :
			(EFictitiousForcesModel)GetFictitiousForcesModel(*ConfigProperties, (int32)EFictitiousForcesModel::Legacy);

		// NOTE: Force-based solver doesn't actually use FictitiousAngularScale here. It gets it from the property collection directly.
		const FRealSingle FictitiousAngularScale = GetFictitiousAngularScale(*ConfigProperties, ClothingSimulationClothDefault::FictitiousAngularScale);
		Solver->SetReferenceVelocityScale(
			GroupId,
			OldReferenceSpaceTransform,
			ReferenceSpaceTransform,
			AppliedReferenceSpaceVelocity,
			AppliedReferenceSpaceAngularVelocity,
			VelocityScaleSpace,
			OutLinearVelocityScale,
			MaxLinearVelocity,
			MaxLinearAcceleration,
			OutAngularVelocityScale,
			MaxAngularVelocity,
			MaxAngularAcceleration,
			FictitiousAngularScale,
			OutMaxVelocityScale,
			FictitiousForcesModel);

		if (bNeedsReset || bNeedsTeleport)
		{
			// Record this frame's velocity as zero
			AppliedReferenceSpaceAngularVelocity = FVec3(0.);
			AppliedReferenceSpaceVelocity = FVec3(0.);
		}
		if (!Solver->IsLegacySolver())
		{
			Solver->SetProperties(ParticleRangeId, *ConfigProperties, LODData[LODIndex]->WeightMaps);
		}
		else
		{
			// Update gravity
			// This code relies on the solver gravity property being already set.
			// In order to use a cloth gravity override, it must first be enabled by the solver so that an override at solver level can still take precedence if needed.
			// In all cases apart from when the cloth override is used, the gravity scale must be combined to the solver gravity value.
			Solver->SetGravity(GroupId, GetGravity(Solver));

			// External forces (legacy wind+field)
			const bool bUsePointBasedWindModel = GetUsePointBasedWindModel(*ConfigProperties, false);
			Solver->AddExternalForces(GroupId, bUsePointBasedWindModel);

			const bool bPointBasedWindDisablesAccurateWind = ClothingSimulationClothConsoleVariables::CVarLegacyDisablesAccurateWind.GetValueOnAnyThread();
			const bool bEnableAerodynamics = !(bUsePointBasedWindModel && bPointBasedWindDisablesAccurateWind);
			Solver->SetWindAndPressureProperties(GroupId, *ConfigProperties, LODData[LODIndex]->WeightMaps, bEnableAerodynamics);

			constexpr float WorldScale = 100.f;  // VelocityField wind is in m/s in the config (same as the wind unit), but cm/s in the solver  TODO: Cleanup the Solver SetWindVelocity functions to be consistent with the unit
			const FVec3f WindVelocity = Softs::FVelocityAndPressureField::GetWindVelocity(*ConfigProperties, FVector3f(0.f)) * WorldScale;
			Solver->SetWindVelocity(GroupId, WindVelocity + Solver->GetWindVelocity());

			// Update general solver properties
			const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
			check(LocalSpaceScale > UE_SMALL_NUMBER);
			const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
			const Softs::FSolverReal MeshScale = Mesh->GetScale() * LocalSpaceScaleInv;

			const FRealSingle DampingCoefficient = GetDampingCoefficient(*ConfigProperties, ClothingSimulationClothDefault::DampingCoefficient);
			const FRealSingle LocalDampingCoefficient = GetLocalDampingCoefficient(*ConfigProperties, ClothingSimulationClothDefault::LocalDampingCoefficient);
			const FRealSingle CollisionThickness = GetCollisionThickness(*ConfigProperties, ClothingSimulationClothDefault::CollisionThickness);
			const FRealSingle FrictionCoefficient = GetFrictionCoefficient(*ConfigProperties, ClothingSimulationClothDefault::FrictionCoefficient);
			Solver->SetProperties(GroupId, DampingCoefficient, LocalDampingCoefficient, CollisionThickness * MeshScale, FrictionCoefficient);

			// Update use of continuous collision detection
			const bool bUseCCD = GetUseCCD(*ConfigProperties, false);
			Solver->SetUseCCD(GroupId, bUseCCD);
		}

		// This will be updated below if single legacy lod
		if (!Config->IsLegacySingleLOD())
		{
			LODData[LODIndex]->Update(Solver, this);
		}
	}

	// Update all LODs dirty properties, since it is easier done than re-updating all properties when switching LODs
	if (Config->IsLegacySingleLOD())
	{
		for (TUniquePtr<FLODData>& LODDatum : LODData)
		{
			LODDatum->Update(Solver, this);
		}
	}

	if (bResetRestLengthsFromMorphTarget)
	{
		check(!Solver->IsLegacySolver()); // Otherwise need to handle particle offsets
		for (int32 Index = 0; Index < LODData.Num(); ++Index)
		{
			const int32 CurrParticleRangeId = GetParticleRangeId(Solver, Index);
			TArray<Softs::FSolverVec3> RestLengthPositions(Mesh->GetPositions(Index));
			TArray<Softs::FSolverVec3> RestLengthNormals(Mesh->GetNormals(Index));
			const int32 MorphTargetIndex = Mesh->FindMorphTargetByName(Index, ResetRestLengthsMorphTargetName);
			if (MorphTargetIndex != INDEX_NONE)
			{
				Mesh->ApplyMorphTarget(Index, MorphTargetIndex, 1.f, RestLengthPositions, RestLengthNormals);
			}
			FClothConstraints& ClothConstraints = Solver->GetClothConstraints(CurrParticleRangeId);
			ClothConstraints.ResetRestLengths(RestLengthPositions, Config->GetProperties(Index), LODData[Index]->WeightMaps);
		}
	}

	// Reset trigger flags
#if CHAOS_DEBUG_DRAW
	if (bNeedsReset)
	{
		TimeSinceLastReset = 0.f;
		TimeSinceLastTeleport = 0.f;
	}
	else if (bNeedsTeleport)
	{
		TimeSinceLastReset += Solver->GetDeltaTime();
		TimeSinceLastTeleport = 0.f;
	}
	else
	{
		TimeSinceLastReset += Solver->GetDeltaTime();
		TimeSinceLastTeleport += Solver->GetDeltaTime();
	}

#endif
	bNeedsTeleport = false;
	bNeedsReset = false;
	bResetRestLengthsFromMorphTarget = false;
}

void FClothingSimulationCloth::PostUpdate(FClothingSimulationSolver* Solver)
{
	check(Solver);

	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODIndex != INDEX_NONE)
	{
		// Update normals
		LODData[LODIndex]->UpdateNormals(Solver);
	}
}

void FClothingSimulationCloth::UpdateFromCache(const FClothingSimulationCacheData& CacheData)
{
	if (const FTransform* CachedReferenceSpaceTransform = CacheData.CachedReferenceSpaceTransforms.Find(GetGroupId()))
	{
		ReferenceSpaceTransform = *CachedReferenceSpaceTransform;
		ReferenceSpaceTransform.SetScale3D(FVec3(1.f));
	}
}

void FClothingSimulationCloth::ApplyPreSimulationTransforms(FClothingSimulationSolver* Solver, const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime)
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODIndex != INDEX_NONE)
	{
		LODData[LODIndex]->ApplyPreSimulationTransformsToAccessoryMeshes(Solver, DeltaLocalSpaceLocation, GroupSpaceTransform, DeltaTime);
	}
}

void FClothingSimulationCloth::PreSubstep(FClothingSimulationSolver* Solver, const Softs::FSolverReal InterpolationAlpha)
{
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODIndex != INDEX_NONE)
	{
		LODData[LODIndex]->PreSubstepAccessoryMeshes(Solver, InterpolationAlpha);
	}
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAnimationPositions(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetAnimationPositions(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetOldAnimationPositions(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetOldAnimationPositions(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAnimationVelocities(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetAnimationVelocities(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAnimationNormals(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetAnimationNormals(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

bool FClothingSimulationCloth::AccessoryMeshExists(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODData.IsValidIndex(LODIndex))
	{
		const FLODData::FSolverData& SolverDatum = LODData[LODIndex]->SolverData.FindChecked(Solver);
		return SolverDatum.AccessoryMeshData.Find(AccessoryMeshName) != nullptr;
	}
	return false;
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAccessoryMeshPositions(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODData.IsValidIndex(LODIndex))
	{
		const FLODData::FSolverData& SolverDatum = LODData[LODIndex]->SolverData.FindChecked(Solver);
		if (const FClothingAccessoryMeshData* const AccessoryMesh = SolverDatum.AccessoryMeshData.Find(AccessoryMeshName))
		{
			return TConstArrayView<Softs::FSolverVec3>(AccessoryMesh->GetAnimationPositions());
		}
	}
	return TConstArrayView<Softs::FSolverVec3>();
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetOldAccessoryMeshPositions(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODData.IsValidIndex(LODIndex))
	{
		const FLODData::FSolverData& SolverDatum = LODData[LODIndex]->SolverData.FindChecked(Solver);
		if (const FClothingAccessoryMeshData* const AccessoryMesh = SolverDatum.AccessoryMeshData.Find(AccessoryMeshName))
		{
			return TConstArrayView<Softs::FSolverVec3>(AccessoryMesh->GetOldAnimationPositions());
		}
	}
	return TConstArrayView<Softs::FSolverVec3>();
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAccessoryMeshNormals(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODData.IsValidIndex(LODIndex))
	{
		const FLODData::FSolverData& SolverDatum = LODData[LODIndex]->SolverData.FindChecked(Solver);
		if (const FClothingAccessoryMeshData* const AccessoryMesh = SolverDatum.AccessoryMeshData.Find(AccessoryMeshName))
		{
			return TConstArrayView<Softs::FSolverVec3>(AccessoryMesh->GetAnimationNormals());
		}
	}
	return TConstArrayView<Softs::FSolverVec3>();
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetAccessoryMeshVelocities(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	if (LODData.IsValidIndex(LODIndex))
	{
		const FLODData::FSolverData& SolverDatum = LODData[LODIndex]->SolverData.FindChecked(Solver);
		if (const FClothingAccessoryMeshData* const AccessoryMesh = SolverDatum.AccessoryMeshData.Find(AccessoryMeshName))
		{
			return TConstArrayView<Softs::FSolverVec3>(AccessoryMesh->GetAnimationVelocities());
		}
	}
	return TConstArrayView<Softs::FSolverVec3>();
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetParticlePositions(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleXs(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetParticleNormals(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetNormals(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationCloth::GetParticleVelocities(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverVec3>(Solver->GetParticleVs(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}

TConstArrayView<Softs::FSolverReal> FClothingSimulationCloth::GetParticleInvMasses(const FClothingSimulationSolver* Solver) const
{
	check(Solver);
	const int32 LODIndex = LODIndices.FindChecked(Solver);
	check(GetParticleRangeId(Solver, LODIndex) != INDEX_NONE);
	return TConstArrayView<Softs::FSolverReal>(Solver->GetParticleInvMasses(GetParticleRangeId(Solver, LODIndex)), GetNumParticles(LODIndex));
}
}  // End namespace Chaos
