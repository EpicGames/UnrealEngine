// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothComplexColliders.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "Chaos/Capsule.h"
#include "Chaos/ExtrudedTaperedCapsule.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/TaperedCylinder.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/Convex.h"
#include "Chaos/Levelset.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "ClothingAsset.h"
#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "HAL/PlatformMath.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Chaos/MLLevelset.h"

DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Update Collider"), STAT_ChaosClothingSimulationColliderUpdate, STATGROUP_ChaosCloth);

namespace Chaos
{

namespace ClothingSimulationColliderConsoleVariables
{
	TAutoConsoleVariable<bool> CVarUseOptimizedTaperedCapsule(TEXT("p.ChaosCloth.UseOptimizedTaperedCapsule"), true, TEXT("Use the optimized TaperedCapsule code instead of using a tapered cylinder and two spheres. Note that only optimized TaperedCapsule supports split tapered capsules"));
}

struct FClothingSimulationCollider::FLODData
{
	FClothCollisionData ClothCollisionData;
	int32 NumGeometries;  // Number of collision bodies
	TMap<FSolverClothPair, int32> CollisionRangeIds;  // Solver particle range ids

	FLODData() : NumGeometries(0) {}

	void Add(
		FClothingSimulationSolver* Solver,
		FClothingSimulationCloth* Cloth,
		const FClothCollisionData& InClothCollisionData,
		const FClothCollisionDataExtended& InClothCollisionDataExtended,
		const FReal InScale = 1.f,
		const TArray<int32>& UsedBoneIndices = TArray<int32>(),
		const TArray<int32>& UsedSubBoneIndices = TArray<int32>());

	void Remove(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);

	void Update(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, const FTransform& ComponentTransform, const TArray<FTransform>& BoneTransforms);

	void Enable(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, bool bEnable);

	void ResetStartPose(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);

	FORCEINLINE static int32 GetMappedBoneIndex(const TArray<int32>& UsedBoneIndices, int32 BoneIndex)
	{
		return UsedBoneIndices.IsValidIndex(BoneIndex) ? UsedBoneIndices[BoneIndex] : INDEX_NONE;
	}
};

void FClothingSimulationCollider::FLODData::Add(
	FClothingSimulationSolver* Solver,
	FClothingSimulationCloth* Cloth,
	const FClothCollisionData& InClothCollisionData,
	const FClothCollisionDataExtended& InClothCollisionDataExtended,
	const FReal InComponentScale,
	const TArray<int32>& UsedBoneIndices,
	const TArray<int32>& UsedSubBoneIndices)
{
	check(Solver);
	check(Cloth);

	// Keep a list of all collisions
	ClothCollisionData = InClothCollisionData;

	// Apply LocalScaleInv.
	const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
	check(LocalSpaceScale > UE_SMALL_NUMBER);
	const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
	const FReal InScale = InComponentScale * LocalSpaceScaleInv;

	// Calculate the number of geometries
	int32 NumSpheres = ClothCollisionData.Spheres.Num();
	TBitArray<TInlineAllocator<8>> CapsuleEnds(false, NumSpheres);  // Init on stack for 256 spheres, any number over this will be allocated

	const int32 NumCapsules = ClothCollisionData.SphereConnections.Num();
	for (int32 Index = 0; Index < NumCapsules; ++Index)
	{
		const FClothCollisionPrim_SphereConnection& Connection = ClothCollisionData.SphereConnections[Index];
		for (int32 SphereIndex : Connection.SphereIndices)
		{
			FBitReference IsCapsuleEnd = CapsuleEnds[SphereIndex];
			if (!IsCapsuleEnd)
			{
				IsCapsuleEnd = true;
				--NumSpheres;
			}
		}
	}

	const int32 NumConvexes = ClothCollisionData.Convexes.Num();
	const int32 NumBoxes = ClothCollisionData.Boxes.Num();
	const int32 NumExtrudedCapsules = ClothCollisionData.ExtrudedTaperedCapsules.Num();
	const int32 NumLevelSets = InClothCollisionDataExtended.LevelSets.Num();
	const int32 NumSkinnedLevelSets = InClothCollisionDataExtended.SkinnedLevelSets.Num();
	const int32 NumMLLevelSets = InClothCollisionDataExtended.MLLevelSets.Num();
	const int32 NumSkinnedTriangleMeshes = InClothCollisionDataExtended.SkinnedTriangleMeshes.Num();

	const bool bSupportsComplexColliders = !Solver->IsLegacySolver();
	const int32 NumSimpleColliders = NumSpheres + NumCapsules + NumConvexes + NumBoxes + NumExtrudedCapsules + NumLevelSets;
	const int32 NumComplexColliders = NumSkinnedLevelSets + NumMLLevelSets + NumSkinnedTriangleMeshes;

	NumGeometries = NumSimpleColliders + (bSupportsComplexColliders ? NumComplexColliders : 0);

	// Retrieve cloth group Id, or use INDEX_NONE if this collider applies to all cloths (when Cloth == nullptr)
	const uint32 GroupId = Cloth ? Cloth->GetGroupId() : INDEX_NONE;

	// The offset will be set to the first collision particle's index
	// Try to reuse existing offsets when Add is called during the collider update (ie Offset isn't INDEX_NONE)
	int32* const CollisionRangeIdPtr = CollisionRangeIds.Find(FSolverClothPair(Solver, Cloth));
	const bool bIsNewCollider = !CollisionRangeIdPtr;
	int32& CollisionRangeId = bIsNewCollider ? CollisionRangeIds.Add(FSolverClothPair(Solver, Cloth)) : *CollisionRangeIdPtr;
	CollisionRangeId = Solver->AddCollisionParticles(NumGeometries, GroupId, bIsNewCollider ? INDEX_NONE : CollisionRangeId);

	// Must try to add collision particles even when zero, so it will remove any existing collisions
	if (NumGeometries == 0)
	{
		return;
	}

	TArrayView<int32> BoneIndices = Solver->GetCollisionBoneIndicesView(CollisionRangeId);
	TArrayView<Softs::FSolverRigidTransform3> BaseTransforms = Solver->GetCollisionBaseTransformsView(CollisionRangeId);

	// Capsules
	const int32 CapsuleIndexOffset = 0;
	if (NumCapsules)
	{
		for (int32 Index = CapsuleIndexOffset; Index < CapsuleIndexOffset + NumCapsules; ++Index)
		{
			const FClothCollisionPrim_SphereConnection& Connection = ClothCollisionData.SphereConnections[Index];

			const int32 SphereIndex0 = Connection.SphereIndices[0];
			const int32 SphereIndex1 = Connection.SphereIndices[1];
			checkSlow(SphereIndex0 != SphereIndex1);
			const FClothCollisionPrim_Sphere& Sphere0 = ClothCollisionData.Spheres[SphereIndex0];
			const FClothCollisionPrim_Sphere& Sphere1 = ClothCollisionData.Spheres[SphereIndex1];

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, Sphere0.BoneIndex);
			checkSlow(Sphere0.BoneIndex == Sphere1.BoneIndex);
			UE_CLOGF(Sphere0.BoneIndex != Sphere1.BoneIndex,
				LogChaosCloth, Warning, "Found a legacy cloth asset with a collision capsule spanning across two bones. This is not supported with the current system.");
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision capsule on bone index %d.", BoneIndices[Index]);

			const FVec3 X0 = Sphere0.LocalPosition * InScale;
			const FVec3 X1 = Sphere1.LocalPosition * InScale;
			const FVec3 Center = (X0 + X1) * 0.5f;
			const FVec3 Axis = (X1 - X0) * 0.5f;
			const FVec3 P0 = Center - Axis;
			const FVec3 P1 = Center + Axis;

			const FReal Radius0 = Sphere0.Radius * InScale;
			const FReal Radius1 = Sphere1.Radius * InScale;
			FReal MinRadius, MaxRadius;
			if (Radius0 <= Radius1) { MinRadius = Radius0; MaxRadius = Radius1; }
			else { MinRadius = Radius1; MaxRadius = Radius0; }

			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			if (Axis.SizeSquared() < SMALL_NUMBER && Connection.OneSidedPlaneNormal.IsNearlyZero())
			{
				// Sphere
				Solver->SetCollisionGeometry(CollisionRangeId, Index,
					MakeImplicitObjectPtr<FSphere>(Center, MaxRadius));
			}
			else if (MaxRadius - MinRadius < KINDA_SMALL_NUMBER && Connection.OneSidedPlaneNormal.IsNearlyZero())
			{
				// Capsule
				Solver->SetCollisionGeometry(CollisionRangeId, Index,
					MakeImplicitObjectPtr<FCapsule>(P0, P1, MaxRadius));
			}
			else
			{
				if (ClothingSimulationColliderConsoleVariables::CVarUseOptimizedTaperedCapsule.GetValueOnAnyThread())
				{
					Solver->SetCollisionGeometry(CollisionRangeId, Index,
						MakeImplicitObjectPtr<FTaperedCapsule>(P0, P1, Radius0, Radius1, Connection.OneSidedPlaneNormal));
				}
				else
				{
					// Tapered capsule approximate by a union of tapered cylinder and two spheres
					TArray<FImplicitObjectPtr> Objects;
					Objects.Reserve(3);
					Objects.Add(MakeImplicitObjectPtr<FTaperedCylinder>(P0, P1, Radius0, Radius1));
					Objects.Add(MakeImplicitObjectPtr<FSphere>(P0, Radius0));
					Objects.Add(MakeImplicitObjectPtr<FSphere>(P1, Radius1));
					Solver->SetCollisionGeometry(CollisionRangeId, Index, MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Objects)));
				}
			}
		}
	}

	// Spheres
	const int32 SphereIndexOffset = CapsuleIndexOffset + NumCapsules;
	if (NumSpheres != 0)
	{
		for (int32 Index = SphereIndexOffset, SphereIndex = 0; SphereIndex < ClothCollisionData.Spheres.Num(); ++SphereIndex)
		{
			// Skip spheres that are the end caps of capsules.
			if (CapsuleEnds[SphereIndex])
			//if (CapsuleEnds.Contains(SphereIndex))
			{
				continue;
			}

			const FClothCollisionPrim_Sphere& Sphere = ClothCollisionData.Spheres[SphereIndex];

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, Sphere.BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision sphere on bone index %d.", BoneIndices[Index]);

			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			Solver->SetCollisionGeometry(CollisionRangeId, Index,
				MakeImplicitObjectPtr<FSphere>(
					Sphere.LocalPosition * InScale,
					Sphere.Radius * InScale));

			++Index;
		}
	}

	// Convexes
	const int32 ConvexIndexOffset = SphereIndexOffset + NumSpheres;
	if (NumConvexes != 0)
	{
		for (int32 Index = ConvexIndexOffset, ConvexIndex = 0; ConvexIndex < NumConvexes; ++Index, ++ConvexIndex)
		{
			const FClothCollisionPrim_Convex& Convex = ClothCollisionData.Convexes[ConvexIndex];

			BaseTransforms[Index] = Softs::FSolverRigidTransform3(Convex.LocalPosition, Convex.LocalRotation);

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, Convex.BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision convex on bone index %d.", BoneIndices[Index]);

			TArray<FConvex::FPlaneType> Planes;
			TArray<TArray<int32>> FaceIndices;

			const int32 NumSurfacePoints = Convex.SurfacePoints.Num();
			const int32 NumFaces = Convex.Faces.Num();

			if (NumSurfacePoints < 4)
			{
				UE_LOGF(LogChaosCloth, Warning, "Invalid convex collision: not enough surface points.");
			}
			else if (NumFaces < 4)
			{
				UE_LOGF(LogChaosCloth, Warning, "Invalid convex collision: not enough faces.");
			}
			else
			{
				// Retrieve convex planes
				Planes.Reserve(NumFaces);
				FaceIndices.Reserve(NumFaces);
				for (const FClothCollisionPrim_ConvexFace& Face : Convex.Faces)
				{
					const FPlane& Plane = Face.Plane;
					FPlane NormalizedPlane(Plane);
					if (!NormalizedPlane.Normalize())
					{
						UE_LOGF(LogChaosCloth, Warning, "Invalid convex collision: bad plane normal.");
						break;
					}
					if (!Face.Indices.Num())
					{
						UE_LOGF(LogChaosCloth, Warning, "Invalid convex collision: no face indices, this asset might be using a deprecated Apex collision shape.");
						break;
					}
					const FConvex::FVec3Type Normal(static_cast<FVector>(NormalizedPlane));
					const FConvex::FVec3Type Base = Normal * NormalizedPlane.W * InScale;
					Planes.Add(FConvex::FPlaneType(Base, Normal));
					FaceIndices.Add(Face.Indices);
				}
			}

			if (Planes.Num() == NumFaces)
			{
				check(Planes.Num() == FaceIndices.Num());

				// Retrieve the convex vertices and face indices
				TArray<FConvex::FVec3Type> Vertices;
				Vertices.SetNum(NumSurfacePoints);
				for (int32 ParticleIndex = 0; ParticleIndex < NumSurfacePoints; ++ParticleIndex)
				{
					Vertices[ParticleIndex] = Convex.SurfacePoints[ParticleIndex];
				}

				// Setup the collision particle geometry
				Solver->SetCollisionGeometry(CollisionRangeId, Index, MakeImplicitObjectPtr<FConvex>(MoveTemp(Planes), MoveTemp(FaceIndices), MoveTemp(Vertices)));
			}
			else
			{
				UE_LOGF(LogChaosCloth, Warning, "Replacing invalid convex collision by a default unit sphere.");
				Solver->SetCollisionGeometry(CollisionRangeId, Index, MakeImplicitObjectPtr<FSphere>(FVec3(0.0f), 1.0f));  // Default to a unit sphere to replace the faulty convex
			}
		}
	}

	// Boxes
	const int32 BoxIndexOffset = ConvexIndexOffset + NumConvexes;
	if (NumBoxes != 0)
	{		
		for (int32 Index = BoxIndexOffset, BoxIndex = 0; BoxIndex < NumBoxes; ++Index, ++BoxIndex)
		{
			const FClothCollisionPrim_Box& Box = ClothCollisionData.Boxes[BoxIndex];
			
			BaseTransforms[Index] = Softs::FSolverRigidTransform3(Box.LocalPosition, Box.LocalRotation);
			
			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, Box.BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision box on bone index %d.", BoneIndices[Index]);

			const FVec3 HalfExtents = Box.HalfExtents * InScale;
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MakeImplicitObjectPtr<TBox<FReal, 3>>(-HalfExtents, HalfExtents));
		}
	}

	// Extruded Tapered Capsules
	const int32 ExtrudedCapsuleIndexOffset = BoxIndexOffset + NumBoxes;
	if (NumExtrudedCapsules != 0)
	{
		for (int32 Index = ExtrudedCapsuleIndexOffset, ExtrudedCapsuleIndex = 0; ExtrudedCapsuleIndex < NumExtrudedCapsules; ++Index, ++ExtrudedCapsuleIndex)
		{
			const FClothCollisionPrim_ExtrudedTaperedCapsule ExtrudedCapsule = ClothCollisionData.ExtrudedTaperedCapsules[ExtrudedCapsuleIndex];
			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, ExtrudedCapsule.BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision ExtrudedCapsule on bone index %d.", BoneIndices[Index]);

			const FVec3f MajorAxis = FVec3f(ExtrudedCapsule.LocalRotation.RotateVector(FVector::UpVector));
			const FVec3f MinorAxis = FVec3f(ExtrudedCapsule.LocalRotation.RotateVector(FVector::ForwardVector));
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MakeImplicitObjectPtr<FExtrudedTaperedCapsule>(FVec3f(ExtrudedCapsule.LocalPosition * InScale), MajorAxis, MinorAxis, ExtrudedCapsule.Length * InScale * .5f, ExtrudedCapsule.Width * InScale * .5f, ExtrudedCapsule.Radius1 * InScale, ExtrudedCapsule.Radius2 * InScale));
		}
	}
	
	// Level sets
	const int32 LevelSetIndexOffset = ExtrudedCapsuleIndexOffset + NumExtrudedCapsules;
	if (NumLevelSets != 0)
	{
		for (int32 Index = LevelSetIndexOffset, LevelSetIndex = 0; LevelSetIndex < NumLevelSets; ++Index, ++LevelSetIndex)
		{
			// Always initialize the collision particle transforms before setting any geometry as otherwise NaNs gets detected during the bounding box updates
			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, InClothCollisionDataExtended.LevelSets[LevelSetIndex].BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision level set on bone index %d.", BoneIndices[Index]);

			// Setup the collision particle geometry
			FImplicitObjectPtr TransformedLevelSet = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(InClothCollisionDataExtended.LevelSets[LevelSetIndex].LevelSet->DeepCopyGeometry(),
				TRigidTransform<FReal, 3>(InClothCollisionDataExtended.LevelSets[LevelSetIndex].Transform));
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MoveTemp(TransformedLevelSet));
		}
	}

	UE_LOGF(LogChaosCloth, VeryVerbose, "Added collisions: %d spheres, %d capsules, %d convexes, %d boxes, %d extruded tapered capsules, %d level sets.", NumSpheres, NumCapsules, NumConvexes, NumBoxes, NumExtrudedCapsules, NumLevelSets);

	if (bSupportsComplexColliders)
	{
		FClothComplexColliders* const ComplexColliders = Solver->GetComplexColliders(CollisionRangeId);
		check(ComplexColliders);
		ComplexColliders->Reset();
		ComplexColliders->AddSubBoneIndices(UsedSubBoneIndices);

		const int32 SkinnedLevelSetIndexOffset = LevelSetIndexOffset + NumLevelSets;
		for (int32 Index = SkinnedLevelSetIndexOffset, SkinnedLevelSetIndex = 0; SkinnedLevelSetIndex < NumSkinnedLevelSets; ++Index, ++SkinnedLevelSetIndex)
		{
			// Always initialize the collision particle transforms before setting any geometry as otherwise NaNs gets detected during the bounding box updates
			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, InClothCollisionDataExtended.SkinnedLevelSets[SkinnedLevelSetIndex].BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision skinned level set on bone index %d.", BoneIndices[Index]);

			FImplicitObjectPtr SkinnedLevelSet = InClothCollisionDataExtended.SkinnedLevelSets[SkinnedLevelSetIndex].WeightedLevelSet->DeepCopyGeometry();
			ComplexColliders->AddSkinnedLevelSet(Index, InClothCollisionDataExtended.SkinnedLevelSets[SkinnedLevelSetIndex].MappedSkinnedBones, SkinnedLevelSet);
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MoveTemp(SkinnedLevelSet));
		}

		const int32 MLLevelSetIndexOffset = SkinnedLevelSetIndexOffset + NumSkinnedLevelSets;
		for (int32 Index = MLLevelSetIndexOffset, MLLevelsetIndex = 0; MLLevelsetIndex < NumMLLevelSets; ++Index, ++MLLevelsetIndex)
		{
			// Always initialize the collision particle transforms before setting any geometry as otherwise NaNs gets detected during the bounding box updates
			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, InClothCollisionDataExtended.MLLevelSets[MLLevelsetIndex].BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision ml level set on bone index %d.", BoneIndices[Index]);

			FImplicitObjectPtr MLLevelSet = InClothCollisionDataExtended.MLLevelSets[MLLevelsetIndex].MLLevelSet->CopyGeometry();
			ComplexColliders->AddMLLevelSet(Index, InClothCollisionDataExtended.MLLevelSets[MLLevelsetIndex].MappedActiveBoneIndices, MLLevelSet);
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MoveTemp(MLLevelSet));
		}

		const int32 SkinnedTriMeshIndexOffset = MLLevelSetIndexOffset + NumMLLevelSets;
		for (int32 Index = SkinnedTriMeshIndexOffset, SkinnedTriMeshIndex = 0; SkinnedTriMeshIndex < NumSkinnedTriangleMeshes; ++Index, ++SkinnedTriMeshIndex)
		{
			// Always initialize the collision particle transforms before setting any geometry as otherwise NaNs gets detected during the bounding box updates
			BaseTransforms[Index] = Softs::FSolverRigidTransform3::Identity;

			BoneIndices[Index] = GetMappedBoneIndex(UsedBoneIndices, InClothCollisionDataExtended.SkinnedTriangleMeshes[SkinnedTriMeshIndex].BoneIndex);
			UE_LOGF(LogChaosCloth, VeryVerbose, "Found collision skinned triangle mesh on bone index %d.", BoneIndices[Index]);

			FImplicitObjectPtr SkinnedTriMesh = InClothCollisionDataExtended.SkinnedTriangleMeshes[SkinnedTriMeshIndex].SkinnedTriangleMesh->CopyGeometry();
			ComplexColliders->AddSkinnedTriangleMesh(Index, InClothCollisionDataExtended.SkinnedTriangleMeshes[SkinnedTriMeshIndex].MappedSkinnedBones, SkinnedTriMesh);
			Solver->SetCollisionGeometry(CollisionRangeId, Index, MoveTemp(SkinnedTriMesh));
		}
	}
}

void FClothingSimulationCollider::FLODData::Remove(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	CollisionRangeIds.Remove(FSolverClothPair(Solver, Cloth));
}

void FClothingSimulationCollider::FLODData::Update(
	FClothingSimulationSolver* Solver,
	FClothingSimulationCloth* Cloth,
	const FTransform& ComponentTransform,
	const TArray<FTransform>& BoneTransforms)
{
	check(Solver);
	check(Cloth);
	if (NumGeometries)
	{
		const int32 CollisionRangeId = CollisionRangeIds.FindChecked(FSolverClothPair(Solver, Cloth));
		TConstArrayView<int32> BoneIndices = Solver->GetCollisionBoneIndicesView(CollisionRangeId);
		TConstArrayView<Softs::FSolverRigidTransform3> BaseTransforms = Solver->GetCollisionBaseTransformsView(CollisionRangeId);
		TArrayView<Softs::FSolverRigidTransform3> CollisionTransforms = Solver->GetCollisionTransformsView(CollisionRangeId);

		FTransform ComponentToLocalSpaceReal = ComponentTransform;
		ComponentToLocalSpaceReal.AddToTranslation(-Solver->GetLocalSpaceLocation());
		const FReal LocalSpaceScale = Solver->GetLocalSpaceScale();
		check(LocalSpaceScale > UE_SMALL_NUMBER);
		const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
		ComponentToLocalSpaceReal.MultiplyScale3D(FVector(LocalSpaceScaleInv));
		ComponentToLocalSpaceReal.ScaleTranslation(LocalSpaceScaleInv);
		const Softs::FSolverTransform3 ComponentToLocalSpace(ComponentToLocalSpaceReal);  // LWC, now in local space, therefore it is safe to use the solver transform type
	
		// Update the collision transforms
		for (int32 Index = 0; Index < NumGeometries; ++Index)
		{
			const int32 BoneIndex = BoneIndices[Index];
			CollisionTransforms[Index] = BoneTransforms.IsValidIndex(BoneIndex) ?
				BaseTransforms[Index] * Softs::FSolverTransform3(BoneTransforms[BoneIndex]) * ComponentToLocalSpace :  // LWC requires the BoneTransform cast to the solver underlying type
				BaseTransforms[Index] * ComponentToLocalSpace;  // External collisions don't map to a bone
		}

		if (FClothComplexColliders* const ComplexColliders = Solver->GetComplexColliders(CollisionRangeId))
		{
			ComplexColliders->Update(ComponentToLocalSpace, BoneTransforms, CollisionTransforms);
		}
	}
}

void FClothingSimulationCollider::FLODData::Enable(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth, bool bEnable)
{
	check(Solver);
	check(Cloth);
	if (NumGeometries)
	{
		const int32 CollisionRangeId = CollisionRangeIds.FindChecked(FSolverClothPair(Solver, Cloth));
		Solver->EnableCollisionParticles(CollisionRangeId, bEnable);
	}
}

void FClothingSimulationCollider::FLODData::ResetStartPose(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	check(Solver);
	check(Cloth);
	if (NumGeometries)
	{
		const int32 CollisionRangeId = CollisionRangeIds.FindChecked(FSolverClothPair(Solver, Cloth));
		Solver->ResetCollisionStartPose(CollisionRangeId, NumGeometries);

		if (FClothComplexColliders* const ComplexColliders = Solver->GetComplexColliders(CollisionRangeId))
		{
			ComplexColliders->ResetStartPose();
		}
	}
}

FClothingSimulationCollider::FClothingSimulationCollider(const UPhysicsAsset* InPhysicsAsset, const FReferenceSkeleton* InReferenceSkeleton)
	: PhysicsAsset(InPhysicsAsset)
	, ReferenceSkeleton(InReferenceSkeleton)
{
	// Prepare LOD array
	// Note: There aren't any collision LODs stored with this constructor version anymore,
	//       it's only this physics asset's and external collisions.
	const int32 NumLODData = (int32)ECollisionDataType::LODs;
	LODData.Reserve(NumLODData);
	for (int32 Index = 0; Index < NumLODData; ++Index)
	{
		LODData.Add(MakeUnique<FLODData>());
	}
}


FClothingSimulationCollider::~FClothingSimulationCollider()
{
}

int32 FClothingSimulationCollider::GetNumGeometries() const
{
	int32 NumGeometries = 0;
	for (const TUniquePtr<FLODData>& LODDatum : LODData)
	{
		NumGeometries += LODDatum->NumGeometries;
	}
	return NumGeometries;
}

FClothCollisionData FClothingSimulationCollider::GetCollisionData(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth) const
{
	FClothCollisionData ClothCollisionData;
	ClothCollisionData.Append(LODData[(int32)ECollisionDataType::LODless]->ClothCollisionData);
	ClothCollisionData.Append(LODData[(int32)ECollisionDataType::External]->ClothCollisionData);

	const int32 LODIndex = LODIndices.FindChecked(FSolverClothPair(Solver, Cloth));
	if (LODIndex >= (int32)ECollisionDataType::LODs)
	{
		ClothCollisionData.Append(LODData[LODIndex]->ClothCollisionData);
	}
	return ClothCollisionData;
}

void FClothingSimulationCollider::ExtractPhysicsAssetCollision(
	const UPhysicsAsset* PhysicsAsset,
	const FReferenceSkeleton* ReferenceSkeleton,
	FClothCollisionData& ClothCollisionData,
	FClothCollisionDataExtended& ClothCollisionDataExtended,
	TArray<int32>& UsedBoneIndices,
	TArray<int32>& UsedSubBoneIndices,
	bool bUseSphylsOnly,
	bool bSkipMissingBones)
{
	ClothCollisionData.Reset();
	ClothCollisionDataExtended.Reset();
	UsedBoneIndices.Reset();
	UsedSubBoneIndices.Reset();

	if (PhysicsAsset)
	{
		UsedBoneIndices.Reserve(PhysicsAsset->SkeletalBodySetups.Num());

		for (const USkeletalBodySetup* BodySetup : PhysicsAsset->SkeletalBodySetups)
		{
			if (!BodySetup)
			{
				continue;
			}

			const int32 MeshBoneIndex = ReferenceSkeleton ? ReferenceSkeleton->FindBoneIndex(BodySetup->BoneName) : INDEX_NONE;
			if (bSkipMissingBones && MeshBoneIndex == INDEX_NONE)
			{
				continue;
			}

			const int32 MappedBoneIndex = UsedBoneIndices.Add(MeshBoneIndex);
			
			// Add capsules
			const FKAggregateGeom& AggGeom = BodySetup->AggGeom;
			if (AggGeom.SphylElems.Num())
			{
				for (const FKSphylElem& SphylElem : AggGeom.SphylElems)
				{
					if (SphylElem.Length == 0.0f)
					{
						// Add extracted sphere collision data
						FClothCollisionPrim_Sphere Sphere;
						Sphere.LocalPosition = SphylElem.Center;
						Sphere.Radius = SphylElem.Radius;
						Sphere.BoneIndex = MappedBoneIndex;
						ClothCollisionData.Spheres.Add(Sphere);
					}
					else
					{
						// Add extracted spheres collision data
						FClothCollisionPrim_Sphere Sphere0;
						FClothCollisionPrim_Sphere Sphere1;
						const FVector OrientedDirection = SphylElem.Rotation.RotateVector(FVector::UpVector);
						const FVector HalfDim = OrientedDirection * (SphylElem.Length / 2.f);
						Sphere0.LocalPosition = SphylElem.Center - HalfDim;
						Sphere1.LocalPosition = SphylElem.Center + HalfDim;
						Sphere0.Radius = SphylElem.Radius;
						Sphere1.Radius = SphylElem.Radius;
						Sphere0.BoneIndex = MappedBoneIndex;
						Sphere1.BoneIndex = MappedBoneIndex;

						// Add extracted sphere connection collision data
						FClothCollisionPrim_SphereConnection SphereConnection;
						SphereConnection.SphereIndices[0] = ClothCollisionData.Spheres.Add(Sphere0);
						SphereConnection.SphereIndices[1] = ClothCollisionData.Spheres.Add(Sphere1);
						ClothCollisionData.SphereConnections.Add(SphereConnection);
					}
				}
			}

			// Add spheres
			for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
			{
				// Add extracted sphere collision data
				FClothCollisionPrim_Sphere Sphere;
				Sphere.LocalPosition = SphereElem.Center;
				Sphere.Radius = SphereElem.Radius;
				Sphere.BoneIndex = MappedBoneIndex;
				ClothCollisionData.Spheres.Add(Sphere);
			}

			// Add tapered capsules
			for (const FKTaperedCapsuleElem& TaperedCapsuleElem : AggGeom.TaperedCapsuleElems)
			{
				if (TaperedCapsuleElem.IsExtruded())
				{
					FClothCollisionPrim_ExtrudedTaperedCapsule Capsule;
					Capsule.LocalPosition = TaperedCapsuleElem.Center;
					Capsule.LocalRotation = TaperedCapsuleElem.Rotation.Quaternion();
					Capsule.Length = TaperedCapsuleElem.Length;
					Capsule.Width = TaperedCapsuleElem.Width;
					Capsule.Radius1 = TaperedCapsuleElem.Radius0;
					Capsule.Radius2 = TaperedCapsuleElem.Radius1;
					Capsule.BoneIndex = MappedBoneIndex;
					ClothCollisionData.ExtrudedTaperedCapsules.Add(MoveTemp(Capsule));
				}
				else if (TaperedCapsuleElem.Length == 0 && !TaperedCapsuleElem.IsOneSidedCollision())
				{
					// Add extracted sphere collision data
					FClothCollisionPrim_Sphere Sphere;
					Sphere.LocalPosition = TaperedCapsuleElem.Center;
					Sphere.Radius = FMath::Max(TaperedCapsuleElem.Radius0, TaperedCapsuleElem.Radius1);
					Sphere.BoneIndex = MappedBoneIndex;
					ClothCollisionData.Spheres.Add(Sphere);
				}
				else
				{
					// Add extracted spheres collision data
					FClothCollisionPrim_Sphere Sphere0;
					FClothCollisionPrim_Sphere Sphere1;
					const FVector OrientedDirection = TaperedCapsuleElem.Rotation.RotateVector(FVector::UpVector);
					const FVector HalfDim = OrientedDirection * (TaperedCapsuleElem.Length / 2.f);
					Sphere0.LocalPosition = TaperedCapsuleElem.Center + HalfDim;
					Sphere1.LocalPosition = TaperedCapsuleElem.Center - HalfDim;
					Sphere0.Radius = TaperedCapsuleElem.Radius0;
					Sphere1.Radius = TaperedCapsuleElem.Radius1;
					Sphere0.BoneIndex = MappedBoneIndex;
					Sphere1.BoneIndex = MappedBoneIndex;

					// Add extracted sphere connection collision data
					FClothCollisionPrim_SphereConnection SphereConnection;
					SphereConnection.SphereIndices[0] = ClothCollisionData.Spheres.Add(Sphere0);
					SphereConnection.SphereIndices[1] = ClothCollisionData.Spheres.Add(Sphere1);
					if (TaperedCapsuleElem.IsOneSidedCollision())
					{
						SphereConnection.OneSidedPlaneNormal = TaperedCapsuleElem.Rotation.RotateVector(FVector::ForwardVector);
					}
					ClothCollisionData.SphereConnections.Add(SphereConnection);
				}
			}

			if (bUseSphylsOnly)
			{
				continue;  // Exit now if only spheres and capsules are required
			}

			// Add boxes
			for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
			{
				// Add extracted box collision data
				FClothCollisionPrim_Box Box;
				Box.LocalPosition = BoxElem.Center;
				Box.LocalRotation = BoxElem.Rotation.Quaternion();
				Box.HalfExtents = FVector(BoxElem.X, BoxElem.Y, BoxElem.Z) * 0.5f;
				Box.BoneIndex = MappedBoneIndex;
				ClothCollisionData.Boxes.Add(Box);
			}

			// Add convexes
			for (const FKConvexElem& ConvexElem : AggGeom.ConvexElems)
			{
				// Add stub for extracted collision data
				FClothCollisionPrim_Convex Convex;
				Convex.BoneIndex = MappedBoneIndex;
				const FImplicitObject& ChaosConvexMesh = *ConvexElem.GetChaosConvexMesh();
				const FConvex& ChaosConvex = ChaosConvexMesh.GetObjectChecked<FConvex>();

				// Copy faces' planes and indices
				const TArray<FConvex::FPlaneType>& Faces = ChaosConvex.GetFaces();
				Convex.Faces.SetNum(Faces.Num());
				for (int32 FaceIndex = 0; FaceIndex < Faces.Num(); ++FaceIndex)
				{
					FClothCollisionPrim_ConvexFace& Face = Convex.Faces[FaceIndex];

					// Copy face plane
					const FConvex::FPlaneType& PlaneConcrete = Faces[FaceIndex];
					Face.Plane = FPlane((FVector)PlaneConcrete.X(), (FVector)PlaneConcrete.Normal());

					// Copy face indices
					const int32 NumVertexIndices = ChaosConvex.NumPlaneVertices(FaceIndex);
					Face.Indices.SetNum(NumVertexIndices);
					for (int32 Vertex = 0; Vertex < NumVertexIndices; ++Vertex)
					{
						Face.Indices[Vertex] = ChaosConvex.GetPlaneVertex(FaceIndex, Vertex);
					}
				}

				// Copy surface points
				const int32 NumSurfacePoints = ChaosConvex.NumVertices();
				Convex.SurfacePoints.Reserve(NumSurfacePoints);
				for (int32 ParticleIndex = 0; ParticleIndex < NumSurfacePoints; ++ParticleIndex)
				{
					Convex.SurfacePoints.Add((FVector)ChaosConvex.GetVertex(ParticleIndex));
				}

				// Add extracted collision data
				ClothCollisionData.Convexes.Add(Convex);
			}

			for (const FKLevelSetElem& LevelSetElem : AggGeom.LevelSetElems)
			{
				if (LevelSetElem.GetLevelSet().IsValid())
				{
					ClothCollisionDataExtended.LevelSets.Emplace( LevelSetElem.GetLevelSet(), LevelSetElem.GetTransform(), MappedBoneIndex );
				}
			}

			for (const FKSkinnedLevelSetElem& SkinnedLevelSetElem : AggGeom.SkinnedLevelSetElems)
			{
				if (SkinnedLevelSetElem.WeightedLevelSet().IsValid())
				{
					TArray<int32> MappedSkinnedBones;
					const TArray<FName>& SkinnedBones = SkinnedLevelSetElem.WeightedLevelSet()->GetUsedBones();
					MappedSkinnedBones.Reserve(SkinnedBones.Num());
					for (const FName& SkinnedBoneName : SkinnedBones)
					{
						const int32 SkinnedBoneIndex = ReferenceSkeleton ? ReferenceSkeleton->FindBoneIndex(SkinnedBoneName) : INDEX_NONE;
						const int32 MappedSkinnedBoneIndex = UsedSubBoneIndices.AddUnique(SkinnedBoneIndex);
						MappedSkinnedBones.Add(MappedSkinnedBoneIndex);
					}
					ClothCollisionDataExtended.SkinnedLevelSets.Emplace(SkinnedLevelSetElem.WeightedLevelSet(), MappedBoneIndex, MoveTemp(MappedSkinnedBones));
				}
			}

			for (const FKMLLevelSetElem& MLLevelSetElem : AggGeom.MLLevelSetElems)
			{				
				if (MLLevelSetElem.GetMLLevelSet().IsValid())
				{
					TArray<int32> MappedActiveBones;
					const TArray<FName>& ActiveBoneNames = MLLevelSetElem.GetMLLevelSet()->GetActiveBoneNames();
					MappedActiveBones.Reserve(ActiveBoneNames.Num());
					for (const FName& ActiveBoneName : ActiveBoneNames)
					{
						const int32 ActiveBoneIndex = ReferenceSkeleton ? ReferenceSkeleton->FindBoneIndex(ActiveBoneName) : INDEX_NONE;
						const int32 MappedActiveBoneIndex = UsedSubBoneIndices.AddUnique(ActiveBoneIndex);
						MappedActiveBones.Add(MappedActiveBoneIndex);
					}
					ClothCollisionDataExtended.MLLevelSets.Emplace( MLLevelSetElem.GetMLLevelSet(), MappedBoneIndex, MoveTemp(MappedActiveBones));
				}
			}

			for (const FKSkinnedTriangleMeshElem& SkinnedTriangleMeshElem : AggGeom.SkinnedTriangleMeshElems)
			{
				if (SkinnedTriangleMeshElem.GetSkinnedTriangleMesh().IsValid())
				{
					TArray<int32> MappedSkinnedBones;
					const TArray<FName>& SkinnedBones = SkinnedTriangleMeshElem.GetSkinnedTriangleMesh()->GetUsedBones();
					MappedSkinnedBones.Reserve(SkinnedBones.Num());
					for (const FName& SkinnedBoneName : SkinnedBones)
					{
						const int32 SkinnedBoneIndex = ReferenceSkeleton ? ReferenceSkeleton->FindBoneIndex(SkinnedBoneName) : INDEX_NONE;
						const int32 MappedSkinnedBoneIndex = UsedSubBoneIndices.AddUnique(SkinnedBoneIndex);
						MappedSkinnedBones.Add(MappedSkinnedBoneIndex);
					}
					ClothCollisionDataExtended.SkinnedTriangleMeshes.Emplace(SkinnedTriangleMeshElem.GetSkinnedTriangleMesh(), MappedBoneIndex, MoveTemp(MappedSkinnedBones));
				}
			}
		}  // End for PhysAsset->SkeletalBodySetups
	}  // End if PhysicsAsset
}


int32 FClothingSimulationCollider::GetNumGeometries(int32 InSlotIndex) const
{
	return LODData.IsValidIndex(InSlotIndex) ? LODData[InSlotIndex]->NumGeometries : 0;
}

int32 FClothingSimulationCollider::GetCollisionRangeId(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, int32 InSlotIndex) const
{
	const int32* const CollisionRangeId = LODData.IsValidIndex(InSlotIndex) ? LODData[InSlotIndex]->CollisionRangeIds.Find(FSolverClothPair(Solver, Cloth)) : nullptr;
	return CollisionRangeId ? *CollisionRangeId : INDEX_NONE;
}

int32 FClothingSimulationCollider::GetCollisionRangeId(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	const int32 LODIndex = LODIndices.FindChecked(FSolverClothPair(Solver, Cloth));
	const int32 SlotIndex =
		(CollisionDataType < ECollisionDataType::LODs) ? (int32)CollisionDataType :
		(LODIndex >= (int32)ECollisionDataType::LODs) ? LODIndex : INDEX_NONE;

	return GetCollisionRangeId(Solver, Cloth, SlotIndex);
}

bool FClothingSimulationCollider::GetCollisionRangeIdAndNumGeometries(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType, int32& OutCollisionRangeId, int32& OutNumGeometries) const
{
	OutCollisionRangeId = INDEX_NONE;
	OutNumGeometries = 0;

	const int32 LODIndex = LODIndices.FindChecked(FSolverClothPair(Solver, Cloth));
	const int32 SlotIndex = 
		(CollisionDataType < ECollisionDataType::LODs) ? (int32)CollisionDataType :
		(LODIndex >= (int32)ECollisionDataType::LODs) ? LODIndex : INDEX_NONE;

	if (LODData.IsValidIndex(SlotIndex))
	{
		OutCollisionRangeId = LODData[SlotIndex]->CollisionRangeIds.FindChecked(FSolverClothPair(Solver, Cloth));
		OutNumGeometries = LODData[SlotIndex]->NumGeometries;
	}

	return OutCollisionRangeId != INDEX_NONE && OutNumGeometries > 0;
}

void FClothingSimulationCollider::Add(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	check(Solver);
	check(Cloth);
	check(Cloth->GetMesh());

	// Can't add a collider twice to the same solver/cloth pair
	check(!LODIndices.Find(FSolverClothPair(Solver, Cloth)));

	// Initialize LODIndex
	int32& LODIndex = LODIndices.Add(FSolverClothPair(Solver, Cloth));
	LODIndex = INDEX_NONE;

	// Initialize scale
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FVec3 Scale3D = (FVec3)Cloth->GetMesh()->GetComponentToWorldTransform().GetScale3D();
	UE_CLOGF(FMath::Abs(Scale3D.X - Scale3D.Y) > KINDA_SMALL_NUMBER || FMath::Abs(Scale3D.X - Scale3D.Z) > KINDA_SMALL_NUMBER,
		LogChaosCloth, Warning, "Object '%ls' has a non uniform scale, and has a cloth simulation attached. "
			"The collision volumes might no longer correctly match the shape of the mesh. "
			"Please update this component transform scale with the same value for all scale axis.",
		*Cloth->GetMesh()->GetDebugName());
	Scale = Scale3D.X;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Bake in the authored ref pose scale of the root bone as set by the interchange importer
	const FReal RootBoneScale = ReferenceSkeleton && ReferenceSkeleton->GetRefBonePose().Num() ?
		ReferenceSkeleton->GetRefBonePose()[0].GetScale3D().X : 1.;
	Scale *= RootBoneScale;

	// Create physics asset collisions, this will affect all LODs, so store it at index 0
	FClothCollisionData PhysicsAssetCollisionData;
	FClothCollisionDataExtended PhysicsAssetCollisionDataExtended;
	TArray<int32> UsedBoneIndices;
	TArray<int32> UsedSubBoneIndices;
	ExtractPhysicsAssetCollision(PhysicsAsset, ReferenceSkeleton, PhysicsAssetCollisionData, PhysicsAssetCollisionDataExtended, UsedBoneIndices, UsedSubBoneIndices);

	LODData[(int32)ECollisionDataType::LODless]->Add(Solver, Cloth, PhysicsAssetCollisionData, PhysicsAssetCollisionDataExtended, Scale, UsedBoneIndices, UsedSubBoneIndices);
}

void FClothingSimulationCollider::Remove(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	LODIndices.Remove(FSolverClothPair(Solver, Cloth));

	for (TUniquePtr<FLODData>& LODDatum: LODData)
	{
		LODDatum->Remove(Solver, Cloth);
	}
}

void FClothingSimulationCollider::PreUpdate(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	check(Solver);
	check(Cloth);

	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationCollider_PreUpdate);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothingSimulationColliderUpdate);

	// Add or re-add the external collision particles
	const int32 ExternalCollisionNumGeometries = GetNumGeometries((int32)ECollisionDataType::External);
	const int32 ExternalCollisionCollisionRangeId = GetCollisionRangeId(Solver, Cloth, (int32)ECollisionDataType::External);

	// TODO: Handle external extended data 
	// Will need to handle refreshing constraints that hold pointers to data in ComplexColliders

	LODData[(int32)ECollisionDataType::External]->Add(Solver, Cloth, CollisionData ? *CollisionData : FClothCollisionData(), FClothCollisionDataExtended());
	
	// TODO: Find a better way in case the same number but different collisions are being re-added (hash collision data? Provide user dirty function?)
	bHasExternalCollisionChanged =
		ExternalCollisionNumGeometries != GetNumGeometries((int32)ECollisionDataType::External) ||
		ExternalCollisionCollisionRangeId != GetCollisionRangeId(Solver, Cloth, (int32)ECollisionDataType::External);
}

void FClothingSimulationCollider::Update(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	check(Solver);
	check(Cloth);
	check(Cloth->GetMesh());
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationCollider_Update);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothingSimulationColliderUpdate);

	// Update the collision transforms
	// Note: All the LODs are updated at once, so that the previous transforms are always correct during LOD switching
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FTransform& ComponentTransform = Cloth->GetMesh()->GetComponentToWorldTransform();
	const TArray<FTransform>& BoneTransforms = Cloth->GetMesh()->GetBoneTransforms();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for (TUniquePtr<FLODData>& LODDatum : LODData)
	{
		LODDatum->Update(Solver, Cloth, ComponentTransform, BoneTransforms);
	}

	// Update current LOD index
	int32& LODIndex = LODIndices.FindChecked(FSolverClothPair(Solver, Cloth));
	const int32 PrevLODIndex = LODIndex;
	LODIndex = (int32)ECollisionDataType::LODs + (Cloth ? Cloth->GetLODIndex(Solver) : INDEX_NONE);
	if (!LODData.IsValidIndex(LODIndex))
	{
		LODIndex = (int32)ECollisionDataType::LODs + INDEX_NONE;
	}

	// Enable particle if the external collisions have changed
	if (bHasExternalCollisionChanged)
	{
		// Enable non LOD collisions at first initialization
		LODData[(int32)ECollisionDataType::External]->Enable(Solver, Cloth, true);

		// Update initial state for collisions
		LODData[(int32)ECollisionDataType::External]->ResetStartPose(Solver, Cloth);
	}

	if (LODIndex != PrevLODIndex)
	{
		if (PrevLODIndex == INDEX_NONE)  // First run
		{
			// Enable non LOD collisions at first initialization
			LODData[(int32)ECollisionDataType::LODless]->Enable(Solver, Cloth, true);

			// Update initial state for collisions
			LODData[(int32)ECollisionDataType::LODless]->ResetStartPose(Solver, Cloth);
		}
		else if (PrevLODIndex >= (int32)ECollisionDataType::LODs)
		{
			// Disable previous LOD
			LODData[PrevLODIndex]->Enable(Solver, Cloth, false);
		}
		if (LODIndex >= (int32)ECollisionDataType::LODs)
		{
			// Enable new LOD
			LODData[LODIndex]->Enable(Solver, Cloth, true);

			if (PrevLODIndex == INDEX_NONE)
			{
				// Update initial state for collisions
				LODData[LODIndex]->ResetStartPose(Solver, Cloth);
			}
		}
	}
}

void FClothingSimulationCollider::ResetStartPose(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth)
{
	for (TUniquePtr<FLODData>& LODDatum : LODData)
	{
		LODDatum->ResetStartPose(Solver, Cloth);
	}
}

TConstArrayView<FSolverVec3> FClothingSimulationCollider::GetCollisionTranslations(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	check(Solver);
	check(Cloth);

	const int32 CollisionRangeId = GetCollisionRangeId(Solver, Cloth, CollisionDataType);
	return CollisionRangeId != INDEX_NONE ? Solver->GetCollisionParticleXsView(CollisionRangeId) : TConstArrayView<FSolverVec3>();
}

TConstArrayView<Softs::FSolverRotation3> FClothingSimulationCollider::GetCollisionRotations(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	check(Solver);
	check(Cloth);

	const int32 CollisionRangeId = GetCollisionRangeId(Solver, Cloth, CollisionDataType);
	return CollisionRangeId != INDEX_NONE ? Solver->GetCollisionParticleRsView(CollisionRangeId) : TConstArrayView<Softs::FSolverRotation3>();
}

TConstArrayView<Softs::FSolverRigidTransform3> FClothingSimulationCollider::GetOldCollisionTransforms(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	check(Solver);
	check(Cloth);

	const int32 CollisionRangeId = GetCollisionRangeId(Solver, Cloth, CollisionDataType);
	return CollisionRangeId != INDEX_NONE ? Solver->GetOldCollisionTransformsView(CollisionRangeId) : TConstArrayView<Softs::FSolverRigidTransform3>();
}

TConstArrayView<FImplicitObjectPtr> FClothingSimulationCollider::GetCollisionGeometry(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	check(Solver);
	check(Cloth);

	const int32 CollisionRangeId = GetCollisionRangeId(Solver, Cloth, CollisionDataType);
	return CollisionRangeId != INDEX_NONE ? Solver->GetCollisionGeometryView(CollisionRangeId) : TConstArrayView<FImplicitObjectPtr>();
}

TConstArrayView<bool> FClothingSimulationCollider::GetCollisionStatus(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const
{
	check(Solver);
	check(Cloth);

	const int32 CollisionRangeId = GetCollisionRangeId(Solver, Cloth, CollisionDataType);
	return CollisionRangeId != INDEX_NONE ? Solver->GetCollisionStatusView(CollisionRangeId) : TConstArrayView<bool>();
}

}  // End namespace Chaos
