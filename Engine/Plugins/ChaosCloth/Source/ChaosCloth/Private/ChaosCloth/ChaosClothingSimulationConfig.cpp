// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosClothConfig.h"
#include "Chaos/CollectionPropertyFacade.h"

#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "Chaos/PBDAnimDriveConstraint.h"
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/PBDSelfCollisionSphereConstraints.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/VelocityField.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace Chaos
{
	FClothingSimulationConfig::FClothingSimulationConfig()
	{
	}

	FClothingSimulationConfig::FClothingSimulationConfig(const TArray<TSharedPtr<const FManagedArrayCollection>>& InPropertyCollections)
	{
		constexpr bool bForceDirty = true;
		Initialize(InPropertyCollections, bForceDirty);
	}

	FClothingSimulationConfig::~FClothingSimulationConfig() = default;

	void FClothingSimulationConfig::Initialize(const UChaosClothConfig* ClothConfig, const UChaosClothSharedSimConfig* ClothSharedConfig, bool bUseLegacyConfig)
	{
		using namespace ::Chaos::Softs;
		constexpr ECollectionPropertyFlags NonAnimatablePropertyFlags =
			ECollectionPropertyFlags::Enabled |
			ECollectionPropertyFlags::Legacy;  // Indicates a property set from a pre-property collection config (e.g. that can be overriden in Dataflow without warning)
		constexpr ECollectionPropertyFlags AnimatablePropertyFlags = NonAnimatablePropertyFlags |
			ECollectionPropertyFlags::Animatable;

		// Clear all properties
		PropertyCollections.Reset(1);
		Properties.Reset(1);

		bIsLegacySingleLOD = true;
		TSharedPtr<FManagedArrayCollection>& PropertyCollection = PropertyCollections.Add_GetRef(MakeShared<FManagedArrayCollection>());
		TUniquePtr<Softs::FCollectionPropertyMutableFacade>& Property = Properties.Add_GetRef(MakeUnique<Softs::FCollectionPropertyMutableFacade>(PropertyCollection));
		Property->DefineSchema();

		// Solver properties
		if (ClothSharedConfig)
		{
			Property->AddValue(FClothingSimulationSolver::NumIterationsName, ClothSharedConfig->IterationCount, AnimatablePropertyFlags);
			Property->AddValue(FClothingSimulationSolver::MaxNumIterationsName, ClothSharedConfig->MaxIterationCount, AnimatablePropertyFlags);
			Property->AddValue(FClothingSimulationSolver::NumSubstepsName, ClothSharedConfig->SubdivisionCount, AnimatablePropertyFlags);
		}

		// Cloth properties
		if (ClothConfig)
		{
			// Mass
			{
				float MassValue;
				switch (ClothConfig->MassMode)
				{
				case EClothMassMode::TotalMass:
					MassValue = ClothConfig->TotalMass;
					break;
				case EClothMassMode::UniformMass:
					MassValue = ClothConfig->UniformMass;
					break;
				default:
				case EClothMassMode::Density:
					MassValue = ClothConfig->Density;
					break;
				}
				Property->AddValue(FClothingSimulationCloth::MassModeName, (int32)ClothConfig->MassMode, NonAnimatablePropertyFlags | ECollectionPropertyFlags::Intrinsic);
				Property->AddValue(FClothingSimulationCloth::MassValueName, MassValue, NonAnimatablePropertyFlags | ECollectionPropertyFlags::Intrinsic);
				Property->AddValue(FClothingSimulationCloth::MinPerParticleMassName, ClothConfig->MinPerParticleMass, NonAnimatablePropertyFlags | ECollectionPropertyFlags::Intrinsic);
			}

			// Edge constraint
			if (ClothConfig->EdgeStiffnessWeighted.Low > 0.f || ClothConfig->EdgeStiffnessWeighted.High > 0.f)
			{
				const int32 EdgeSpringStiffnessIndex = Property->AddProperty(Softs::FPBDEdgeSpringConstraints::EdgeSpringStiffnessName, AnimatablePropertyFlags);
				Property->SetWeightedValue(EdgeSpringStiffnessIndex, ClothConfig->EdgeStiffnessWeighted.Low, ClothConfig->EdgeStiffnessWeighted.High);
				Property->SetStringValue(EdgeSpringStiffnessIndex, TEXT("EdgeStiffness"));
			}

			// Bending constraint
			if (ClothConfig->BendingStiffnessWeighted.Low > 0.f || ClothConfig->BendingStiffnessWeighted.High > 0.f ||
				(ClothConfig->bUseBendingElements && (ClothConfig->BucklingStiffnessWeighted.Low > 0.f || ClothConfig->BucklingStiffnessWeighted.High > 0.f)))
			{
				if (ClothConfig->bUseBendingElements)
				{
					const int32 BendingElementStiffnessIndex = Property->AddProperty(Softs::FPBDBendingConstraints::BendingElementStiffnessName, AnimatablePropertyFlags);
					Property->SetWeightedValue(BendingElementStiffnessIndex, ClothConfig->BendingStiffnessWeighted.Low, ClothConfig->BendingStiffnessWeighted.High);
					Property->SetStringValue(BendingElementStiffnessIndex, TEXT("BendingStiffness"));

					Property->AddValue(Softs::FPBDBendingConstraints::BucklingRatioName, ClothConfig->BucklingRatio, AnimatablePropertyFlags);

					const int32 BucklingStiffnessIndex = Property->AddProperty(Softs::FPBDBendingConstraints::BucklingStiffnessName, AnimatablePropertyFlags);
					Property->SetWeightedValue(BucklingStiffnessIndex, ClothConfig->BucklingStiffnessWeighted.Low, ClothConfig->BucklingStiffnessWeighted.High);
					Property->SetStringValue(BucklingStiffnessIndex, TEXT("BucklingStiffness"));

					const int32 RestAngleTypeIndex = Property->AddProperty(Softs::FPBDBendingConstraints::RestAngleTypeName, NonAnimatablePropertyFlags);
					Property->SetValue(RestAngleTypeIndex, (int32)Chaos::Softs::FPBDBendingConstraintsBase::ERestAngleConstructionType::FlatnessRatio);

					const int32 FlatnessRatioIndex = Property->AddProperty(Softs::FPBDBendingConstraints::FlatnessRatioName, NonAnimatablePropertyFlags);
					Property->SetWeightedValue(FlatnessRatioIndex, ClothConfig->FlatnessRatio.Low, ClothConfig->FlatnessRatio.High);
					Property->SetStringValue(FlatnessRatioIndex, TEXT("FlatnessRatio"));
				}
				else  // Not using bending elements
				{
					const int32 BendingSpringStiffnessIndex = Property->AddProperty(Softs::FPBDBendingSpringConstraints::BendingSpringStiffnessName, AnimatablePropertyFlags);
					Property->SetWeightedValue(BendingSpringStiffnessIndex, ClothConfig->BendingStiffnessWeighted.Low, ClothConfig->BendingStiffnessWeighted.High);
					Property->SetStringValue(BendingSpringStiffnessIndex, TEXT("BendingStiffness"));
				}
			}

			// Area constraint
			if (ClothConfig->AreaStiffnessWeighted.Low > 0.f || ClothConfig->AreaStiffnessWeighted.High > 0.f)
			{
				const int32 AreaSpringStiffnessIndex = Property->AddProperty(Softs::FPBDAreaSpringConstraints::AreaSpringStiffnessName, AnimatablePropertyFlags);
				Property->SetWeightedValue(AreaSpringStiffnessIndex, ClothConfig->AreaStiffnessWeighted.Low, ClothConfig->AreaStiffnessWeighted.High);
				Property->SetStringValue(AreaSpringStiffnessIndex, TEXT("AreaStiffness"));
			}

			// Long range attachment
			if (ClothConfig->TetherStiffness.Low > 0.f || ClothConfig->TetherStiffness.High > 0.f)
			{
				Property->AddValue(FClothingSimulationCloth::UseGeodesicTethersName, ClothConfig->bUseGeodesicDistance, NonAnimatablePropertyFlags);

				const int32 TetherStiffnessIndex = Property->AddProperty(Softs::FPBDLongRangeConstraints::TetherStiffnessName, AnimatablePropertyFlags);
				Property->SetWeightedValue(TetherStiffnessIndex, ClothConfig->TetherStiffness.Low, ClothConfig->TetherStiffness.High);
				Property->SetStringValue(TetherStiffnessIndex, TEXT("TetherStiffness"));

				const int32 TetherScaleIndex = Property->AddProperty(Softs::FPBDLongRangeConstraints::TetherScaleName, AnimatablePropertyFlags);
				Property->SetWeightedValue(TetherScaleIndex, ClothConfig->TetherScale.Low, ClothConfig->TetherScale.High);
				Property->SetStringValue(TetherScaleIndex, TEXT("TetherScale"));
			}

			// AnimDrive
			if (ClothConfig->AnimDriveStiffness.Low > 0.f || ClothConfig->AnimDriveStiffness.High > 0.f)
			{
				const int32 AnimDriveStiffnessIndex = Property->AddProperty(Softs::FPBDAnimDriveConstraint::AnimDriveStiffnessName, AnimatablePropertyFlags);
				Property->SetWeightedValue(AnimDriveStiffnessIndex, ClothConfig->AnimDriveStiffness.Low, ClothConfig->AnimDriveStiffness.High);
				Property->SetStringValue(AnimDriveStiffnessIndex, TEXT("AnimDriveStiffness"));

				const int32 AnimDriveDampingIndex = Property->AddProperty(Softs::FPBDAnimDriveConstraint::AnimDriveDampingName, AnimatablePropertyFlags);
				Property->SetWeightedValue(AnimDriveDampingIndex, ClothConfig->AnimDriveDamping.Low, ClothConfig->AnimDriveDamping.High);
				Property->SetStringValue(AnimDriveDampingIndex, TEXT("AnimDriveDamping"));
			}

			// Gravity
			{
				Property->AddValue(FClothingSimulationCloth::GravityScaleName, ClothConfig->GravityScale, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::UseGravityOverrideName, ClothConfig->bUseGravityOverride, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::GravityOverrideName, FVector3f(ClothConfig->Gravity), AnimatablePropertyFlags);
			}

			// Velocity scale
			{
				Property->AddValue(FClothingSimulationCloth::VelocityScaleSpaceName, (int32)ClothConfig->VelocityScaleSpace, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::LinearVelocityScaleName, FVector3f(ClothConfig->LinearVelocityScale), AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::AngularVelocityScaleName, ClothConfig->AngularVelocityScale, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::FictitiousAngularScaleName, ClothConfig->FictitiousAngularScale, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::FictitiousForcesModelName, (int32)ClothConfig->FictitiousForcesModel, AnimatablePropertyFlags);

				static const FVector3f DefaultLinearClamp = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
				Property->AddValue(FClothingSimulationCloth::MaxLinearVelocityName, ClothConfig->bEnableLinearVelocityClamping ? ClothConfig->MaxLinearVelocity : DefaultLinearClamp, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::MaxLinearAccelerationName, ClothConfig->bEnableLinearAccelerationClamping ? ClothConfig->MaxLinearAcceleration : DefaultLinearClamp, AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::MaxAngularVelocityName, ClothConfig->bEnableAngularVelocityClamping ? ClothConfig->MaxAngularVelocity : TNumericLimits<float>::Max(), AnimatablePropertyFlags);
				Property->AddValue(FClothingSimulationCloth::MaxAngularAccelerationName, ClothConfig->bEnableAngularAccelerationClamping ? ClothConfig->MaxAngularAcceleration : TNumericLimits<float>::Max(), AnimatablePropertyFlags);
			}

			// Aerodynamics
			Property->AddValue(FClothingSimulationCloth::UsePointBasedWindModelName, ClothConfig->bUsePointBasedWindModel, NonAnimatablePropertyFlags);
			{
				const int32 DragIndex = Property->AddProperty(Softs::FVelocityAndPressureField::DragName, AnimatablePropertyFlags);
				Property->SetWeightedValue(DragIndex, ClothConfig->Drag.Low, ClothConfig->Drag.High);
				Property->SetStringValue(DragIndex, TEXT("Drag"));

				if (ClothConfig->bEnableOuterDrag)
				{
					const int32 OuterDragIndex = Property->AddProperty(Softs::FVelocityAndPressureField::OuterDragName, AnimatablePropertyFlags);
					Property->SetWeightedValue(OuterDragIndex, ClothConfig->OuterDrag.Low, ClothConfig->OuterDrag.High);
					Property->SetStringValue(OuterDragIndex, TEXT("OuterDrag"));
				}

				const int32 LiftIndex = Property->AddProperty(Softs::FVelocityAndPressureField::LiftName, AnimatablePropertyFlags);
				Property->SetWeightedValue(LiftIndex, ClothConfig->Lift.Low, ClothConfig->Lift.High);
				Property->SetStringValue(LiftIndex, TEXT("Lift"));

				if (ClothConfig->bEnableOuterLift)
				{
					const int32 OuterLiftIndex = Property->AddProperty(Softs::FVelocityAndPressureField::OuterLiftName, AnimatablePropertyFlags);
					Property->SetWeightedValue(OuterLiftIndex, ClothConfig->OuterLift.Low, ClothConfig->OuterLift.High);
					Property->SetStringValue(OuterLiftIndex, TEXT("OuterLift"));
				}

				constexpr float AirDensity = 1.225f;  // Air density in kg/m^3
				Property->AddValue(Softs::FVelocityAndPressureField::FluidDensityName, AirDensity, AnimatablePropertyFlags);

				Property->AddValue(Softs::FVelocityAndPressureField::WindVelocityName, FVector3f(0.f), AnimatablePropertyFlags);  // Wind velocity must exist to be animatable
			}

			// Pressure
			{
				const int32 PressureIndex = Property->AddProperty(Softs::FVelocityAndPressureField::PressureName, AnimatablePropertyFlags);
				Property->SetWeightedValue(PressureIndex, ClothConfig->Pressure.Low, ClothConfig->Pressure.High);
				Property->SetStringValue(PressureIndex, TEXT("Pressure"));
			}

			// Damping
			Property->AddValue(FClothingSimulationCloth::DampingCoefficientName, ClothConfig->DampingCoefficient, AnimatablePropertyFlags);
			Property->AddValue(FClothingSimulationCloth::LocalDampingCoefficientName, ClothConfig->LocalDampingCoefficient, AnimatablePropertyFlags);

			// Collision
			Property->AddValue(FClothingSimulationCloth::CollisionThicknessName, ClothConfig->CollisionThickness, AnimatablePropertyFlags);
			Property->AddValue(FClothingSimulationCloth::FrictionCoefficientName, ClothConfig->FrictionCoefficient, AnimatablePropertyFlags);
			Property->AddValue(FClothingSimulationCloth::UseCCDName, ClothConfig->bUseCCD, AnimatablePropertyFlags);
			Property->AddValue(Softs::FPBDCollisionSpringConstraints::UseSelfCollisionsName, ClothConfig->bUseSelfCollisions, NonAnimatablePropertyFlags);
			Property->AddValue(Softs::FPBDCollisionSpringConstraints::SelfCollisionThicknessName, ClothConfig->SelfCollisionThickness, NonAnimatablePropertyFlags);
			Property->AddValue(Softs::FPBDTriangleMeshCollisions::UseSelfIntersectionsName, ClothConfig->bUseSelfIntersections, NonAnimatablePropertyFlags);
			Property->AddValue(Softs::FPBDCollisionSpringConstraints::SelfCollisionFrictionName, ClothConfig->SelfCollisionFriction, NonAnimatablePropertyFlags);

			if (ClothConfig->bUseSelfCollisionSpheres)
			{
				Property->AddValue(Softs::FPBDSelfCollisionSphereConstraints::SelfCollisionSphereRadiusName, ClothConfig->SelfCollisionSphereRadius, NonAnimatablePropertyFlags);
				Property->AddValue(Softs::FPBDSelfCollisionSphereConstraints::SelfCollisionSphereStiffnessName, ClothConfig->SelfCollisionSphereStiffness, AnimatablePropertyFlags);
				Property->AddStringValue(Softs::FPBDSelfCollisionSphereConstraints::SelfCollisionSphereSetNameName, Softs::FPBDSelfCollisionSphereConstraints::SelfCollisionSphereSetNameName.ToString(), NonAnimatablePropertyFlags);
			}

			// Max distance
			{
				const int32 MaxDistanceIndex = Property->AddProperty(Softs::FPBDSphericalConstraint::MaxDistanceName, AnimatablePropertyFlags);
				Property->SetWeightedValue(MaxDistanceIndex, 0.f, 1.f);  // Backward compatibility with legacy mask must use a unit range since the multiplier is in the mask
				Property->SetStringValue(MaxDistanceIndex, TEXT("MaxDistance"));
			}

			// Backstop
			{
				const int32 BackstopDistanceIndex = Property->AddProperty(Softs::FPBDSphericalBackstopConstraint::BackstopDistanceName, AnimatablePropertyFlags);
				Property->SetWeightedValue(BackstopDistanceIndex, 0.f, 1.f);  // Backward compatibility with legacy mask must use a unit range since the multiplier is in the mask
				Property->SetStringValue(BackstopDistanceIndex, TEXT("BackstopDistance"));

				const int32 BackstopRadiusIndex = Property->AddProperty(Softs::FPBDSphericalBackstopConstraint::BackstopRadiusName, AnimatablePropertyFlags);
				Property->SetWeightedValue(BackstopRadiusIndex, 0.f, 1.f);  // Backward compatibility with legacy mask must use a unit range since the multiplier is in the mask
				Property->SetStringValue(BackstopRadiusIndex, TEXT("BackstopRadius"));

				Property->AddValue(Softs::FPBDSphericalBackstopConstraint::UseLegacyBackstopName, ClothConfig->bUseLegacyBackstop, NonAnimatablePropertyFlags);
			}
		}

		// Mark this as a potential legacy config, but leave the behavior control to the client code (usually means constraint are removed with 0 stiffness, or missing weight maps)
		Property->AddValue(UseLegacyConfigName, bUseLegacyConfig, NonAnimatablePropertyFlags);
	}

	void FClothingSimulationConfig::Initialize(const TArray<TSharedPtr<const FManagedArrayCollection>>& InPropertyCollections, bool bForceDirty, int32 LODIndex)
	{
		if (LODIndex != INDEX_NONE && !InPropertyCollections.IsValidIndex(LODIndex))
		{
			return;
		}
		PropertyCollections.SetNum(InPropertyCollections.Num());
		Properties.SetNum(InPropertyCollections.Num());

		const int32 StartIndex = (LODIndex != INDEX_NONE) ? LODIndex : 0;
		const int32 EndIndex = (LODIndex != INDEX_NONE) ? LODIndex : InPropertyCollections.Num() - 1;

		for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
		{
			const TSharedPtr<const FManagedArrayCollection>& InPropertyCollection = InPropertyCollections[Index];
			::Chaos::Softs::FCollectionPropertyConstFacade InPropertyFacade(InPropertyCollection);
			if (bForceDirty || InPropertyFacade.IsDirty())
			{
				TSharedPtr<FManagedArrayCollection>& PropertyCollection = PropertyCollections[Index];
				TUniquePtr<Softs::FCollectionPropertyMutableFacade>& Property = Properties[Index];
				if (!PropertyCollection || !Property)
				{
					PropertyCollection = MakeShared<FManagedArrayCollection>();
					Property = MakeUnique<Softs::FCollectionPropertyMutableFacade>(PropertyCollection);
				}
				Property->Copy(*InPropertyCollection);
			}
		}
	}

	const Softs::FCollectionPropertyConstFacade& FClothingSimulationConfig::GetProperties(int32 LODIndex) const
	{
		return bIsLegacySingleLOD ? *Properties[0] : *Properties[LODIndex];
	}

	Softs::FCollectionPropertyFacade& FClothingSimulationConfig::GetProperties(int32 LODIndex)
	{
		return bIsLegacySingleLOD ? *Properties[0] : *Properties[LODIndex];
	}
}  // End namespace Chaos
