// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationDefaultAccessoryMesh.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"

namespace Chaos
{
	const FName FClothingSimulationDefaultAccessoryMesh::DefaultAccessoryMeshName(TEXT("DefaultAccessoryMesh"));

	FClothingSimulationDefaultAccessoryMesh::FClothingSimulationDefaultAccessoryMesh(const FClothingSimulationMesh& InMesh, const int32 InLODIndex)
		: FClothingSimulationAccessoryMesh(InMesh, DefaultAccessoryMeshName)
		, LODIndex(InLODIndex)
	{
	}

	FClothingSimulationDefaultAccessoryMesh::~FClothingSimulationDefaultAccessoryMesh() = default;

	int32 FClothingSimulationDefaultAccessoryMesh::GetNumPoints() const
	{
		return Mesh.GetNumPoints(LODIndex);
	}

	int32 FClothingSimulationDefaultAccessoryMesh::GetNumPatternPoints() const
	{
		return Mesh.GetNumPatternPoints(LODIndex);
	}

	TConstArrayView<FVector3f> FClothingSimulationDefaultAccessoryMesh::GetPositions() const
	{
		return Mesh.GetPositions(LODIndex);
	}

	TConstArrayView<FVector2f> FClothingSimulationDefaultAccessoryMesh::GetPatternPositions() const
	{
		return Mesh.GetPatternPositions(LODIndex);
	}

	TConstArrayView<FVector3f> FClothingSimulationDefaultAccessoryMesh::GetNormals() const
	{
		return Mesh.GetNormals(LODIndex);
	}

	TConstArrayView<FClothVertBoneData> FClothingSimulationDefaultAccessoryMesh::GetBoneData() const
	{
		return Mesh.GetBoneData(LODIndex);
	}

	int32 FClothingSimulationDefaultAccessoryMesh::FindMorphTargetByName(const FString& Name) const
	{
		return Mesh.FindMorphTargetByName(LODIndex, Name);
	}

	TConstArrayView<FString> FClothingSimulationDefaultAccessoryMesh::GetAllMorphTargetNames() const
	{
		return Mesh.GetAllMorphTargetNames(LODIndex);
	}

	TConstArrayView<FVector3f> FClothingSimulationDefaultAccessoryMesh::GetMorphTargetPositionDeltas(int32 MorphTargetIndex) const
	{
		return Mesh.GetMorphTargetPositionDeltas(LODIndex, MorphTargetIndex);
	}

	TConstArrayView<FVector3f> FClothingSimulationDefaultAccessoryMesh::GetMorphTargetTangentZDeltas(int32 MorphTargetIndex) const
	{
		return Mesh.GetMorphTargetTangentZDeltas(LODIndex, MorphTargetIndex);
	}

	TConstArrayView<int32> FClothingSimulationDefaultAccessoryMesh::GetMorphTargetIndices(int32 MorphTargetIndex) const
	{
		return Mesh.GetMorphTargetIndices(LODIndex, MorphTargetIndex);
	}
}  // End namespace Chaos
