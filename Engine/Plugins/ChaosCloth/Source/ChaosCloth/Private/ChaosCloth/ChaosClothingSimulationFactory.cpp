// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothingSimulationFactory.h"
#include "ChaosCloth/ChaosClothConfig.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosWeightMapTarget.h"
#include "ChaosCloth/ChaosClothingSimulationInteractor.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosClothingSimulationFactory)

IClothingSimulationInterface* UChaosClothingSimulationFactory::CreateSimulation() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	return new Chaos::FClothingSimulation();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationFactory::DestroySimulation(IClothingSimulationInterface* InSimulation) const
{
	delete InSimulation;
}

bool UChaosClothingSimulationFactory::SupportsAsset(const UClothingAssetBase* InAsset) const
{
	return Cast<UClothingAssetCommon>(InAsset) != nullptr;
}

bool UChaosClothingSimulationFactory::SupportsRuntimeInteraction() const
{
	return true;
}

UClothingSimulationInteractor* UChaosClothingSimulationFactory::CreateInteractor()
{
	return NewObject<UChaosClothingSimulationInteractor>(GetTransientPackage());
}

TArrayView<const TSubclassOf<UClothConfigBase>> UChaosClothingSimulationFactory::GetClothConfigClasses() const
{
	static const TArray<TSubclassOf<UClothConfigBase>> ClothConfigClasses(
		{
			TSubclassOf<UClothConfigBase>(UChaosClothConfig::StaticClass()),
			TSubclassOf<UClothConfigBase>(UChaosClothSharedSimConfig::StaticClass())
		});
	return ClothConfigClasses;
}

const UEnum* UChaosClothingSimulationFactory::GetWeightMapTargetEnum() const
{
	return StaticEnum<EChaosWeightMapTarget>();
}

