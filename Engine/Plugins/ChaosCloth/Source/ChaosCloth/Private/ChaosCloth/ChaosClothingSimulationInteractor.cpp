// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/ChaosClothingSimulationInteractor.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"

#include "Chaos/PBDAnimDriveConstraint.h"
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/PBDSelfCollisionSphereConstraints.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/VelocityField.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosClothingSimulationInteractor)

namespace ChaosClothingInteractor
{
	static const float InvStiffnessLogBase = 1.f / FMath::Loge(1.e3f);  // Log base for updating old linear stiffnesses to the new stiffness exponentiation
}

void UChaosClothingInteractor::Sync(IClothingSimulationInterface* Simulation)
{
	check(Simulation);

	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosClothingInteractor_Sync);
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	if (Chaos::FClothingSimulationCloth* const Cloth = static_cast<Chaos::FClothingSimulation*>(Simulation)->GetCloth(ClothingId))
	{
		for (Chaos::Internal::FClothingInteractorCommand& Command : Commands)
		{
			Command.Execute(Cloth);
		}
		Commands.Reset();
		for (Chaos::Internal::FClothingInteractorConfigCommand& ConfigCommand : ConfigCommands)
		{
			check(Cloth->GetConfig()->IsLegacySingleLOD());
			ConfigCommand.Execute(Cloth->GetConfig(), 0);
		}
		ConfigCommands.Reset();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Call to base class' sync
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	UClothingInteractor::Sync(static_cast<IClothingSimulation*>(Simulation));  // TODO: 5.9 remove the casts to pass the correct interface
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingInteractor::SetMaterialLinear(float EdgeStiffnessLinear, float BendingStiffnessLinear, float AreaStiffnessLinear)
{
	const FVector2f EdgeStiffness((FMath::Clamp(FMath::Loge(EdgeStiffnessLinear) * ChaosClothingInteractor::InvStiffnessLogBase + 1.f, 0.f, 1.f)), 1.f);
	const FVector2f BendingStiffness((FMath::Clamp(FMath::Loge(BendingStiffnessLinear) * ChaosClothingInteractor::InvStiffnessLogBase + 1.f, 0.f, 1.f)), 1.f);
	const FVector2f AreaStiffness((FMath::Clamp(FMath::Loge(AreaStiffnessLinear) * ChaosClothingInteractor::InvStiffnessLogBase + 1.f, 0.f, 1.f)), 1.f);

	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([EdgeStiffness, BendingStiffness, AreaStiffness](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDEdgeSpringConstraints::EdgeSpringStiffnessName, EdgeStiffness);
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDBendingSpringConstraints::BendingSpringStiffnessName, BendingStiffness);
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAreaSpringConstraints::AreaSpringStiffnessName, AreaStiffness);
	}));
}

void UChaosClothingInteractor::SetMaterial(FVector2D EdgeStiffness, FVector2D BendingStiffness, FVector2D AreaStiffness)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([EdgeStiffness, BendingStiffness, AreaStiffness](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDEdgeSpringConstraints::EdgeSpringStiffnessName, FVector2f(EdgeStiffness));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDBendingSpringConstraints::BendingSpringStiffnessName, FVector2f(BendingStiffness));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAreaSpringConstraints::AreaSpringStiffnessName, FVector2f(AreaStiffness));
	}));
}

void UChaosClothingInteractor::SetMaterialBuckling(FVector2D BucklingRatio, FVector2D BucklingStiffness)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([BucklingRatio, BucklingStiffness](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FPBDBendingConstraints::BucklingRatioName, (float)BucklingRatio[0]);  // TODO: Make BuckingRatio weighted
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDBendingConstraints::BucklingStiffnessName, FVector2f(BucklingStiffness));
	}));
}

void UChaosClothingInteractor::SetLongRangeAttachmentLinear(float TetherStiffnessLinear, float TetherScale)
{
	// Deprecated
	const FVector2f TetherStiffness((FMath::Clamp(FMath::Loge(TetherStiffnessLinear) * ChaosClothingInteractor::InvStiffnessLogBase + 1.f, 0.f, 1.f)), 1.f);
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([TetherStiffness, TetherScale](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDLongRangeConstraints::TetherStiffnessName, TetherStiffness);
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDLongRangeConstraints::TetherScaleName, FVector2f(TetherScale));
	}));
}

void UChaosClothingInteractor::SetLongRangeAttachment(FVector2D TetherStiffness, FVector2D TetherScale)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([TetherStiffness, TetherScale](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDLongRangeConstraints::TetherStiffnessName, FVector2f(TetherStiffness));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDLongRangeConstraints::TetherScaleName, FVector2f(TetherScale));
	}));
}

void UChaosClothingInteractor::SetCollision(float CollisionThickness, float FrictionCoefficient, bool bUseCCD, float SelfCollisionThickness)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([CollisionThickness, FrictionCoefficient, bUseCCD, SelfCollisionThickness](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::CollisionThicknessName, CollisionThickness);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::FrictionCoefficientName, FrictionCoefficient);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::UseCCDName, bUseCCD);
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FPBDCollisionSpringConstraints::SelfCollisionThicknessName, SelfCollisionThickness);
	}));
}

void UChaosClothingInteractor::SetBackstop(bool bEnabled)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([bEnabled](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetEnabled(Chaos::Softs::FPBDSphericalBackstopConstraint::BackstopRadiusName, bEnabled);  // BackstopRadius controls whether the backstop is enabled or not
	}));
}
void UChaosClothingInteractor::SetDamping(float DampingCoefficient, float LocalDampingCoefficient)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([DampingCoefficient, LocalDampingCoefficient](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::DampingCoefficientName, DampingCoefficient);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::LocalDampingCoefficientName, LocalDampingCoefficient);
	}));
}

void UChaosClothingInteractor::SetAerodynamics(float DragCoefficient, float LiftCoefficient, FVector WindVelocity)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([DragCoefficient, LiftCoefficient, WindVelocity](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		constexpr float AirDensity = 1.225f;
		constexpr float WorldScale = 100.f;  // Unreal's world unit is the cm
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::DragName, FVector2f(DragCoefficient));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::LiftName, FVector2f(LiftCoefficient));
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FVelocityAndPressureField::FluidDensityName, AirDensity);
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FVelocityAndPressureField::WindVelocityName, FVector3f(WindVelocity) / WorldScale);
	}));
}

void UChaosClothingInteractor::SetWind(FVector2D Drag, FVector2D Lift, float AirDensity, FVector WindVelocity, FVector2D OuterDrag, FVector2D OuterLift)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([Drag, Lift, AirDensity, WindVelocity, OuterDrag, OuterLift](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		constexpr float WorldScale = 100.f;  // Unreal's world unit is the cm
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::DragName, FVector2f(Drag));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::OuterDragName, FVector2f(OuterDrag));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::LiftName, FVector2f(Lift));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::OuterLiftName, FVector2f(OuterLift));
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FVelocityAndPressureField::FluidDensityName, (float)AirDensity * FMath::Cube(WorldScale));  // AirDensity is here in kg/cm^3 for legacy reason but must be in kg/m^3 in the config UI
		Config->GetProperties(LODIndex).SetValue(Chaos::Softs::FVelocityAndPressureField::WindVelocityName, FVector3f(WindVelocity) / WorldScale);
	}));
}

void UChaosClothingInteractor::SetPressure(FVector2D Pressure)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([Pressure](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FVelocityAndPressureField::PressureName, FVector2f(Pressure));
	}));
}

void UChaosClothingInteractor::SetGravity(float GravityScale, bool bIsGravityOverridden, FVector GravityOverride)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([GravityScale, bIsGravityOverridden, GravityOverride](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::GravityScaleName, GravityScale);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::UseGravityOverrideName, bIsGravityOverridden);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::GravityOverrideName, FVector3f(GravityOverride));
	}));
}

void UChaosClothingInteractor::SetAnimDriveLinear(float AnimDriveStiffnessLinear)
{
	// Deprecated
	const FVector2f AnimDriveStiffness(0.f, FMath::Clamp(FMath::Loge(AnimDriveStiffnessLinear) * ChaosClothingInteractor::InvStiffnessLogBase + 1.f, 0.f, 1.f));
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([AnimDriveStiffness](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		// The Anim Drive stiffness Low value needs to be 0 in order to keep backward compatibility with existing mask (this wouldn't be an issue if this property had no legacy mask)
		static const FVector2f AnimDriveDamping(0.f, 1.f);
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAnimDriveConstraint::AnimDriveStiffnessName, AnimDriveStiffness);
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAnimDriveConstraint::AnimDriveDampingName, AnimDriveDamping);
	}));
}

void UChaosClothingInteractor::SetAnimDrive(FVector2D AnimDriveStiffness, FVector2D AnimDriveDamping)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([AnimDriveStiffness, AnimDriveDamping](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAnimDriveConstraint::AnimDriveStiffnessName, FVector2f(AnimDriveStiffness));
		Config->GetProperties(LODIndex).SetWeightedFloatValue(Chaos::Softs::FPBDAnimDriveConstraint::AnimDriveDampingName, FVector2f(AnimDriveDamping));
	}));
}

void UChaosClothingInteractor::SetVelocityScale(FVector LinearVelocityScale, float AngularVelocityScale, float FictitiousAngularScale)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([LinearVelocityScale, AngularVelocityScale, FictitiousAngularScale](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::LinearVelocityScaleName, FVector3f(LinearVelocityScale));
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::AngularVelocityScaleName, AngularVelocityScale);
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::FictitiousAngularScaleName, FictitiousAngularScale);
	}));
}

void UChaosClothingInteractor::SetVelocityClamps(bool bEnableLinearVelocityClamping, FVector MaxLinearVelocity, bool bEnableLinearAccelerationClamping, FVector MaxLinearAcceleration,
	bool bEnableAngularVelocityClamping, float MaxAngularVelocity, bool bEnableAngularAccelerationClamping, float MaxAngularAcceleration)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([bEnableLinearVelocityClamping, MaxLinearVelocity, bEnableLinearAccelerationClamping, MaxLinearAcceleration, bEnableAngularVelocityClamping, MaxAngularVelocity, bEnableAngularAccelerationClamping, MaxAngularAcceleration](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
		{
			static const FVector3f DefaultLinearClamp = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
			Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::MaxLinearVelocityName, bEnableLinearVelocityClamping ? FVector3f(MaxLinearVelocity) : DefaultLinearClamp);
			Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::MaxLinearAccelerationName, bEnableLinearAccelerationClamping ? FVector3f(MaxLinearAcceleration) : DefaultLinearClamp);
			Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::MaxAngularVelocityName, bEnableAngularVelocityClamping ? MaxAngularVelocity : TNumericLimits<float>::Max());
			Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationCloth::MaxAngularAccelerationName, bEnableAngularAccelerationClamping ? MaxAngularAcceleration : TNumericLimits<float>::Max());
		}));
}

void UChaosClothingInteractor::ResetAndTeleport(bool bReset, bool bTeleport)
{
	if (bReset)
	{
		Commands.Add(Chaos::Internal::FClothingInteractorCommand::CreateLambda([](Chaos::FClothingSimulationCloth* Cloth)
		{
			Cloth->Reset();
		}));
	}
	if (bTeleport)
	{
		Commands.Add(Chaos::Internal::FClothingInteractorCommand::CreateLambda([](Chaos::FClothingSimulationCloth* Cloth)
		{
			Cloth->Teleport();
		}));
	}
}

void UChaosClothingSimulationInteractor::Sync(IClothingSimulationInterface* Simulation, const IClothingSimulationContext* Context)
{
	check(Simulation);
	check(Context);

	for (Chaos::Internal::FClothingSimulationInteractorCommand& Command : Commands)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
		Command.Execute(static_cast<Chaos::FClothingSimulation*>(Simulation), static_cast<const Chaos::FClothingSimulationContext*>(Context));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	Commands.Reset();
	for (Chaos::Internal::FClothingInteractorConfigCommand& ConfigCommand : ConfigCommands)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
		ensure(static_cast<Chaos::FClothingSimulation*>(Simulation)->GetSolver()->GetConfig()->IsLegacySingleLOD());
		ConfigCommand.Execute(static_cast<Chaos::FClothingSimulation*>(Simulation)->GetSolver()->GetConfig(), 0);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	ConfigCommands.Reset();

	// Call base class' sync 
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	UClothingSimulationInteractor::Sync(static_cast<IClothingSimulation*>(Simulation), const_cast<IClothingSimulationContext*>(Context));  // TODO: 5.9 remove the casts to pass the correct interface
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationInteractor::PhysicsAssetUpdated()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	Commands.Add(Chaos::Internal::FClothingSimulationInteractorCommand::CreateLambda([](Chaos::FClothingSimulation* Simulation, const Chaos::FClothingSimulationContext* /*Context*/)
	{
		Simulation->RefreshPhysicsAsset();
	}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationInteractor::ClothConfigUpdated()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	Commands.Add(Chaos::Internal::FClothingSimulationInteractorCommand::CreateLambda([](Chaos::FClothingSimulation* Simulation, const Chaos::FClothingSimulationContext* Context)
	{
		Simulation->RefreshClothConfig(Context);
	}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationInteractor::SetAnimDriveSpringStiffness(float Stiffness)
{
	// Set the anim drive stiffness through the ChaosClothInteractor to allow the value to be overridden by the cloth interactor if needed
	for (const auto& ClothingInteractor : UClothingSimulationInteractor::ClothingInteractors)
	{
		if (UChaosClothingInteractor* const ChaosClothingInteractor = Cast<UChaosClothingInteractor>(ClothingInteractor.Value))
		{
			ChaosClothingInteractor->SetAnimDriveLinear(Stiffness);
		}
	}
}

void UChaosClothingSimulationInteractor::EnableGravityOverride(const FVector& Gravity)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	Commands.Add(Chaos::Internal::FClothingSimulationInteractorCommand::CreateLambda([Gravity](Chaos::FClothingSimulation* Simulation, const Chaos::FClothingSimulationContext* /*Context*/)
	{
		Simulation->SetGravityOverride(Gravity);
	}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationInteractor::DisableGravityOverride()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	Commands.Add(Chaos::Internal::FClothingSimulationInteractorCommand::CreateLambda([](Chaos::FClothingSimulation* Simulation, const Chaos::FClothingSimulationContext* /*Context*/)
	{
		Simulation->DisableGravityOverride();
	}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UChaosClothingSimulationInteractor::SetNumIterations(int32 NumIterations)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([NumIterations](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationSolver::NumIterationsName, NumIterations);
	}));
}

void UChaosClothingSimulationInteractor::SetMaxNumIterations(int32 MaxNumIterations)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([MaxNumIterations](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationSolver::MaxNumIterationsName, MaxNumIterations);
	}));
}

void UChaosClothingSimulationInteractor::SetNumSubsteps(int32 NumSubsteps)
{
	ConfigCommands.Add(Chaos::Internal::FClothingInteractorConfigCommand::CreateLambda([NumSubsteps](Chaos::FClothingSimulationConfig* Config, int32 LODIndex)
	{
		Config->GetProperties(LODIndex).SetValue(Chaos::FClothingSimulationSolver::NumSubstepsName, NumSubsteps);
	}));
}

UClothingInteractor* UChaosClothingSimulationInteractor::CreateClothingInteractor()
{
	return NewObject<UChaosClothingInteractor>(this);
}

