// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosWeightMapTarget.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "ClothingSimulation.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/ArrayView.h"
#include "Async/ParallelFor.h"
#include "SkeletalMeshTypes.h"  // For FMeshToMeshVertData


DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Wrap Deform Mesh"), STAT_ChaosClothWrapDeformMesh, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Wrap Deform Cloth LOD"), STAT_ChaosClothWrapDeformClothLOD, STATGROUP_ChaosCloth);

namespace Chaos
{

FClothingSimulationMesh::FClothingSimulationMesh(const FString& InDebugName)
#if !UE_BUILD_SHIPPING
	: DebugName(InDebugName)
#endif
{
}

FClothingSimulationMesh::~FClothingSimulationMesh() = default;

Softs::FSolverReal FClothingSimulationMesh::GetScale() const
{
	return GetComponentToWorldTransform().GetScale3D().GetMax();
}

bool FClothingSimulationMesh::WrapDeformLOD(
	int32 PrevLODIndex,
	int32 LODIndex,
	const FSolverVec3* Normals,
	const FSolverVec3* Positions,
	FSolverVec3* OutPositions) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_WrapDeformLOD);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothWrapDeformMesh);

	const int32 NumLODsPassed = FMath::Abs(LODIndex - PrevLODIndex);
	if (NumLODsPassed != 1 || !IsValidLODIndex(PrevLODIndex) || !IsValidLODIndex(LODIndex))
	{
		return false;
	}

	const int32 NumPoints = GetNumPoints(LODIndex);
	const TConstArrayView<FMeshToMeshVertData> SkinData = (PrevLODIndex < LODIndex) ?
		GetTransitionUpSkinData(LODIndex) :
		GetTransitionDownSkinData(LODIndex);

	if (SkinData.Num() != NumPoints)
	{
		return false;
	}

	for (int32 Index = 0; Index < NumPoints; ++Index)  // TODO: Profile for parallel for
	{
		const FMeshToMeshVertData& VertData = SkinData[Index];

		const int32 VertIndex0 = (int32)VertData.SourceMeshVertIndices[0];  // Note: The source is uint16. Watch out for large mesh sections!
		const int32 VertIndex1 = (int32)VertData.SourceMeshVertIndices[1];
		const int32 VertIndex2 = (int32)VertData.SourceMeshVertIndices[2];

		OutPositions[Index] = 
			Positions[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.X + Normals[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			Positions[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Y + Normals[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			Positions[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Z + Normals[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W;
	}

	return true;
}

bool FClothingSimulationMesh::WrapDeformLOD(
	int32 PrevLODIndex,
	int32 LODIndex,
	const Softs::FSolverVec3* Normals,
	const Softs::FPAndInvM* PositionAndInvMs,
	const Softs::FSolverVec3* Velocities,
	Softs::FPAndInvM* OutPositionAndInvMs0,
	Softs::FSolverVec3* OutPositions1,
	Softs::FSolverVec3* OutVelocities) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_WrapDeformLOD);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothWrapDeformClothLOD);

	const int32 NumLODsPassed = FMath::Abs(LODIndex - PrevLODIndex);
	if (NumLODsPassed != 1 || !IsValidLODIndex(PrevLODIndex) || !IsValidLODIndex(LODIndex))
	{
		return false;
	}

	const int32 NumPoints = GetNumPoints(LODIndex);
	const TConstArrayView<FMeshToMeshVertData> SkinData = (PrevLODIndex < LODIndex) ?
		GetTransitionUpSkinData(LODIndex) :
		GetTransitionDownSkinData(LODIndex);

	if (SkinData.Num() != NumPoints)
	{
		return false;
	}

	for (int32 Index = 0; Index < NumPoints; ++Index)  // TODO: Profile for parallel for
	{
		const FMeshToMeshVertData& VertData = SkinData[Index];

		const int32 VertIndex0 = (int32)VertData.SourceMeshVertIndices[0];  // Note: The source is uint16. Watch out for large mesh sections!
		const int32 VertIndex1 = (int32)VertData.SourceMeshVertIndices[1];
		const int32 VertIndex2 = (int32)VertData.SourceMeshVertIndices[2];

		OutPositionAndInvMs0[Index].P = OutPositions1[Index] =
			PositionAndInvMs[VertIndex0].P * (FSolverReal)VertData.PositionBaryCoordsAndDist.X + Normals[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			PositionAndInvMs[VertIndex1].P * (FSolverReal)VertData.PositionBaryCoordsAndDist.Y + Normals[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			PositionAndInvMs[VertIndex2].P * (FSolverReal)VertData.PositionBaryCoordsAndDist.Z + Normals[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W;

		OutVelocities[Index] = 
			Velocities[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.X +
			Velocities[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Y +
			Velocities[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Z;
	}

	return true;
}

bool FClothingSimulationMesh::WrapDeformLOD(
	int32 PrevLODIndex,
	int32 LODIndex,
	const TConstArrayView<Softs::FSolverVec3>& Positions,
	const TConstArrayView<Softs::FSolverVec3>& Normals,
	TArrayView<Softs::FSolverVec3>& OutPositions,
	TArrayView<Softs::FSolverVec3>& OutNormals) const
{

	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationMesh_WrapDeformLOD);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothWrapDeformMesh);

	const int32 NumLODsPassed = FMath::Abs(LODIndex - PrevLODIndex);
	if (NumLODsPassed != 1 || !IsValidLODIndex(PrevLODIndex) || !IsValidLODIndex(LODIndex))
	{
		return false;
	}

	const int32 NumPoints = OutPositions.Num();

	const TConstArrayView<FMeshToMeshVertData> SkinData = (PrevLODIndex < LODIndex) ?
		GetTransitionUpSkinData(LODIndex) :
		GetTransitionDownSkinData(LODIndex);

	if (SkinData.Num() != NumPoints)
	{
		return false;
	}

	for (int32 Index = 0; Index < NumPoints; ++Index)  // TODO: Profile for parallel for
	{
		const FMeshToMeshVertData& VertData = SkinData[Index];

		const int32 VertIndex0 = (int32)VertData.SourceMeshVertIndices[0];  // Note: The source is uint16. Watch out for large mesh sections!
		const int32 VertIndex1 = (int32)VertData.SourceMeshVertIndices[1];
		const int32 VertIndex2 = (int32)VertData.SourceMeshVertIndices[2];

		OutPositions[Index] =
			Positions[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.X + Normals[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			Positions[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Y + Normals[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W +
			Positions[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Z + Normals[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.W;

		OutNormals[Index] =
			(Normals[VertIndex0] * (FSolverReal)VertData.PositionBaryCoordsAndDist.X +
			Normals[VertIndex1] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Y +
			Normals[VertIndex2] * (FSolverReal)VertData.PositionBaryCoordsAndDist.Z).GetSafeNormal();
	}

	return true;
}

void FClothingSimulationMesh::ApplyMorphTarget(
	int32 LODIndex,
	int32 ActiveMorphTargetIndex,
	float ActiveMorphTargetWeight,
	const TArrayView<Softs::FSolverVec3>& InOutPositions,
	const TArrayView<Softs::FSolverVec3>& InOutNormals) const
{
	// Exit if any inputs are missing or not ready, and if the LOD is invalid
	if (!IsValidLODIndex(LODIndex))
	{
		return;
	}

	const FClothingSimulationDefaultAccessoryMesh DefaultMesh = GetDefaultAccessoryMesh(LODIndex);
	DefaultMesh.ApplyMorphTarget(ActiveMorphTargetIndex, ActiveMorphTargetWeight, InOutPositions, InOutNormals);
}


void FClothingSimulationMesh::Update(
	FClothingSimulationSolver* Solver,
	int32 PrevLODIndex,
	int32 LODIndex,
	int32 PrevParticleRangeId,
	int32 ParticleRangeId,
	int32 ActiveMorphTargetIndex,
	float ActiveMorphTargetWeight)
{
	check(Solver);

	// Exit if any inputs are missing or not ready, and if the LOD is invalid
	if (!IsValidLODIndex(LODIndex) || GetNumPoints(LODIndex) == 0)
	{
		return;
	}
	
	// Skin current LOD positions
	TArrayView<FSolverVec3> OutPositions = Solver->GetAnimationPositionsView(ParticleRangeId);
	TArrayView<FSolverVec3> OutNormals = Solver->GetAnimationNormalsView(ParticleRangeId);
	const FClothingSimulationDefaultAccessoryMesh DefaultMesh = GetDefaultAccessoryMesh(LODIndex);
	DefaultMesh.Update(Solver, OutPositions, OutNormals, ActiveMorphTargetIndex, ActiveMorphTargetWeight);

	// Update old positions after LOD Switching
	if (LODIndex != PrevLODIndex)
	{
		bool bValidWrap = false;
		TArrayView<FSolverVec3> OutOldPositions = Solver->GetOldAnimationPositionsView(ParticleRangeId);
		TArrayView<FSolverVec3> OutOldNormals = Solver->GetOldAnimationNormalsView(ParticleRangeId);
		if (IsValidLODIndex(PrevLODIndex) && GetNumPoints(PrevLODIndex) > 0)
		{
			// TODO: Using the more accurate skinning method here would require double buffering the context at the skeletal mesh level
			const TConstArrayView<FSolverVec3> SrcWrapPositions = Solver->GetOldAnimationPositionsView(PrevParticleRangeId);
			const TConstArrayView<FSolverVec3> SrcWrapNormals = Solver->GetOldAnimationNormalsView(PrevParticleRangeId);

			bValidWrap = WrapDeformLOD(PrevLODIndex, LODIndex, SrcWrapPositions, SrcWrapNormals, OutOldPositions, OutOldNormals);
		}
		if (!bValidWrap)
		{
			// The previous LOD is invalid, reset old positions with the new LOD
			for (int32 Index = 0; Index < OutOldPositions.Num(); ++Index)
			{
				OutOldPositions[Index] = OutPositions[Index];
				OutOldNormals[Index] = OutNormals[Index];
			}
		}
	}
}

}  // End namespace Chaos
