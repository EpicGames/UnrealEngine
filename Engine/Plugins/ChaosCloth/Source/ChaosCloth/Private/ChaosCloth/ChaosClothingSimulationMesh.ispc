// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Matrix.isph"
#include "Math/Transform.isph"

#define MaxTotalInfluences 12

struct FClothVertBoneData
{
	int32 NumInfluences;
	uint16 BoneIndices[MaxTotalInfluences];
	float BoneWeights[MaxTotalInfluences];
};

static inline void AddInfluence(uniform FVector4f& OutPosition, uniform FVector4f& OutNormal, const uniform FVector3f& RefParticle, const uniform FVector3f& RefNormal, const uniform FMatrix44f *uniform BoneMatrix, const uniform float Weight)
{
	const uniform FVector4f M0M1M2 = *((uniform FVector4f *uniform)&BoneMatrix->M[0]);

	const uniform FVector4f VTempXP = RefParticle.V[0] * M0M1M2;
	const uniform FVector4f VTempXN = RefNormal.V[0] * M0M1M2;

	const uniform FVector4f M4M5M6 = *((uniform FVector4f *uniform)&BoneMatrix->M[4]);

	const uniform FVector4f VTempYP = RefParticle.V[1] * M4M5M6 + VTempXP;
	const uniform FVector4f VTempYN = RefNormal.V[1] * M4M5M6 + VTempXN;

	const uniform FVector4f M8M9M10 = *((uniform FVector4f *uniform)&BoneMatrix->M[8]);

	const uniform FVector4f VTempZP = RefParticle.V[2] * M8M9M10 + VTempYP;
	const uniform FVector4f Norm = RefNormal.V[2] * M8M9M10 + VTempYN;

	const uniform FVector4f VTempWP = *((uniform FVector4f *uniform)&BoneMatrix->M[12]);
	const uniform FVector4f Pos = VTempZP + VTempWP;

	OutPosition = OutPosition + Pos * Weight;
	OutNormal = OutNormal + Norm * Weight;
}

static inline uniform FVector3f GetSafeNormal(const uniform FVector3f &V)
{
	//We want N / ||N|| and to avoid inf
	//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
	const uniform float SizeSqr = VectorSizeSquared(V);
	return VectorSelect(SizeSqr <= FLT_MIN, FloatForwardVector, V / sqrt(SizeSqr));
}

export void SkinPhysicsMesh(uniform FVector3f OutPositions[],
							uniform FVector3f OutNormals[],
							const uniform FVector3f Vertices[],
							const uniform FVector3f Normals[],
							const uniform FClothVertBoneData BoneData[],
							const uniform int32 BoneMap[],
							const uniform FMatrix44f BoneMatrices[],
							const uniform FTransform3f &ComponentToLocalSpace,
							const uniform int32 NumPoints)
{
	for(uniform int i = 0; i < NumPoints; ++i)
	{
		const uniform FVector3f RefParticle = Vertices[i];
		const uniform FVector3f RefNormal = Normals[i];

		uniform FVector4f OutPosition = FloatZero;
		uniform FVector4f OutNormal = FloatZero;

		const uniform int NumInfluences = BoneData[i].NumInfluences;

		for(uniform int j = 0; j < NumInfluences; ++j)
		{
			const uniform int BoneMatrixIndex = BoneMap[BoneData[i].BoneIndices[j]];
			const uniform float Weight = BoneData[i].BoneWeights[j];

			const uniform FMatrix44f *uniform BoneMatrix = &BoneMatrices[BoneMatrixIndex];
			AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrix, Weight);
		}

		const uniform FVector3f Position = SetVector(OutPosition);
		const uniform FVector3f Normal = SetVector(OutNormal);

		OutPositions[i] = TransformPosition(ComponentToLocalSpace, Position);
		OutNormals[i] = GetSafeNormal(TransformVector(ComponentToLocalSpace, Normal));
	}
}
