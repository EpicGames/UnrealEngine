// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationSkeletalMesh.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "Components/SkeletalMeshComponent.h"
#include "ChaosWeightMapTarget.h"
#include "ClothingAsset.h"
#include "ClothingSimulation.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSimulationFactory.h"
#include "Engine/SkeletalMesh.h"

namespace Chaos
{
	FClothingSimulationSkeletalMesh::FClothingSimulationSkeletalMesh(const UClothingAssetCommon* InAsset, const USkeletalMeshComponent* InSkeletalMeshComponent)
		: FClothingSimulationMesh(InSkeletalMeshComponent->GetOwner() ?
				FString::Format(TEXT("{0}|{1}"), { InSkeletalMeshComponent->GetOwner()->GetActorNameOrLabel(), InSkeletalMeshComponent->GetName() }) :
				InSkeletalMeshComponent->GetName())
		, Asset(InAsset)
		, SkeletalMeshComponent(InSkeletalMeshComponent)
	{
#if CHAOS_DEBUG_DRAW
		const FReferenceSkeleton* const ReferenceSkeleton = &CastChecked<USkeletalMesh>(Asset->GetOuter())->GetRefSkeleton();
		const int32 ReferenceBoneIndex = GetReferenceBoneIndex();
		if (ReferenceSkeleton && ReferenceBoneIndex >= 0 && ReferenceBoneIndex < ReferenceSkeleton->GetNum())
		{
			ReferenceBoneName = ReferenceSkeleton->GetBoneName(ReferenceBoneIndex);
		}
#endif
	}

	int32 FClothingSimulationSkeletalMesh::GetNumLODs() const
	{
		return Asset ? Asset->LodData.Num() : 0;
	}

	int32 FClothingSimulationSkeletalMesh::GetLODIndex() const
	{
		int32 LODIndex = INDEX_NONE;

		if (SkeletalMeshComponent)
		{
			const int32 OwnerLODIndex = SkeletalMeshComponent->GetPredictedLODLevel();
			LODIndex = GetLODIndexFromOwnerLODIndex(OwnerLODIndex);
		}
		return LODIndex;
	}

	int32 FClothingSimulationSkeletalMesh::GetLODIndexFromOwnerLODIndex(int32 OwnerLODIndex) const
	{
		int32 LODIndex = INDEX_NONE;

		if (Asset)
		{
			if (Asset->LodMap.IsValidIndex(OwnerLODIndex))
			{
				const int32 ClothLODIndex = Asset->LodMap[OwnerLODIndex];
				if (Asset->LodData.IsValidIndex(ClothLODIndex))
				{
					LODIndex = ClothLODIndex;
				}
			}
		}
		return LODIndex;

	}

	int32 FClothingSimulationSkeletalMesh::GetOwnerLODIndex(int32 LODIndex) const
	{
		const int32 OwnerLODIndex = Asset ? Asset->LodMap.Find(LODIndex) : INDEX_NONE;
		return OwnerLODIndex != INDEX_NONE ? OwnerLODIndex : 0;  // Safer to return the default LOD 0 than INDEX_NONE in this case
	}

	bool FClothingSimulationSkeletalMesh::IsValidLODIndex(int32 LODIndex) const
	{
		return Asset && Asset->LodData.IsValidIndex(LODIndex);
	}

	int32 FClothingSimulationSkeletalMesh::GetNumPoints(int32 LODIndex) const
	{
		return IsValidLODIndex(LODIndex) ? Asset->LodData[LODIndex].PhysicalMeshData.Vertices.Num() : 0;
	}

	int32 FClothingSimulationSkeletalMesh::GetNumPatternPoints(int32 LODIndex) const
	{
		return 0;
	}

	TConstArrayView<FVector3f> FClothingSimulationSkeletalMesh::GetPositions(int32 LODIndex) const
	{
		if (IsValidLODIndex(LODIndex))
		{
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			return TConstArrayView<FVector3f>(ClothPhysicalMeshData.Vertices);
		}
		return TConstArrayView<FVector3f>();
	}

	TConstArrayView<FVector2f> FClothingSimulationSkeletalMesh::GetPatternPositions(int32 LODIndex) const
	{
		return TConstArrayView<FVector2f>();
	}

	TConstArrayView<FVector3f> FClothingSimulationSkeletalMesh::GetNormals(int32 LODIndex) const
	{
		if (IsValidLODIndex(LODIndex))
		{
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			return TConstArrayView<FVector3f>(ClothPhysicalMeshData.Normals);
		}
		return TConstArrayView<FVector3f>();
	}

	TConstArrayView<uint32> FClothingSimulationSkeletalMesh::GetIndices(int32 LODIndex) const
	{
		return IsValidLODIndex(LODIndex) ?
			TConstArrayView<uint32>(Asset->LodData[LODIndex].PhysicalMeshData.Indices) :
			TConstArrayView<uint32>();
	}

	TConstArrayView<uint32> FClothingSimulationSkeletalMesh::GetPatternIndices(int32 LODIndex) const
	{
		return TConstArrayView<uint32>();
	}

	TConstArrayView<uint32> FClothingSimulationSkeletalMesh::GetPatternToWeldedIndices(int32 LODIndex) const
	{
		return TConstArrayView<uint32>();
	}

	TArray<FName> FClothingSimulationSkeletalMesh::GetWeightMapNames(int32 LODIndex) const
	{
		TArray<FName> WeightMapNames;
		if (IsValidLODIndex(LODIndex))
		{
			// This must match the order of GetWeightMaps
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			WeightMapNames.Reserve(ClothPhysicalMeshData.WeightMaps.Num());

			const UEnum* const ChaosWeightMapTargetEnum = StaticEnum<EChaosWeightMapTarget>();
			for (const TPair<uint32, FPointWeightMap>& TargetIDAndMap : ClothPhysicalMeshData.WeightMaps)
			{
				WeightMapNames.Add(FName(ChaosWeightMapTargetEnum->GetNameStringByValue(TargetIDAndMap.Get<0>())));
			}
		}

		return WeightMapNames;
	}

	TMap<FString, int32> FClothingSimulationSkeletalMesh::GetWeightMapIndices(int32 LODIndex) const
	{
		TMap<FString, int32> WeightMapIndices;
		const TArray<FName> WeightMapNames = GetWeightMapNames(LODIndex);
		WeightMapIndices.Reserve(WeightMapNames.Num());
		for (int32 WeightMapIndex = 0; WeightMapIndex < WeightMapNames.Num(); ++WeightMapIndex)
		{
			const FName& WeightMapName = WeightMapNames[WeightMapIndex];
			WeightMapIndices.Emplace(WeightMapName.ToString(), WeightMapIndex);
		}
		return WeightMapIndices;
	}

	TArray<TConstArrayView<FRealSingle>> FClothingSimulationSkeletalMesh::GetWeightMaps(int32 LODIndex) const
	{
		TArray<TConstArrayView<FRealSingle>> WeightMaps;
		if (IsValidLODIndex(LODIndex))
		{
			// This must match the order of GetWeightMapNames
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;

			WeightMaps.Reserve(ClothPhysicalMeshData.WeightMaps.Num());
			for (const TPair<uint32, FPointWeightMap>& TargetIDAndMap : ClothPhysicalMeshData.WeightMaps)
			{
				WeightMaps.Add(TargetIDAndMap.Get<1>().Values);
			}
		}
		return WeightMaps;
	}

	TMap<FString, const TSet<int32>*> FClothingSimulationSkeletalMesh::GetVertexSets(int32 LODIndex) const
	{
		TMap<FString, const TSet<int32>*> VertexSets;
		if (IsValidLODIndex(LODIndex))
		{
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			static const FString SelfCollisionSphereSetName(TEXT("SelfCollisionSphereSetName"));
			VertexSets.Add(SelfCollisionSphereSetName, &ClothPhysicalMeshData.SelfCollisionVertexSet);
		}

		return VertexSets;
	}

	TMap<FString, const TSet<int32>*> FClothingSimulationSkeletalMesh::GetFaceSets(int32 LODIndex) const
	{
		// Not supported
		return TMap<FString, const TSet<int32>*>();
	}

	TMap<FString, TConstArrayView<int32>> FClothingSimulationSkeletalMesh::GetFaceIntMaps(int32 LODIndex) const
	{
		// Not supported
		return TMap<FString, TConstArrayView<int32>>();
	}

	TArray<TConstArrayView<TTuple<int32, int32, float>>> FClothingSimulationSkeletalMesh::GetTethers(int32 LODIndex, bool bUseGeodesicTethers) const
	{
		TArray<TConstArrayView<TTuple<int32, int32, float>>> Tethers;
		if (IsValidLODIndex(LODIndex))
		{
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			const FClothTetherData& ClothTetherData = bUseGeodesicTethers ? ClothPhysicalMeshData.GeodesicTethers : ClothPhysicalMeshData.EuclideanTethers;
		
			const int32 NumTetherBatches = ClothTetherData.Tethers.Num();
			Tethers.Reserve(NumTetherBatches);
			for (int32 Index = 0; Index < NumTetherBatches; ++Index)
			{
				Tethers.Emplace(TConstArrayView<TTuple<int32, int32, float>>(ClothTetherData.Tethers[Index]));
			}
		}
		return Tethers;
	}

	int32 FClothingSimulationSkeletalMesh::GetReferenceBoneIndex() const
	{
		return Asset ? Asset->ReferenceBoneIndex : INDEX_NONE;
	}

	FTransform FClothingSimulationSkeletalMesh::GetReferenceBoneTransform() const
	{
		if (const FClothingSimulationContextCommon* const Context = GetContext())
		{
			const int32 ReferenceBoneIndex = GetReferenceBoneIndex();
			const TArray<FTransform>& BoneTransforms = Context->BoneTransforms;

			return BoneTransforms.IsValidIndex(ReferenceBoneIndex) ?
				BoneTransforms[ReferenceBoneIndex] * Context->ComponentToWorld :
				Context->ComponentToWorld;
		}
		return FTransform::Identity;
	}

	const TArray<FTransform>& FClothingSimulationSkeletalMesh::GetBoneTransforms() const
	{
		static TArray<FTransform> EmptyArray;
		return GetContext() ? GetContext()->BoneTransforms : EmptyArray;
	}

	const FTransform& FClothingSimulationSkeletalMesh::GetComponentToWorldTransform() const
	{
		return GetContext() ? GetContext()->ComponentToWorld : FTransform::Identity;
	}

	const TArray<FMatrix44f>& FClothingSimulationSkeletalMesh::GetRefToLocalMatrices() const
	{
		static TArray<FMatrix44f> EmptyArray;
		return GetContext() ? GetContext()->RefToLocals : EmptyArray;
	}

	TConstArrayView<int32> FClothingSimulationSkeletalMesh::GetBoneMap() const
	{
		return Asset ? Asset->UsedBoneIndices : TConstArrayView<int32>();
	}

	TConstArrayView<FClothVertBoneData> FClothingSimulationSkeletalMesh::GetBoneData(int32 LODIndex) const
	{
		if (IsValidLODIndex(LODIndex))
		{
			const FClothLODDataCommon& ClothLODData = Asset->LodData[LODIndex];
			const FClothPhysicalMeshData& ClothPhysicalMeshData = ClothLODData.PhysicalMeshData;
			UE_CLOGF(ClothPhysicalMeshData.MaxBoneWeights > 12, LogChaosCloth, Warning, "The cloth physics mesh skinning code can't cope with more than 12 bone influences.");
			return ClothPhysicalMeshData.BoneData;
		}
		return TConstArrayView<FClothVertBoneData>();
	}

	TConstArrayView<FMeshToMeshVertData> FClothingSimulationSkeletalMesh::GetTransitionUpSkinData(int32 LODIndex) const
	{
		return IsValidLODIndex(LODIndex) ? TConstArrayView<FMeshToMeshVertData>(Asset->LodData[LODIndex].TransitionUpSkinData) : TConstArrayView<FMeshToMeshVertData>();
	}

	TConstArrayView<FMeshToMeshVertData> FClothingSimulationSkeletalMesh::GetTransitionDownSkinData(int32 LODIndex) const
	{
		return IsValidLODIndex(LODIndex) ? TConstArrayView<FMeshToMeshVertData>(Asset->LodData[LODIndex].TransitionDownSkinData) : TConstArrayView<FMeshToMeshVertData>();
	}

	const FClothingSimulationContextCommon* FClothingSimulationSkeletalMesh::GetContext() const
	{
		if (SkeletalMeshComponent)
		{
			for (const FClothingSimulationInstance& ClothingSimulationInstance : SkeletalMeshComponent->GetClothingSimulationInstances())
			{
				if (UClothingSimulationFactory* const ClothingSimulationFactory = ClothingSimulationInstance.GetClothingSimulationFactory())
				{
					if (ClothingSimulationFactory->SupportsAsset(Asset))
					{
						return static_cast<const FClothingSimulationContextCommon*>(ClothingSimulationInstance.GetClothingSimulationContext());
					}
				}
			}
		}
		return nullptr;
	}
}  // End namespace Chaos
