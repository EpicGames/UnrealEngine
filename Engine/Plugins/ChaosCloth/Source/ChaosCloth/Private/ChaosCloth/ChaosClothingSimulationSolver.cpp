// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothComplexColliders.h"
#include "ChaosCloth/ChaosClothingSimulationCollider.h"
#include "ChaosCloth/ChaosClothingSimulationMesh.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "Chaos/SoftsEvolution.h"
#include "Chaos/PBDEvolution.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h" 
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "BuoyancyField.h"

#if INTEL_ISPC
#include "ChaosClothingSimulationSolver.ispc.generated.h"
#endif

#if !UE_BUILD_SHIPPING
#include "FramePro/FramePro.h"
#include "HAL/IConsoleManager.h"
#else
#define FRAMEPRO_ENABLED 0
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update"), STAT_ChaosClothSolverUpdate, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update Cloths"), STAT_ChaosClothSolverUpdateCloths, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update Solver Fields"), STAT_ChaosClothSolverUpdateSolverFields, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update Pre Solver Step"), STAT_ChaosClothSolverUpdatePreSolverStep, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update Solver Step"), STAT_ChaosClothSolverUpdateSolverStep, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Update Post Solver Step"), STAT_ChaosClothSolverUpdatePostSolverStep, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Calculate Bounds"), STAT_ChaosClothSolverCalculateBounds, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Particle Pre Simulation Transforms"), STAT_ChaosClothParticlePreSimulationTransforms, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Collision Pre Simulation Transforms"), STAT_ChaosClothCollisionPreSimulationTransforms, STATGROUP_ChaosCloth);
DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Solver Particle Pre Substep Kinematic Interpolation"), STAT_ChaosClothParticlePreSubstepKinematicInterpolation, STATGROUP_ChaosCloth);

#if INTEL_ISPC

#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_PreSimulationTransforms_ISPC_Enabled = CHAOS_PRE_SIMULATION_TRANSFORMS_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosPreSimulationTransformsISPCEnabled(TEXT("p.Chaos.PreSimulationTransforms.ISPC"), bChaos_PreSimulationTransforms_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in ApplySimulationTransforms"));
bool bChaos_CalculateBounds_ISPC_Enabled = CHAOS_CALCULATE_BOUNDS_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosCalculateBoundsISPCEnabled(TEXT("p.Chaos.CalculateBounds.ISPC"), bChaos_CalculateBounds_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in CalculateBounds"));
bool bChaos_PreSubstepInterpolation_ISPC_Enabled = CHAOS_PRE_SUBSTEP_INTERPOLATION_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosPreSubstepInterpolationISPCEnabled(TEXT("p.Chaos.PreSubstepInterpolation.ISPC"), bChaos_PreSubstepInterpolation_ISPC_Enabled, TEXT("Whether to use ISPC optimization in PreSubstep"));
#endif

static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector) != sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FTransform3f) == sizeof(Chaos::Softs::FSolverRigidTransform3), "sizeof(ispc::FVector) != sizeof(Chaos::Softs::FSolverRigidTransform3)");
#endif

namespace Chaos
{

static int32 ClothSolverMinParallelBatchSize = 1000;
static bool bClothSolverParallelClothPreUpdate = true;
static bool bClothSolverParallelClothUpdate = true;
static bool bClothSolverParallelClothPostUpdate = true;
static bool bClothSolverDisableTimeDependentNumIterations = false;
static bool bClothSolverUseVelocityScale = true;
static float ClothSolverMaxVelocity = 0.f;

#if !UE_BUILD_SHIPPING
static int32 ClothSolverDebugHitchLength = 0;
static int32 ClothSolverDebugHitchInterval = 0;
static bool bClothSolverDisableCollision = false;

FAutoConsoleVariableRef CVarClothSolverMinParallelBatchSize(TEXT("p.ChaosCloth.Solver.MinParallelBatchSize"), ClothSolverMinParallelBatchSize, TEXT("The minimum number of particle to process in parallel batch by the solver."));
FAutoConsoleVariableRef CVarClothSolverParallelClothPreUpdate(TEXT("p.ChaosCloth.Solver.ParallelClothPreUpdate"), bClothSolverParallelClothPreUpdate, TEXT("Pre-transform the cloth particles for each cloth in parallel."));
FAutoConsoleVariableRef CVarClothSolverParallelClothUpdate(TEXT("p.ChaosCloth.Solver.ParallelClothUpdate"), bClothSolverParallelClothUpdate, TEXT("Skin the physics mesh and do the other cloth update for each cloth in parallel."));
FAutoConsoleVariableRef CVarClothSolverParallelClothPostUpdate(TEXT("p.ChaosCloth.Solver.ParallelClothPostUpdate"), bClothSolverParallelClothPostUpdate, TEXT("Pre-transform the cloth particles for each cloth in parallel."));
FAutoConsoleVariableRef CVarClothSolverDebugHitchLength(TEXT("p.ChaosCloth.Solver.DebugHitchLength"), ClothSolverDebugHitchLength, TEXT("Hitch length in ms. Create artificial hitches to debug simulation jitter. 0 to disable"));
FAutoConsoleVariableRef CVarClothSolverDebugHitchInterval(TEXT("p.ChaosCloth.Solver.DebugHitchInterval"), ClothSolverDebugHitchInterval, TEXT("Hitch interval in frames. Create artificial hitches to debug simulation jitter. 0 to disable"));
FAutoConsoleVariableRef CVarClothSolverDisableCollision(TEXT("p.ChaosCloth.Solver.DisableCollision"), bClothSolverDisableCollision, TEXT("Disable all collision particles. Needs reset of the simulation (p.ChaosCloth.Reset)."));
#endif

FAutoConsoleVariableRef CVarClothSolverDisableTimeDependentNumIterations(TEXT("p.ChaosCloth.Solver.DisableTimeDependentNumIterations"), bClothSolverDisableTimeDependentNumIterations, TEXT("Make the number of iterations independent from the time step."));
FAutoConsoleVariableRef CVarClothSolverUseVelocityScale(TEXT("p.ChaosCloth.Solver.UseVelocityScale"), bClothSolverUseVelocityScale, TEXT("Use the velocity scale to compensate for clamping to MaxPhysicsDelta, in order to avoid miscalculating velocities during hitches."));
FAutoConsoleVariableRef CVarClothSolverMaxVelocity(TEXT("p.ChaosCloth.Solver.MaxVelocity"), ClothSolverMaxVelocity, TEXT("Maximum relative velocity of the cloth particles relatively to their animated positions equivalent. 0 to disable."));

// Temporary hack to return to previous broken acceleration clamping behavior.
static bool bClothSolverBrokenAccelerationClampingToZero = false;
FAutoConsoleVariableRef CVarClothSolverBrokenAccelerationClampingHack(TEXT("p.ChaosCloth.Solver.BrokenAccelerationClampingHack"), bClothSolverBrokenAccelerationClampingToZero, TEXT("Return to previous broken behavior where acceleration clamping would clamp to zero velocity. This cvar will be removed."));

static bool bClothSolverEnableResetIfExtremeEdgeDeformation = true;
FAutoConsoleVariableRef CVarClothSolverEnableResetIfExtremeEdgeDeformation(TEXT("p.ChaosCloth.Solver.EnableResetIfExtremeEdgeDeformation"), bClothSolverEnableResetIfExtremeEdgeDeformation, TEXT("Set to false to disable reset to animation positions if there's any extreme edge deformation."));

namespace ClothingSimulationSolverDefault
{
	static const Softs::FSolverVec3 Gravity((Softs::FSolverReal)0., (Softs::FSolverReal)0., (Softs::FSolverReal)-980.665);  // cm/s^2
	static const Softs::FSolverVec3 WindVelocity((Softs::FSolverReal)0.);
	static const int32 NumIterations = 1;
	static const int32 MinNumIterations = 1;
	static const int32 MaxNumIterations = 10;
	static const int32 NumSubsteps = 1;
	static const int32 MinNumSubsteps = 1;
	static const FRealSingle DynamicSubstepDeltaTime = 0.f;
	static const bool bEnableNumSelfCollisionSubsteps = false;
	static const bool bUseSelfCollisionSubstepsForSkinnedTriangleMeshes = true;
	static const int32 NumSelfCollisionSubsteps = 1;
	static const FRealSingle SolverFrequency = 60.f;
	static const FRealSingle SelfCollisionThickness = 2.f;
	static const FRealSingle CollisionThickness = 1.2f;
	static const FRealSingle FrictionCoefficient = 0.2f;
	static const FRealSingle DampingCoefficient = 0.01f;
	static const FRealSingle LocalDampingCoefficient = 0.f;
}

namespace ClothingSimulationSolverConstant
{
	static const FRealSingle WorldScale = 100.f;  // World is in cm, but values like wind speed and density are in SI unit and relates to m.
	static const Softs::FSolverReal StartDeltaTime = (Softs::FSolverReal)(1. / 30.);  // Initialize filtered timestep at 30fps
}

namespace Private
{
template<typename T>
TConstArrayView<T> GetPBDParticleConstArrayView(const TUniquePtr<Softs::FPBDEvolution>& PBDEvolution, const int32 ParticleRangeId, const TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TConstArrayView<T>() : TConstArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetParticleRangeSize(ParticleRangeId));
}

template<typename T>
TArrayView<T> GetPBDParticleArrayView(TUniquePtr<Softs::FPBDEvolution>& PBDEvolution, const int32 ParticleRangeId, TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TArrayView<T>() : TArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetParticleRangeSize(ParticleRangeId));
}

template<typename T>
TConstArrayView<T> GetParticleConstArrayView(const TUniquePtr<Softs::FEvolution>& Evolution, const TUniquePtr<Softs::FPBDEvolution>& PBDEvolution,
	const int32 ParticleRangeId, const TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TConstArrayView<T>() : (Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetConstArrayView(Array) :
		TConstArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetParticleRangeSize(ParticleRangeId)));
}

template<typename T>
TArrayView<T> GetParticleArrayView(TUniquePtr<Softs::FEvolution>& Evolution, TUniquePtr<Softs::FPBDEvolution>& PBDEvolution,
	const int32 ParticleRangeId, TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TArrayView<T>() : (Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetArrayView(Array) :
		TArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetParticleRangeSize(ParticleRangeId)));
}

template<typename T>
TConstArrayView<T> GetPBDCollisionParticleConstArrayView(const TUniquePtr<Softs::FPBDEvolution>& PBDEvolution, const int32 ParticleRangeId, const TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TConstArrayView<T>() : TConstArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetCollisionParticleRangeSize(ParticleRangeId));
}

template<typename T>
TArrayView<T> GetPBDCollisionParticleArrayView(TUniquePtr<Softs::FPBDEvolution>& PBDEvolution, const int32 ParticleRangeId, TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TArrayView<T>() : TArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetCollisionParticleRangeSize(ParticleRangeId));
}

template<typename T>
TConstArrayView<T> GetCollisionParticleConstArrayView(const TUniquePtr<Softs::FEvolution>& Evolution, const TUniquePtr<Softs::FPBDEvolution>& PBDEvolution,
	const int32 ParticleRangeId, const TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TConstArrayView<T>() : (Evolution ? Evolution->GetCollisionParticleRange(ParticleRangeId).GetConstArrayView(Array) :
		TConstArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetCollisionParticleRangeSize(ParticleRangeId)));
}

template<typename T>
TArrayView<T> GetCollisionParticleArrayView(TUniquePtr<Softs::FEvolution>& Evolution, TUniquePtr<Softs::FPBDEvolution>& PBDEvolution,
	const int32 ParticleRangeId, TArray<T>& Array)
{
	return ParticleRangeId == INDEX_NONE ? TArrayView<T>() : (Evolution ? Evolution->GetCollisionParticleRange(ParticleRangeId).GetArrayView(Array) :
		TArrayView<T>(Array.GetData() + ParticleRangeId, PBDEvolution->GetCollisionParticleRangeSize(ParticleRangeId)));
}
}

FClothingSimulationSolver::FClothingSimulationSolver(FClothingSimulationConfig* InConfig, bool bUseLegacySolver)
	: Evolution(nullptr), PBDEvolution(nullptr)
	, OldLocalSpaceLocation(0.)
	, LocalSpaceLocation(0.)
	, LocalSpaceRotation(FRotation3::Identity)
	, VelocityScale(1.)
	, Time(0.)
	, DeltaTime(ClothingSimulationSolverConstant::StartDeltaTime)
	, CollisionParticlesOffset(0)
	, CollisionParticlesSize(0)
	, Gravity(ClothingSimulationSolverDefault::Gravity)
	, WindVelocity(ClothingSimulationSolverDefault::WindVelocity)
	, LegacyWindAdaption(0.f)
	, bIsClothGravityOverrideEnabled(false)
	, bEnableSolver(true)
{
	SetConfig(InConfig); // This will generate a local default config if nullptr so we have something to use if there are no cloth assets..

	if (!bUseLegacySolver)
	{
		Evolution.Reset(
			new Softs::FEvolution(
				Config->GetProperties(SolverLOD)
			)
		);

		// Add simulation groups arrays
		Evolution->AddGroupArray(&PreSimulationTransforms);
		Evolution->AddGroupArray(&FictitiousAngularVelocities);
		Evolution->AddGroupArray(&ReferenceSpaceLocations);
		Evolution->AddGroupArray(&AppliedReferenceSpaceVelocities);
		Evolution->AddGroupArray(&AppliedReferenceSpaceAngularVelocities);
#if CHAOS_DEBUG_DRAW
		Evolution->AddGroupArray(&OriginalReferenceSpaceVelocities);
		Evolution->AddGroupArray(&OriginalReferenceSpaceAngularVelocities);
#endif

		Evolution->AddParticleArray(&Normals);
		Evolution->AddParticleArray(&OldAnimationPositions);
		Evolution->AddParticleArray(&AnimationPositions);
		Evolution->AddParticleArray(&InterpolatedAnimationPositions);
		Evolution->AddParticleArray(&OldAnimationNormals);
		Evolution->AddParticleArray(&AnimationNormals);
		Evolution->AddParticleArray(&InterpolatedAnimationNormals);
		Evolution->AddParticleArray(&AnimationVelocities);

		Evolution->AddCollisionParticleArray(&CollisionBoneIndices);
		Evolution->AddCollisionParticleArray(&CollisionBaseTransforms);
		Evolution->AddCollisionParticleArray(&OldCollisionTransforms);
		Evolution->AddCollisionParticleArray(&CollisionTransforms);
		Evolution->AddCollisionParticleArray(&Collided);
		Evolution->AddCollisionParticleArray(&LastSubframeCollisionTransformsCCD);

		Evolution->SetKinematicUpdateFunction(
			[this](Softs::FSolverParticlesRange& ParticlesInput, const Softs::FSolverReal Dt, const Softs::FSolverReal LocalTime)
		{
			Softs::FPAndInvM* const PAndInvM = ParticlesInput.GetPAndInvM().GetData();
			Softs::FSolverVec3* const V = ParticlesInput.GetV().GetData();
			const Softs::FSolverVec3* const InterpPos = ParticlesInput.GetConstArrayView(InterpolatedAnimationPositions).GetData();
			const Softs::FSolverVec3* const AnimV = ParticlesInput.GetConstArrayView(AnimationVelocities).GetData();
			for (int32 Index = 0; Index < ParticlesInput.GetRangeSize(); ++Index)
			{
				if (PAndInvM[Index].InvM == (FSolverReal)0.)
				{
					PAndInvM[Index].P = InterpPos[Index];  // X is the step initial condition, here it's P that needs to be updated so that constraints works with the correct step target

					// This could be set per frame since we currently use linear interpolation, but it's cleaner to set here.
					V[Index] = AnimV[Index];
				}
			}
		});

		Evolution->SetCollisionKinematicUpdateFunction(
			[this](Softs::FSolverCollisionParticlesRange& ParticlesInput, const Softs::FSolverReal Dt, const Softs::FSolverReal LocalTime)
		{
			checkSlow(Dt > SMALL_NUMBER && DeltaTime > SMALL_NUMBER);
			const Softs::FSolverReal Alpha = (LocalTime - Time) / DeltaTime;
			const Softs::FSolverRigidTransform3* const CollisionTransformsLocal = ParticlesInput.GetConstArrayView(CollisionTransforms).GetData();
			const Softs::FSolverRigidTransform3* const OldCollisionTransformsLocal = ParticlesInput.GetConstArrayView(OldCollisionTransforms).GetData();
			Softs::FSolverRigidTransform3* const LastSubframeCollisionTransformsCCDLocal = ParticlesInput.GetArrayView(LastSubframeCollisionTransformsCCD).GetData();
			Softs::FSolverVec3* const X = ParticlesInput.XArray().GetData();
			Softs::FSolverVec3* const V = ParticlesInput.GetV().GetData();
			Softs::FSolverVec3* const W = ParticlesInput.GetW().GetData();
			Softs::FSolverRotation3* const R = ParticlesInput.GetR().GetData();

			for (int32 Index = 0; Index < ParticlesInput.GetRangeSize(); ++Index)
			{
				LastSubframeCollisionTransformsCCDLocal[Index] = Softs::FSolverRigidTransform3(X[Index], R[Index]);
				const Softs::FSolverVec3 NewX =
					Alpha * CollisionTransformsLocal[Index].GetTranslation() + ((Softs::FSolverReal)1. - Alpha) * OldCollisionTransformsLocal[Index].GetTranslation();
				V[Index] = (NewX - X[Index]) / Dt;
				X[Index] = NewX;
				const Softs::FSolverRotation3 NewR = Softs::FSolverRotation3::Slerp(OldCollisionTransformsLocal[Index].GetRotation(), CollisionTransformsLocal[Index].GetRotation(), Alpha);
				const Softs::FSolverRotation3 Delta = NewR * R[Index].Inverse();
				const Softs::FSolverReal Angle = Delta.GetAngle();
				const Softs::FSolverVec3 Axis = Delta.GetRotationAxis();
				W[Index] = (Softs::FSolverVec3)Axis * Angle / Dt;
				R[Index] = NewR;
			}

			const int32 CollisionRangeId = ParticlesInput.GetRangeId();
			if (FClothComplexColliders* const Colliders = GetComplexColliders(CollisionRangeId))
			{
				Colliders->KinematicUpdate(ParticlesInput, Dt, Alpha);
			}
		});
	}
	else
	{
		Softs::FSolverParticles LocalParticles;
		Softs::FSolverCollisionParticles RigidParticles;
		PBDEvolution.Reset(
			new Softs::FPBDEvolution(
				MoveTemp(LocalParticles),
				MoveTemp(RigidParticles),
				{}, // CollisionTriangles
				FMath::Min(ClothingSimulationSolverDefault::NumIterations, ClothingSimulationSolverDefault::MaxNumIterations),
				(Softs::FSolverReal)ClothingSimulationSolverDefault::CollisionThickness,
				(Softs::FSolverReal)ClothingSimulationSolverDefault::SelfCollisionThickness,
				(Softs::FSolverReal)ClothingSimulationSolverDefault::FrictionCoefficient,
				(Softs::FSolverReal)ClothingSimulationSolverDefault::DampingCoefficient,
				(Softs::FSolverReal)ClothingSimulationSolverDefault::LocalDampingCoefficient));

		// Add simulation groups arrays
		PBDEvolution->AddArray(&PreSimulationTransforms);
		PBDEvolution->AddArray(&FictitiousAngularVelocities);
		PBDEvolution->AddArray(&ReferenceSpaceLocations);
		PBDEvolution->AddArray(&AppliedReferenceSpaceVelocities);
		PBDEvolution->AddArray(&AppliedReferenceSpaceAngularVelocities);
#if CHAOS_DEBUG_DRAW
		PBDEvolution->AddArray(&OriginalReferenceSpaceVelocities);
		PBDEvolution->AddArray(&OriginalReferenceSpaceAngularVelocities);
#endif

		PBDEvolution->Particles().AddArray(&Normals);
		PBDEvolution->Particles().AddArray(&OldAnimationPositions);
		PBDEvolution->Particles().AddArray(&AnimationPositions);
		PBDEvolution->Particles().AddArray(&InterpolatedAnimationPositions);
		PBDEvolution->Particles().AddArray(&OldAnimationNormals);
		PBDEvolution->Particles().AddArray(&AnimationNormals);
		PBDEvolution->Particles().AddArray(&InterpolatedAnimationNormals);
		PBDEvolution->Particles().AddArray(&AnimationVelocities);

		PBDEvolution->CollisionParticles().AddArray(&CollisionBoneIndices);
		PBDEvolution->CollisionParticles().AddArray(&CollisionBaseTransforms);
		PBDEvolution->CollisionParticles().AddArray(&OldCollisionTransforms);
		PBDEvolution->CollisionParticles().AddArray(&CollisionTransforms);
		PBDEvolution->CollisionParticles().AddArray(&Collided);
		PBDEvolution->CollisionParticles().AddArray(&LastSubframeCollisionTransformsCCD);

		PBDEvolution->SetKinematicUpdateFunction(
			[this](Softs::FSolverParticles& ParticlesInput, const Softs::FSolverReal Dt, const Softs::FSolverReal LocalTime, const int32 Index)
		{
			ParticlesInput.P(Index) = InterpolatedAnimationPositions[Index];  // X is the step initial condition, here it's P that needs to be updated so that constraints works with the correct step target
			// This could be set per frame since we currently use linear interpolation, but it's cleaner to set here.
			ParticlesInput.V(Index) = AnimationVelocities[Index];
		});

		PBDEvolution->SetCollisionKinematicUpdateFunction(
			[this](Softs::FSolverCollisionParticles& ParticlesInput, const Softs::FSolverReal Dt, const Softs::FSolverReal LocalTime, const int32 Index)
		{
			LastSubframeCollisionTransformsCCD[Index] = Softs::FSolverRigidTransform3(ParticlesInput.GetX(Index), ParticlesInput.GetR(Index));

			checkSlow(Dt > SMALL_NUMBER && DeltaTime > SMALL_NUMBER);
			const Softs::FSolverReal Alpha = (LocalTime - Time) / DeltaTime;
			const Softs::FSolverVec3 NewX =
				Alpha * CollisionTransforms[Index].GetTranslation() + ((Softs::FSolverReal)1. - Alpha) * OldCollisionTransforms[Index].GetTranslation();
			ParticlesInput.V(Index) = (NewX - ParticlesInput.GetX(Index)) / Dt;
			ParticlesInput.X(Index) = NewX;
			const Softs::FSolverRotation3 NewR = Softs::FSolverRotation3::Slerp(OldCollisionTransforms[Index].GetRotation(), CollisionTransforms[Index].GetRotation(), Alpha);
			const Softs::FSolverRotation3 Delta = NewR * ParticlesInput.GetR(Index).Inverse();
			const Softs::FSolverReal Angle = Delta.GetAngle();
			const Softs::FSolverVec3 Axis = Delta.GetRotationAxis();
			ParticlesInput.W(Index) = (Softs::FSolverVec3)Axis * Angle / Dt;
			ParticlesInput.SetR(Index, NewR);
		});
	}
}

FClothingSimulationSolver::~FClothingSimulationSolver()
{
	// If the PropertyCollection is owned by this object, so does the current config object
	if (PropertyCollection.IsValid())
	{
		delete Config;
	}
}

void FClothingSimulationSolver::SetLocalSpaceLocation(const FVec3& InLocalSpaceLocation, bool bReset)
{
	LocalSpaceLocation = InLocalSpaceLocation;
	if (bReset)
	{
		OldLocalSpaceLocation = InLocalSpaceLocation;
	}
}

void FClothingSimulationSolver::SetLocalSpaceScale(FReal InLocalSpaceScale, bool bReset)
{
	LocalSpaceScale = InLocalSpaceScale;
	if (bReset)
	{
		OldLocalSpaceScale = InLocalSpaceScale;
	}
}

void FClothingSimulationSolver::SetCloths(TArray<FClothingSimulationCloth*>&& InCloths)
{
	// Remove old cloths
	RemoveCloths();

	// Update array
	Cloths = MoveTemp(InCloths);

	// Add the new cloths' particles
	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		check(Cloth);

		// Add the cloth's particles
		Cloth->Add(this);

		// Set initial state
		Cloth->PreUpdate(this);
		Cloth->Update(this);
	}

	// Update external collision's offset
	if (PBDEvolution)
	{
		CollisionParticlesOffset = PBDEvolution->CollisionParticles().Size();
	}
}

void FClothingSimulationSolver::AddCloth(FClothingSimulationCloth* InCloth)
{
	check(InCloth);

	if (Cloths.Find(InCloth) != INDEX_NONE)
	{
		return;
	}

	// Add the cloth to the solver update array
	Cloths.Emplace(InCloth);

	// Reset external collisions so that there is never any external collision particles below cloth's ones
	ResetCollisionParticles(CollisionParticlesOffset);

	// Add the cloth's particles
	InCloth->Add(this);

	// Set initial state
	InCloth->PreUpdate(this);
	InCloth->Update(this);

	// Update external collision's offset
	CollisionParticlesOffset = PBDEvolution ? PBDEvolution->CollisionParticles().Size() : 0;
}

void FClothingSimulationSolver::RemoveCloth(FClothingSimulationCloth* InCloth)
{
	if (Cloths.Find(InCloth) == INDEX_NONE)
	{
		return;
	}

	// Remove reference to this solver
	InCloth->Remove(this);

	// Remove collider from array
	Cloths.RemoveSwap(InCloth);

	// Reset all particles, collisions, constraints
	Reset();

	// Re-add the remaining cloths' particles
	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		// Add the cloth's particles
		Cloth->Add(this);

		// Set initial state
		Cloth->PreUpdate(this);
		Cloth->Update(this);
	}
	
	// Update external collision's offset
	CollisionParticlesOffset = PBDEvolution ? PBDEvolution->CollisionParticles().Size() : 0;
}

void FClothingSimulationSolver::RemoveCloths()
{
	// Remove all cloths from array
	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		Cloth->Remove(this);
	}
	Cloths.Reset();

	// Reset solver collisions
	// Reset cloth particles and associated elements
	Reset();
}

void FClothingSimulationSolver::RefreshCloth(FClothingSimulationCloth* InCloth)
{
	if (Cloths.Find(InCloth) == INDEX_NONE)
	{
		return;
	}

	// TODO: Add different ways to refresh cloths without recreating everything (collisions, constraints, particles)
	RefreshCloths();
}

void FClothingSimulationSolver::RefreshCloths()
{
	// Remove the cloths' & collisions' particles
	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		// Remove any solver data held by the cloth 
		Cloth->Remove(this);
	}

	// Reset evolution particles and constraints
	Reset();

	// Re-add the cloths' & collisions' particles
	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		// Re-Add the cloth's and collisions' particles
		Cloth->Add(this);

		// Set initial state
		Cloth->PreUpdate(this);
		Cloth->Update(this);
	}

	// Update solver collider's offset
	CollisionParticlesOffset = PBDEvolution ? PBDEvolution->CollisionParticles().Size() : 0;
}

void FClothingSimulationSolver::SetConfig(FClothingSimulationConfig* InConfig)
{
	// If the PropertyCollection is owned by this object, so does the current config object
	if (PropertyCollection.IsValid())
	{
		delete Config;
		PropertyCollection.Reset();
	}

	if (InConfig)
	{
		Config = InConfig;
	}
	else
	{
		// Create a default empty config object for coherence
		PropertyCollection = MakeShared<FManagedArrayCollection>();
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // TODO Remove these pragmas when the method is removed and the TArray can be correctly deducted again
		Config = new FClothingSimulationConfig({ PropertyCollection });
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FClothingSimulationSolver::SetSolverLOD(int32 LODIndex)
{
	checkfSlow(Config && Config->GetNumLODs(), TEXT("Config must be initialized first."));
	SolverLOD = FMath::Min(LODIndex, Config->GetNumLODs() - 1);
}

void FClothingSimulationSolver::Reset()
{
	if (Evolution)
	{
		constexpr bool bResetGlobalRules = false;
		Evolution->Reset(bResetGlobalRules);
		ClothsConstraints.Reset();
	}
	else
	{
		ResetParticles();
		ResetCollisionParticles();
	}
}

void FClothingSimulationSolver::ResetParticles()
{
	if (PBDEvolution)
	{
		PBDEvolution->ResetParticles();
		PBDEvolution->ResetConstraintRules();
		ClothsConstraints.Reset();
	}
}

void FClothingSimulationSolver::ResetCollisionParticles(int32 InCollisionParticlesOffset)
{
	if (PBDEvolution)
	{
		PBDEvolution->ResetCollisionParticles(InCollisionParticlesOffset);
		CollisionParticlesOffset = InCollisionParticlesOffset;
		CollisionParticlesSize = 0;
	}
}

int32 FClothingSimulationSolver::AddParticles(int32 NumParticles, uint32 GroupId)
{
	if (!NumParticles)
	{
		return INDEX_NONE;
	}
	constexpr bool bActivateFalse = false;
	const int32 ParticleRangeId = Evolution ? Evolution->AddSoftBody(GroupId, NumParticles, bActivateFalse) : 
		PBDEvolution->AddParticleRange(NumParticles, GroupId, bActivateFalse);

	// Add an empty constraints container for this range
	check(!ClothsConstraints.Find(ParticleRangeId));  // We cannot already have this ParticleRangeId in the map, particle ranges are always added, never removed (unless reset)

	if (Evolution)
	{
		ClothsConstraints.Emplace(ParticleRangeId, MakeUnique<FClothConstraints>())
			->Initialize(
				Evolution.Get(),
				&PerSolverField,
				InterpolatedAnimationPositions,
				InterpolatedAnimationNormals,
				AnimationVelocities,
				Normals,
				LastSubframeCollisionTransformsCCD,
				Collided,
				CollisionContacts,
				CollisionNormals,
				CollisionPhis,
				ParticleRangeId);
	}
	else
	{
		check(PBDEvolution);
		ClothsConstraints.Emplace(ParticleRangeId, MakeUnique<FClothConstraints>())
			->Initialize(PBDEvolution.Get(), InterpolatedAnimationPositions, OldAnimationPositions, InterpolatedAnimationNormals, AnimationVelocities, ParticleRangeId, NumParticles);
	}

	// Always starts with particles disabled
	EnableParticles(ParticleRangeId, false);

	return ParticleRangeId;
}

void FClothingSimulationSolver::EnableParticles(int32 ParticleRangeId, bool bEnable)
{
	if (Evolution)
	{
		Evolution->ActivateSoftBody(ParticleRangeId, bEnable);
	}
	else
	{
		PBDEvolution->ActivateParticleRange(ParticleRangeId, bEnable);
		GetClothConstraints(ParticleRangeId).Enable(bEnable);
	}
}

void FClothingSimulationSolver::ResetStartPose(int32 ParticleRangeId, int32 NumParticles)
{
	Softs::FPAndInvM* const PandInvMs = GetParticlePandInvMs(ParticleRangeId);
	Softs::FSolverVec3* const Xs = GetParticleXs(ParticleRangeId);
	Softs::FSolverVec3* const Vs = GetParticleVs(ParticleRangeId);
	const Softs::FSolverVec3* const Positions = GetAnimationPositions(ParticleRangeId);
	const Softs::FSolverVec3* const AnimNormals = GetAnimationNormals(ParticleRangeId);
	Softs::FSolverVec3* const OldPositions = GetOldAnimationPositions(ParticleRangeId);
	Softs::FSolverVec3* const InterpolatedPositions = GetInterpolatedAnimationPositions(ParticleRangeId);
	Softs::FSolverVec3* const AnimationVs = GetAnimationVelocities(ParticleRangeId);
	Softs::FSolverVec3* const OldAnimNormals = GetOldAnimationNormals(ParticleRangeId);
	Softs::FSolverVec3* const InterpolatedNormals = GetInterpolatedAnimationNormals(ParticleRangeId);

	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		PandInvMs[Index].P = Xs[Index] = OldPositions[Index] = InterpolatedPositions[Index] = Positions[Index];
		OldAnimNormals[Index] = InterpolatedNormals[Index] = AnimNormals[Index];
		Vs[Index] = AnimationVelocities[Index] = Softs::FSolverVec3(0.);
	}
}

const TArray<Softs::FPAndInvM>& FClothingSimulationSolver::GetParticlePandInvMs() const
{
	return Evolution ? Evolution->GetParticles().GetPAndInvM() : PBDEvolution->GetParticles().GetPAndInvM();
}

const TArray<Softs::FSolverVec3>& FClothingSimulationSolver::GetParticleXs() const
{
	return Evolution ? Evolution->GetParticles().XArray() : PBDEvolution->GetParticles().XArray();
}

const TArray<Softs::FSolverVec3>& FClothingSimulationSolver::GetParticleVs() const
{
	return Evolution ? Evolution->GetParticles().GetV() : PBDEvolution->GetParticles().GetV();
}

const TArray<Softs::FSolverReal>& FClothingSimulationSolver::GetParticleInvMasses() const
{
	return Evolution ? Evolution->GetParticles().GetInvM() : PBDEvolution->GetParticles().GetInvM();
}

TConstArrayView<Softs::FPAndInvM> FClothingSimulationSolver::GetParticlePandInvMsView(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetPAndInvM() : Private::GetPBDParticleConstArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetPAndInvM());
}

TArrayView<Softs::FPAndInvM> FClothingSimulationSolver::GetParticlePandInvMsView(int32 ParticleRangeId)
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetPAndInvM() : Private::GetPBDParticleArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetPAndInvM());
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetParticleXsView(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).XArray() : Private::GetPBDParticleConstArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().XArray());
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetParticleXsView(int32 ParticleRangeId)
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).XArray() : Private::GetPBDParticleArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().XArray());
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetParticleVsView(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetV() : Private::GetPBDParticleConstArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetV());
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetParticleVsView(int32 ParticleRangeId)
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetV() : Private::GetPBDParticleArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetV());
}

TConstArrayView<Softs::FSolverReal> FClothingSimulationSolver::GetParticleInvMassesView(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetInvM() : Private::GetPBDParticleConstArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetInvM());
}

TArrayView<Softs::FSolverReal> FClothingSimulationSolver::GetParticleInvMassesView(int32 ParticleRangeId)
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetInvM() : Private::GetPBDParticleArrayView(PBDEvolution, ParticleRangeId, PBDEvolution->GetParticles().GetInvM());
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetOldAnimationPositionsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, OldAnimationPositions); 
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetOldAnimationPositionsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, OldAnimationPositions);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationPositionsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationPositions);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationPositionsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationPositions);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetInterpolatedAnimationPositionsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, InterpolatedAnimationPositions);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetInterpolatedAnimationPositionsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, InterpolatedAnimationPositions);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetOldAnimationNormalsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, OldAnimationNormals);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetOldAnimationNormalsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, OldAnimationNormals);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationNormalsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationNormals);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationNormalsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationNormals);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetInterpolatedAnimationNormalsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, InterpolatedAnimationNormals);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetInterpolatedAnimationNormalsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, InterpolatedAnimationNormals);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetNormalsView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, Normals);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetNormalsView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, Normals);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationVelocitiesView(int32 ParticleRangeId) const
{
	return Private::GetParticleConstArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationVelocities);
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetAnimationVelocitiesView(int32 ParticleRangeId)
{
	return Private::GetParticleArrayView(Evolution, PBDEvolution, ParticleRangeId, AnimationVelocities);
}

int32 FClothingSimulationSolver::AddCollisionParticles(int32 NumCollisionParticles, uint32 GroupId, int32 RecycledCollisionRangeId)
{
	constexpr bool bActivateFalse = false;

	int32 CollisionRangeId = INDEX_NONE;
	if (Evolution)
	{
		if (Evolution->IsValidCollisionParticleRange(RecycledCollisionRangeId))
		{
			if (Evolution->GetCollisionParticleRange(RecycledCollisionRangeId).GetRangeSize() == NumCollisionParticles)
			{
				return RecycledCollisionRangeId;
			}
			// Size has changed. Remove the recycled collision range and reallocate some new particles
			Evolution->RemoveCollisionParticleRange(RecycledCollisionRangeId);
			ComplexColliders.Remove(RecycledCollisionRangeId);
			const TArray<int32>& GroupSoftBodies = Evolution->GetGroupSoftBodies(GroupId);
			for (int32 SoftBodyId : GroupSoftBodies)
			{
				if (TUniquePtr<FClothConstraints>* ClothConstraints = ClothsConstraints.Find(SoftBodyId))
				{
					check(ClothConstraints);
					(*ClothConstraints)->OnCollisionRangeRemoved(RecycledCollisionRangeId);
				}
			}
		}
		if (NumCollisionParticles)
		{
			CollisionRangeId = Evolution->AddCollisionParticleRange(GroupId, NumCollisionParticles, bActivateFalse);

			// Add an empty ComplexColliders
			ComplexColliders.Emplace(CollisionRangeId, MakeUnique<FClothComplexColliders>(Evolution.Get(), CollisionRangeId));
		}
	}
	else
	{
		// Try reusing the particle range
		// This is used by external collisions so that they can be added/removed between every solver update.
		// If it doesn't match then remove all ranges above the given offset to start again.
		// This rely on the assumption that these ranges are added again in the same update order.
		if (RecycledCollisionRangeId == CollisionParticlesOffset + CollisionParticlesSize)
		{
			CollisionParticlesSize += NumCollisionParticles;

			// Check that the range still exists
			if (CollisionParticlesOffset + CollisionParticlesSize <= (int32)PBDEvolution->CollisionParticles().Size() &&  // Check first that the range hasn't been reset
				NumCollisionParticles == PBDEvolution->GetCollisionParticleRangeSize(RecycledCollisionRangeId))  // This will assert if range has been reset
			{
				return RecycledCollisionRangeId;
			}
			// Size has changed. must reset this collision range (and all of those following up) and reallocate some new particles
			PBDEvolution->ResetCollisionParticles(RecycledCollisionRangeId);
		}

		if (NumCollisionParticles)
		{
			CollisionRangeId = PBDEvolution->AddCollisionParticleRange(NumCollisionParticles, GroupId, /*bActivate =*/ false);
		}
	}

	if (CollisionRangeId != INDEX_NONE)
	{
		// Always initialize the collision particle's transforms as otherwise setting the geometry will get NaNs detected during the bounding box updates
		Softs::FSolverRotation3* const Rs = GetCollisionParticleRs(CollisionRangeId);
		Softs::FSolverVec3* const Xs = GetCollisionParticleXs(CollisionRangeId);

		for (int32 Index = 0; Index < NumCollisionParticles; ++Index)
		{
			Xs[Index] = Softs::FSolverVec3(0.);
			Rs[Index] = Softs::FSolverRotation3::FromIdentity();
		}

		// Always starts with particles disabled
		EnableCollisionParticles(CollisionRangeId, false);
	}

	return CollisionRangeId;
}

void FClothingSimulationSolver::EnableCollisionParticles(int32 Offset, bool bEnable)
{
#if !UE_BUILD_SHIPPING
	const bool bFilteredEnable = bClothSolverDisableCollision ? false : bEnable;
#else
	const bool bFilteredEnable = bEnable;
#endif
	if (Evolution)
	{
		Evolution->ActivateCollisionParticleRange(Offset, bFilteredEnable);
	}
	else
	{
		check(PBDEvolution);
		PBDEvolution->ActivateCollisionParticleRange(Offset, bFilteredEnable);
	}
}

void FClothingSimulationSolver::ResetCollisionStartPose(int32 CollisionRangeId, int32 NumCollisionParticles)
{
	const Softs::FSolverRigidTransform3* const Transforms = GetCollisionTransforms(CollisionRangeId);
	Softs::FSolverRigidTransform3* const OldTransforms = GetOldCollisionTransforms(CollisionRangeId);
	Softs::FSolverRotation3* const Rs = GetCollisionParticleRs(CollisionRangeId);
	Softs::FSolverVec3* const Xs = GetCollisionParticleXs(CollisionRangeId);
	Softs::FSolverRigidTransform3* const LastSubframeCollisionTransformsCCDs = GetLastSubframeCollisionTransformsCCD(CollisionRangeId);

	for (int32 Index = 0; Index < NumCollisionParticles; ++Index)
	{
		OldTransforms[Index] = Transforms[Index];
		Xs[Index] = Transforms[Index].GetTranslation();
		Rs[Index] = Transforms[Index].GetRotation();
		LastSubframeCollisionTransformsCCDs[Index] = Transforms[Index];
	}
}

TConstArrayView<int32> FClothingSimulationSolver::GetCollisionBoneIndicesView(int32 CollisionRangeId) const
{
	return Private::GetCollisionParticleConstArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionBoneIndices);
}

TArrayView<int32> FClothingSimulationSolver::GetCollisionBoneIndicesView(int32 CollisionRangeId)
{
	return Private::GetCollisionParticleArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionBoneIndices);
}

TConstArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetCollisionBaseTransformsView(int32 CollisionRangeId) const
{
	return Private::GetCollisionParticleConstArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionBaseTransforms);
}

TArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetCollisionBaseTransformsView(int32 CollisionRangeId)
{
	return Private::GetCollisionParticleArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionBaseTransforms);
}

TConstArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetOldCollisionTransformsView(int32 CollisionRangeId) const
{
	return Private::GetCollisionParticleConstArrayView(Evolution, PBDEvolution, CollisionRangeId, OldCollisionTransforms);
}

TArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetOldCollisionTransformsView(int32 CollisionRangeId)
{
	return Private::GetCollisionParticleArrayView(Evolution, PBDEvolution, CollisionRangeId, OldCollisionTransforms);
}

TConstArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetCollisionTransformsView(int32 CollisionRangeId) const
{
	return Private::GetCollisionParticleConstArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionTransforms);
}

TArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetCollisionTransformsView(int32 CollisionRangeId)
{
	return Private::GetCollisionParticleArrayView(Evolution, PBDEvolution, CollisionRangeId, CollisionTransforms);
}

TConstArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetCollisionParticleXsView(int32 CollisionRangeId) const
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).XArray() : Private::GetPBDCollisionParticleConstArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->CollisionParticles().XArray());
}

TArrayView<Softs::FSolverVec3> FClothingSimulationSolver::GetCollisionParticleXsView(int32 CollisionRangeId)
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).XArray() : Private::GetPBDCollisionParticleArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->CollisionParticles().XArray());
}

TConstArrayView<Softs::FSolverRotation3> FClothingSimulationSolver::GetCollisionParticleRsView(int32 CollisionRangeId) const
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).GetR() : Private::GetPBDCollisionParticleConstArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->CollisionParticles().GetR());
}

TArrayView<Softs::FSolverRotation3> FClothingSimulationSolver::GetCollisionParticleRsView(int32 CollisionRangeId)
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).GetR() : Private::GetPBDCollisionParticleArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->CollisionParticles().GetR());
}

TConstArrayView<FImplicitObjectPtr> FClothingSimulationSolver::GetCollisionGeometryView(int32 CollisionRangeId) const
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).GetAllGeometry() : Private::GetPBDCollisionParticleConstArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->CollisionParticles().GetAllGeometry());
}

TConstArrayView<bool> FClothingSimulationSolver::GetCollisionStatusView(int32 CollisionRangeId) const
{
	return Evolution ? Evolution->GetCollisionParticleRange(CollisionRangeId).GetConstArrayView(Collided) : Private::GetPBDCollisionParticleConstArrayView(PBDEvolution, CollisionRangeId, PBDEvolution->GetCollisionStatus());
}

TConstArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetLastSubframeCollisionTransformsCCDView(int32 CollisionRangeId) const
{
	return Private::GetCollisionParticleConstArrayView(Evolution, PBDEvolution, CollisionRangeId, LastSubframeCollisionTransformsCCD);
}

TArrayView<Softs::FSolverRigidTransform3> FClothingSimulationSolver::GetLastSubframeCollisionTransformsCCDView(int32 CollisionRangeId)
{
	return Private::GetCollisionParticleArrayView(Evolution, PBDEvolution, CollisionRangeId, LastSubframeCollisionTransformsCCD);
}

void FClothingSimulationSolver::SetCollisionGeometry(int32 CollisionRangeId, int32 Index, FImplicitObjectPtr&& Geometry)
{
	if (Evolution)
	{
		Evolution->GetCollisionParticleRange(CollisionRangeId).SetGeometry(Index, MoveTemp(Geometry));
	}
	else
	{
		PBDEvolution->CollisionParticles().SetGeometry(CollisionRangeId + Index, MoveTemp(Geometry));
	}
}

const TArray<Softs::FSolverVec3>& FClothingSimulationSolver::GetCollisionContacts() const
{
	return Evolution ? CollisionContacts : PBDEvolution->GetCollisionContacts();
}

const TArray<Softs::FSolverReal>& FClothingSimulationSolver::GetCollisionPhis() const
{
	return Evolution ? CollisionPhis : PBDEvolution->GetCollisionPhis();
}

const TArray<Softs::FSolverVec3>& FClothingSimulationSolver::GetCollisionNormals() const
{
	return Evolution ? CollisionNormals : PBDEvolution->GetCollisionNormals();
}

void FClothingSimulationSolver::SetParticleMassUniform(int32 ParticleRangeId, const FVector2f& UniformMass, const TConstArrayView<FRealSingle>& UniformMassMultipliers, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate)
{
	if (Evolution)
	{
		Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(ParticleRangeId);

		const Softs::FPBDFlatWeightMapView UniformMasses(UniformMass, UniformMassMultipliers, Particles.GetRangeSize());

		// Set mass from uniform mass
		const TSet<int32> Vertices = Mesh.GetVertices();
		for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
		{
			Particles.M(Index) = Vertices.Contains(Index) ? UniformMasses.GetValue(Index) : (Softs::FSolverReal)0.;
		}
		ParticleMassClampAndKinematicStateUpdate(Particles, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
	else
	{
		// Retrieve the particle block size
		const int32 Size = PBDEvolution->GetParticleRangeSize(ParticleRangeId);
		const Softs::FPBDFlatWeightMapView UniformMasses(UniformMass, UniformMassMultipliers, Size);

		// Set mass from uniform mass
		const TSet<int32> Vertices = Mesh.GetVertices();
		Softs::FSolverParticles& Particles = PBDEvolution->Particles();
		for (int32 Index = 0; Index < Size; ++Index)
		{
			const int32 IndexWithOffset = Index + ParticleRangeId;
			Particles.M(IndexWithOffset) = Vertices.Contains(IndexWithOffset) ? UniformMasses.GetValue(Index) : (Softs::FSolverReal)0.;
		}
		ParticleMassClampAndKinematicStateUpdate(ParticleRangeId, Size, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
}

void FClothingSimulationSolver::SetParticleMassFromTotalMass(int32 ParticleRangeId, FRealSingle TotalMass, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate)
{
	if (Evolution)
	{
		Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(ParticleRangeId);

		// Set mass per area
		const Softs::FSolverReal TotalArea = SetParticleMassPerArea(Particles, Mesh);

		// Find density
		const Softs::FSolverReal Density = TotalArea > (Softs::FSolverReal)0. ? (Softs::FSolverReal)TotalMass / TotalArea : (Softs::FSolverReal)1.;

		// Update mass from mesh and density
		ParticleMassUpdateDensity(Particles, Mesh, Softs::FSolverVec2(Density), TConstArrayView<FRealSingle>());

		ParticleMassClampAndKinematicStateUpdate(Particles, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
	else
	{
		// Retrieve the particle block size
		const int32 Size = PBDEvolution->GetParticleRangeSize(ParticleRangeId);

		// Set mass per area
		const Softs::FSolverReal TotalArea = SetParticleMassPerArea(ParticleRangeId, Size, Mesh);

		// Find density
		const Softs::FSolverReal Density = TotalArea > (Softs::FSolverReal)0. ? (Softs::FSolverReal)TotalMass / TotalArea : (Softs::FSolverReal)1.;

		// Update mass from mesh and density
		ParticleMassUpdateDensity(Mesh, ParticleRangeId, Size, Softs::FSolverVec2(Density), TConstArrayView<FRealSingle>());

		ParticleMassClampAndKinematicStateUpdate(ParticleRangeId, Size, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
}

void FClothingSimulationSolver::SetParticleMassFromDensity(int32 ParticleRangeId, const FVector2f& Density, const TConstArrayView<float>& DensityMultipliers, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate)
{
	if (Evolution)
	{
		Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(ParticleRangeId);

		// Set mass per area
		const Softs::FSolverReal TotalArea = SetParticleMassPerArea(Particles, Mesh);

		// Set density from cm2 to m2
		const Softs::FSolverVec2 DensityScaled = Density / FMath::Square(ClothingSimulationSolverConstant::WorldScale);

		// Update mass from mesh and density
		ParticleMassUpdateDensity(Particles, Mesh, DensityScaled, DensityMultipliers);

		ParticleMassClampAndKinematicStateUpdate(Particles, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
	else
	{
		// Retrieve the particle block size
		const int32 Size = PBDEvolution->GetParticleRangeSize(ParticleRangeId);

		// Set mass per area
		const Softs::FSolverReal TotalArea = SetParticleMassPerArea(ParticleRangeId, Size, Mesh);

		// Set density from cm2 to m2
		const Softs::FSolverVec2 DensityScaled = Density / FMath::Square(ClothingSimulationSolverConstant::WorldScale);

		// Update mass from mesh and density
		ParticleMassUpdateDensity(Mesh, ParticleRangeId, Size, DensityScaled, DensityMultipliers);

		ParticleMassClampAndKinematicStateUpdate(ParticleRangeId, Size, (Softs::FSolverReal)MinPerParticleMass, KinematicPredicate);
	}
}

void FClothingSimulationSolver::SetReferenceVelocityScale(uint32 GroupId,
	const FRigidTransform3& OldReferenceSpaceTransform,
	const FRigidTransform3& ReferenceSpaceTransform,
	TVec3<FReal>& InOutAppliedReferenceVelocity, // Old reference velocity is passed in. New reference velocity is returned.
	TVec3<FReal>& InOutAppliedReferenceAngularVelocity, // Old reference velocity is passed in. New reference velocity is returned.
	const EChaosSoftsSimulationSpace VelocityScaleSpace, // the space the following linear velocity properties are in.
	const TVec3<FRealSingle>& LinearVelocityScale,
	const TVec3<FRealSingle>& MaxLinearVelocity,
	const TVec3<FRealSingle>& MaxLinearAcceleration,
	FRealSingle AngularVelocityScale,
	FRealSingle MaxAngularVelocity,
	FRealSingle MaxAngularAcceleration,
	FRealSingle FictitiousAngularScale,
	FRealSingle MaxVelocityScale,
	EFictitiousForcesModel FictitiousForcesModel
)
{
	FRigidTransform3 OldRootBoneLocalTransform = OldReferenceSpaceTransform;
	OldRootBoneLocalTransform.AddToTranslation(-OldLocalSpaceLocation);
	check(OldLocalSpaceScale > UE_SMALL_NUMBER);
	const FReal OldLocalSpaceScaleInv = 1. / OldLocalSpaceScale;
	OldRootBoneLocalTransform.ScaleTranslation(OldLocalSpaceScaleInv);

	const FVec3 OldAppliedReferenceVelocity(InOutAppliedReferenceVelocity); // In world space
	const FVec3 OldAppliedReferenceAngularVelocity(InOutAppliedReferenceAngularVelocity); // In world space

	const FReal SolverVelocityScale = bClothSolverUseVelocityScale ? VelocityScale : (FReal)1.;

	auto CalculateClampedVelocityScale = [MaxVelocityScale, SolverVelocityScale](FRealSingle InVelocityScale)
	{
		FReal CombinedVelocityScale = SolverVelocityScale * InVelocityScale;
		if (CombinedVelocityScale <= FMath::Min((FReal)1., (FReal)MaxVelocityScale))
		{
			// When combined velocity scale is <= 1 (or smaller max velocity scale), just use it.
			return CombinedVelocityScale;
		}
		// Otherwise, clamp SolverVelocityScale (what's calculated when doing delta time smoothing) to 1, and apply MaxVelocityScale clamp on the user supplied value.
		return FMath::Clamp(SolverVelocityScale, (FReal)0., (FReal)1.) * FMath::Clamp(InVelocityScale, (FReal)0., (FReal)MaxVelocityScale);
	};

	// Calculate deltas
	const FRotation3 OldReferenceSpaceRotationInverse = OldReferenceSpaceTransform.GetRotation().Inverse();
	const FVec3 WorldDeltaTranslation = FRigidTransform3::SubtractTranslations(ReferenceSpaceTransform, OldReferenceSpaceTransform);
	const FRotation3 FullDeltaRotationQ = OldReferenceSpaceRotationInverse * ReferenceSpaceTransform.GetRotation();
	FReal FullDeltaAngle = FullDeltaRotationQ.GetAngle();
	const FVec3 FullAxis = FullDeltaRotationQ.GetRotationAxis();
	if (FullDeltaAngle > (FReal)PI)
	{
		FullDeltaAngle -= (FReal)2. * (FReal)PI;
	}	
	const FVec3 FullDeltaRotation = FullDeltaAngle * FullAxis;

	// Choose which space to do linear velocity scale and clamp calculations based on VelocityScaleSpace
	FRotation3 VelocityScaleSpaceToOldRefSpace;
	FVec3 VelocityScaleSpaceDeltaTranslation;
	switch (VelocityScaleSpace)
	{
	case EChaosSoftsSimulationSpace::WorldSpace:
		VelocityScaleSpaceToOldRefSpace = OldReferenceSpaceRotationInverse;
		VelocityScaleSpaceDeltaTranslation = WorldDeltaTranslation;
		break;
	case EChaosSoftsSimulationSpace::ComponentSpace:
		VelocityScaleSpaceToOldRefSpace = OldReferenceSpaceRotationInverse * LocalSpaceRotation;
		VelocityScaleSpaceDeltaTranslation = LocalSpaceRotation.UnrotateVector(WorldDeltaTranslation);
		break;
	case EChaosSoftsSimulationSpace::ReferenceBoneSpace:
	default:
		VelocityScaleSpaceToOldRefSpace = FRotation3::Identity;
		VelocityScaleSpaceDeltaTranslation = OldReferenceSpaceRotationInverse * WorldDeltaTranslation;
		break;
	}

	// Apply linear velocity scale.
	const FVec3 LinearRatio = FVec3(
		CalculateClampedVelocityScale(LinearVelocityScale[0]),
		CalculateClampedVelocityScale(LinearVelocityScale[1]),
		CalculateClampedVelocityScale(LinearVelocityScale[2]));

	FVec3 AppliedDeltaTranslation = LinearRatio * VelocityScaleSpaceDeltaTranslation;

	// Apply angular velocity scale
	const FReal AngularRatio = CalculateClampedVelocityScale(AngularVelocityScale);
	FVec3 AppliedDeltaRotation = AngularRatio * FullDeltaRotation;

	// Apply clamps (since we're ultimately just modifying a single transform, remove accelerations from the velocity first, then clamp velocities).
	if (DeltaTime > 0.f)
	{	
		FVec3 AppliedLinearVelocity = AppliedDeltaTranslation / DeltaTime;
		FVec3 AppliedAngularVelocity = AppliedDeltaRotation / DeltaTime;

		// Apply linear acceleration clamping
		if (MaxLinearAcceleration != TVec3<FRealSingle>(TNumericLimits<FRealSingle>::Max()))
		{
			FVec3 OldScaledVelocity;
			switch (VelocityScaleSpace)
			{
			case EChaosSoftsSimulationSpace::WorldSpace:
				OldScaledVelocity = OldAppliedReferenceVelocity;
				break;
			case EChaosSoftsSimulationSpace::ComponentSpace:
				OldScaledVelocity = LocalSpaceRotation.UnrotateVector(OldAppliedReferenceVelocity);
				break;
			case EChaosSoftsSimulationSpace::ReferenceBoneSpace:
			default:
				OldScaledVelocity = (OldReferenceSpaceRotationInverse * OldAppliedReferenceVelocity);
				break;
			}
			
			const FVec3 ScaledAccelerationTimesDt = AppliedLinearVelocity - OldScaledVelocity;
			for (int32 Index = 0; Index < 3; ++Index)
			{
				if (MaxLinearAcceleration[Index] != TNumericLimits<FRealSingle>::Max() && MaxLinearAcceleration[Index] >= 0.f
					&& FMath::Abs(ScaledAccelerationTimesDt[Index]) > (FReal)(MaxLinearAcceleration[Index] * DeltaTime * LocalSpaceScale))
				{
					AppliedLinearVelocity[Index] = OldScaledVelocity[Index] + FMath::Sign(ScaledAccelerationTimesDt[Index]) * (FReal)(MaxLinearAcceleration[Index] * DeltaTime * LocalSpaceScale);
				}
			}
		}

		// Apply linear velocity clamping 
		if (MaxLinearVelocity != TVec3<FRealSingle>(TNumericLimits<FRealSingle>::Max()))
		{
			for (int32 Index = 0; Index < 3; ++Index)
			{
				if (MaxLinearVelocity[Index] != TNumericLimits<FRealSingle>::Max() && MaxLinearVelocity[Index] >= 0.f &&
					FMath::Abs(AppliedLinearVelocity[Index]) > (FReal)MaxLinearVelocity[Index] * LocalSpaceScale)
				{
					AppliedLinearVelocity[Index] = FMath::Sign(AppliedLinearVelocity[Index]) * (FReal)MaxLinearVelocity[Index] * LocalSpaceScale;
				}
			}
		}

		// Temporary cvar to return to previous broken behavior where previous reference velocity wasn't stored.
		if (!bClothSolverBrokenAccelerationClampingToZero)
		{
			// Only update old reference velocity when ticking.
			switch (VelocityScaleSpace)
			{
			case EChaosSoftsSimulationSpace::WorldSpace:
				InOutAppliedReferenceVelocity = AppliedLinearVelocity;
				break;
			case EChaosSoftsSimulationSpace::ComponentSpace:
				InOutAppliedReferenceVelocity = LocalSpaceRotation.RotateVector(AppliedLinearVelocity);
				break;
			case EChaosSoftsSimulationSpace::ReferenceBoneSpace:
			default:
				InOutAppliedReferenceVelocity = OldReferenceSpaceTransform.GetRotation().RotateVector(AppliedLinearVelocity);
				break;
			}
		}

		AppliedDeltaTranslation = AppliedLinearVelocity * DeltaTime;

		// Apply angular acceleration clamping
		if (MaxAngularAcceleration != TNumericLimits<FRealSingle>::Max() && MaxAngularAcceleration >= 0.f)
		{
			const FVec3 OldScaledAngularVelocity = OldAppliedReferenceAngularVelocity;
			const FVec3 ScaledReferenceSpaceAngularAccelerationTimesDt = AppliedAngularVelocity - OldScaledAngularVelocity;
			const FReal LenSq = ScaledReferenceSpaceAngularAccelerationTimesDt.SquaredLength();
			if (LenSq > FMath::Square(MaxAngularAcceleration * DeltaTime))
			{
				const FReal Scale = MaxAngularAcceleration * DeltaTime * FMath::InvSqrt(LenSq);
				AppliedAngularVelocity = OldScaledAngularVelocity + Scale * ScaledReferenceSpaceAngularAccelerationTimesDt;
			}
		}
		// Apply angular velocity clamping
		if (MaxAngularVelocity != TNumericLimits<FRealSingle>::Max() && MaxAngularVelocity >= 0.f)
		{
			const FReal LenSq = AppliedAngularVelocity.SquaredLength();
			if (LenSq > FMath::Square(MaxAngularVelocity))
			{
				const FReal Scale = MaxAngularVelocity * FMath::InvSqrt(LenSq);
				AppliedAngularVelocity *= Scale;
			}
		}
		// Only update old reference velocity when ticking.
		InOutAppliedReferenceAngularVelocity = AppliedAngularVelocity;

		AppliedDeltaRotation = AppliedAngularVelocity * DeltaTime;
	}

	check(LocalSpaceScale > UE_SMALL_NUMBER);
	const FReal LocalSpaceScaleInv = 1. / LocalSpaceScale;
	const FVec3 DeltaPosition = VelocityScaleSpaceToOldRefSpace * (LocalSpaceScaleInv * (VelocityScaleSpaceDeltaTranslation - AppliedDeltaTranslation));
	const FVec3 DeltaRotation = FullDeltaRotation - AppliedDeltaRotation;
	const FReal DeltaAngle = DeltaRotation.Length();
	const FVec3 Axis = DeltaAngle > UE_KINDA_SMALL_NUMBER ? DeltaRotation / DeltaAngle : FVec3::XAxisVector;

	const UE::Math::TQuat<FReal> DeltaRotationQuat(Axis, DeltaAngle);

	// Transform points back into the previous frame of reference before applying the adjusted deltas
	const FRigidTransform3 PreSimulationTransform = OldRootBoneLocalTransform.Inverse() * FRigidTransform3(DeltaPosition, DeltaRotationQuat) * OldRootBoneLocalTransform;
	PreSimulationTransforms[GroupId] = Softs::FSolverRigidTransform3(  // Store the delta in solver precision, no need for LWC here
		Softs::FSolverVec3(PreSimulationTransform.GetTranslation()),
		Softs::FSolverRotation3(PreSimulationTransform.GetRotation()));

	// Fictitious angular scale only applied for PBDEvolution. It's applied by ExternalForces for Evolution.
	const FReal AppliedFictitiousAngularScale = (FictitiousForcesModel == EFictitiousForcesModel::None) ? (FReal)0.f : (PBDEvolution ? FMath::Min((FReal)2., (FReal)FictitiousAngularScale) : (FReal)1.);

	FVec3 FictitiousAngularDisplacement;
	switch (FictitiousForcesModel)
	{
	default:
	case EFictitiousForcesModel::None:
		FictitiousAngularDisplacement = FVec3(0);
		break;
	case EFictitiousForcesModel::Legacy:
		FictitiousAngularDisplacement = ReferenceSpaceTransform.TransformVector(DeltaRotation) * AppliedFictitiousAngularScale;
		break;
	case EFictitiousForcesModel::AppliedScale:
		FictitiousAngularDisplacement = ReferenceSpaceTransform.TransformVector(AppliedDeltaRotation) * AppliedFictitiousAngularScale;
		break;
	case EFictitiousForcesModel::FullScale:
		FictitiousAngularDisplacement = ReferenceSpaceTransform.TransformVector(FullDeltaRotation) * AppliedFictitiousAngularScale;
		break;
	}

	// Save the reference bone relative angular velocity for calculating the fictitious forces
	const bool bHasNonZeroDeltaTime = (DeltaTime > (Softs::FSolverReal)0.f);
	FictitiousAngularVelocities[GroupId] = bHasNonZeroDeltaTime ? Softs::FSolverVec3(FictitiousAngularDisplacement) / DeltaTime : Softs::FSolverVec3(0.f);
	ReferenceSpaceLocations[GroupId] = LocalSpaceScaleInv * (ReferenceSpaceTransform.GetLocation() - LocalSpaceLocation);
	AppliedReferenceSpaceVelocities[GroupId] = bHasNonZeroDeltaTime ? FSolverVec3(LocalSpaceScaleInv * InOutAppliedReferenceVelocity) : Softs::FSolverVec3(0.f);
	AppliedReferenceSpaceAngularVelocities[GroupId] = bHasNonZeroDeltaTime ? FSolverVec3(InOutAppliedReferenceAngularVelocity) : Softs::FSolverVec3(0.f);
#if CHAOS_DEBUG_DRAW
	OriginalReferenceSpaceVelocities[GroupId] = bHasNonZeroDeltaTime ? FSolverVec3(LocalSpaceScaleInv * WorldDeltaTranslation / DeltaTime) : Softs::FSolverVec3(0.f);
	OriginalReferenceSpaceAngularVelocities[GroupId] = bHasNonZeroDeltaTime ? FSolverVec3(FullDeltaRotation / DeltaTime) : Softs::FSolverVec3(0.f);
#endif
}

Softs::FSolverReal FClothingSimulationSolver::SetParticleMassPerArea(Softs::FSolverParticlesRange& Particles, const FTriangleMesh& Mesh)
{
	// Zero out masses
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		Particles.M(Index) = (Softs::FSolverReal)0.;
	}

	// Assign per particle mass proportional to connected area.
	const TArray<TVec3<int32>>& SurfaceElements = Mesh.GetSurfaceElements();
	Softs::FSolverReal TotalArea = (Softs::FSolverReal)0.;
	for (const TVec3<int32>& Tri : SurfaceElements)
	{
		const Softs::FSolverReal TriArea = (Softs::FSolverReal)0.5 * Softs::FSolverVec3::CrossProduct(
			Particles.X(Tri[1]) - Particles.X(Tri[0]),
			Particles.X(Tri[2]) - Particles.X(Tri[0])).Size();
		TotalArea += TriArea;
		const Softs::FSolverReal ThirdTriArea = TriArea / (Softs::FSolverReal)3.;
		Particles.M(Tri[0]) += ThirdTriArea;
		Particles.M(Tri[1]) += ThirdTriArea;
		Particles.M(Tri[2]) += ThirdTriArea;
	}

	UE_LOGF(LogChaosCloth, Verbose, "Total area: %f, SI total area: %f", TotalArea, TotalArea / FMath::Square(ClothingSimulationSolverConstant::WorldScale));
	return TotalArea;

}

void FClothingSimulationSolver::ParticleMassUpdateDensity(Softs::FSolverParticlesRange& Particles, const FTriangleMesh& Mesh, const Softs::FSolverVec2& Density, const TConstArrayView<float>& DensityMultipliers)
{
	const Softs::FPBDFlatWeightMapView Densities(Density, DensityMultipliers, Particles.GetRangeSize());
	const TSet<int32> Vertices = Mesh.GetVertices();
	FReal TotalMass = 0.f;
	for (const int32 Vertex : Vertices)
	{
		Particles.M(Vertex) *= Densities.GetValue(Vertex);
		TotalMass += Particles.M(Vertex);
	}

	UE_LOGF(LogChaosCloth, Verbose, "Total mass: %f, ", TotalMass);
}

void FClothingSimulationSolver::ParticleMassClampAndKinematicStateUpdate(Softs::FSolverParticlesRange& Particles, Softs::FSolverReal MinPerParticleMass, const TFunctionRef<bool(int32)>& KinematicPredicate)
{
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		Particles.M(Index) = FMath::Max(Particles.M(Index), MinPerParticleMass);
		Particles.InvM(Index) = KinematicPredicate(Index) ? (Softs::FSolverReal)0. : (Softs::FSolverReal)1. / Particles.M(Index);
	}
}

Softs::FSolverReal FClothingSimulationSolver::SetParticleMassPerArea(int32 Offset, int32 Size, const FTriangleMesh& Mesh)
{
	check(PBDEvolution);

	// Zero out masses
	Softs::FSolverParticles& Particles = PBDEvolution->Particles();
	for (int32 Index = Offset; Index < Offset + Size; ++Index)
	{
		Particles.M(Index) = (Softs::FSolverReal)0.;
	}

	// Assign per particle mass proportional to connected area.
	const TArray<TVec3<int32>>& SurfaceElements = Mesh.GetSurfaceElements();
	Softs::FSolverReal TotalArea = (Softs::FSolverReal)0.;
	for (const TVec3<int32>& Tri : SurfaceElements)
	{
		const Softs::FSolverReal TriArea = (Softs::FSolverReal)0.5 * Softs::FSolverVec3::CrossProduct(
			Particles.X(Tri[1]) - Particles.X(Tri[0]),
			Particles.X(Tri[2]) - Particles.X(Tri[0])).Size();
		TotalArea += TriArea;
		const Softs::FSolverReal ThirdTriArea = TriArea / (Softs::FSolverReal)3.;
		Particles.M(Tri[0]) += ThirdTriArea;
		Particles.M(Tri[1]) += ThirdTriArea;
		Particles.M(Tri[2]) += ThirdTriArea;
	}

	UE_LOGF(LogChaosCloth, Verbose, "Total area: %f, SI total area: %f", TotalArea, TotalArea / FMath::Square(ClothingSimulationSolverConstant::WorldScale));
	return TotalArea;
}

void FClothingSimulationSolver::ParticleMassUpdateDensity(const FTriangleMesh& Mesh, int32 Offset, int32 Size, const Softs::FSolverVec2& Density, const TConstArrayView<float>& DensityMultipliers)
{
	check(PBDEvolution);

	const Softs::FPBDFlatWeightMapView Densities(Density, DensityMultipliers, Size);
	const TSet<int32> Vertices = Mesh.GetVertices();
	Softs::FSolverParticles& Particles = PBDEvolution->Particles();
	FReal TotalMass = 0.f;
	for (const int32 Vertex : Vertices)
	{
		Particles.M(Vertex) *= Densities.GetValue(Vertex - Offset);
		TotalMass += Particles.M(Vertex);
	}

	UE_LOGF(LogChaosCloth, Verbose, "Total mass: %f, ", TotalMass);
}

void FClothingSimulationSolver::ParticleMassClampAndKinematicStateUpdate(int32 Offset, int32 Size, Softs::FSolverReal MinPerParticleMass, const TFunctionRef<bool(int32)>& KinematicPredicate)
{
	check(PBDEvolution);

	Softs::FSolverParticles& Particles = PBDEvolution->Particles();
	for (int32 Index = Offset; Index < Offset + Size; ++Index)
	{
		Particles.M(Index) = FMath::Max(Particles.M(Index), MinPerParticleMass);
		Particles.InvM(Index) = KinematicPredicate(Index - Offset) ? (Softs::FSolverReal)0. : (Softs::FSolverReal)1. / Particles.M(Index);
	}
}

void FClothingSimulationSolver::SetProperties(int32 ParticleRangeId, const Softs::FCollectionPropertyConstFacade& InPropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (Evolution)
	{
		const uint32 GroupId = Evolution->GetSoftBodyGroupId(ParticleRangeId);
		Evolution->SetSoftBodyProperties(ParticleRangeId, InPropertyCollection, WeightMaps, ReferenceSpaceLocations[GroupId], AppliedReferenceSpaceVelocities[GroupId], AppliedReferenceSpaceAngularVelocities[GroupId]);

		// Set properties to constraints that come from the solver (e.g., solver-level gravity, wind)
		GetClothConstraints(ParticleRangeId).UpdateFromSolver(
			Gravity, bIsClothGravityOverrideEnabled,
			FictitiousAngularVelocities[GroupId], ReferenceSpaceLocations[GroupId],
			WindVelocity, LegacyWindAdaption);
	}
}

void FClothingSimulationSolver::SetProperties(uint32 GroupId, FRealSingle DampingCoefficient, FRealSingle LocalDampingCoefficient, FRealSingle CollisionThickness, FRealSingle FrictionCoefficient)
{
	if (PBDEvolution)
	{
		PBDEvolution->SetDamping(DampingCoefficient, GroupId);
		PBDEvolution->SetLocalDamping(LocalDampingCoefficient, GroupId);
		PBDEvolution->SetCollisionThickness(CollisionThickness, GroupId);
		PBDEvolution->SetCoefficientOfFriction(FrictionCoefficient, GroupId);
	}
}

void FClothingSimulationSolver::SetUseCCD(uint32 GroupId, bool bUseCCD)
{
	if (PBDEvolution)
	{
		PBDEvolution->SetUseCCD(bUseCCD, GroupId);
	}
}

void FClothingSimulationSolver::SetGravity(uint32 GroupId, const TVec3<FRealSingle>& InGravity)
{
	if (PBDEvolution)
	{
		PBDEvolution->SetGravity(Softs::FSolverVec3(InGravity), GroupId);
	}
}

void FClothingSimulationSolver::SetWindVelocity(const TVec3<FRealSingle>& InWindVelocity, FRealSingle InLegacyWindAdaption)
{
	WindVelocity = InWindVelocity * ClothingSimulationSolverConstant::WorldScale;
	LegacyWindAdaption = InLegacyWindAdaption;
}

int32 FClothingSimulationSolver::GetNumIterations() const
{
	return Config ? GetNumIterations(Config->GetProperties(SolverLOD), ClothingSimulationSolverDefault::NumIterations) :
		ClothingSimulationSolverDefault::NumIterations;
}

int32 FClothingSimulationSolver::GetMaxNumIterations() const
{
	return Config ?
		GetMaxNumIterations(Config->GetProperties(SolverLOD), ClothingSimulationSolverDefault::MaxNumIterations) :
		ClothingSimulationSolverDefault::MaxNumIterations;
}

int32 FClothingSimulationSolver::GetNumSubsteps() const
{
	return Config ?
		GetNumSubsteps(Config->GetProperties(SolverLOD), ClothingSimulationSolverDefault::NumSubsteps) :
		ClothingSimulationSolverDefault::NumSubsteps;
}

void FClothingSimulationSolver::SetWindVelocity(uint32 GroupId, const TVec3<FRealSingle>& InWindVelocity)
{
	if (PBDEvolution)
	{
		Softs::FVelocityAndPressureField& VelocityAndPressureField = PBDEvolution->GetVelocityAndPressureField(GroupId);
		VelocityAndPressureField.SetVelocity(Softs::FSolverVec3(InWindVelocity));
	}
}

void FClothingSimulationSolver::SetWindAndPressureGeometry(
	uint32 GroupId,
	const FTriangleMesh& TriangleMesh,
	const Softs::FCollectionPropertyConstFacade& InPropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (PBDEvolution)
	{
		Softs::FVelocityAndPressureField& VelocityAndPressureField = PBDEvolution->GetVelocityAndPressureField(GroupId);
		VelocityAndPressureField.SetGeometry(&TriangleMesh, InPropertyCollection, WeightMaps, ClothingSimulationSolverConstant::WorldScale);
	}
}

void FClothingSimulationSolver::SetWindAndPressureGeometry(
	uint32 GroupId,
	const FTriangleMesh& TriangleMesh,
	const TConstArrayView<FRealSingle>& DragMultipliers,
	const TConstArrayView<FRealSingle>& OuterDragMultipliers,
	const TConstArrayView<FRealSingle>& LiftMultipliers,
	const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
	const TConstArrayView<FRealSingle>& PressureMultipliers)
{
	if (PBDEvolution)
	{
		Softs::FVelocityAndPressureField& VelocityAndPressureField = PBDEvolution->GetVelocityAndPressureField(GroupId);
		VelocityAndPressureField.SetGeometry(&TriangleMesh, DragMultipliers, OuterDragMultipliers, LiftMultipliers, OuterLiftMultipliers, PressureMultipliers);
	}
}

void FClothingSimulationSolver::SetWindAndPressureProperties(
	uint32 GroupId,
	const Softs::FCollectionPropertyConstFacade& InPropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	bool bEnableAerodynamics)
{
	if (PBDEvolution)
	{
		Softs::FVelocityAndPressureField& VelocityAndPressureField = PBDEvolution->GetVelocityAndPressureField(GroupId);
		VelocityAndPressureField.SetProperties(InPropertyCollection, WeightMaps, ClothingSimulationSolverConstant::WorldScale, bEnableAerodynamics);
	}
}

void FClothingSimulationSolver::SetWindAndPressureProperties(
	uint32 GroupId,
	const TVec2<FRealSingle>& Drag,
	const TVec2<FRealSingle>& OuterDrag,
	const TVec2<FRealSingle>& Lift,
	const TVec2<FRealSingle>& OuterLift,
	FRealSingle FluidDensity,
	const TVec2<FRealSingle>& Pressure)
{
	if (PBDEvolution)
	{
		Softs::FVelocityAndPressureField& VelocityAndPressureField = PBDEvolution->GetVelocityAndPressureField(GroupId);
		VelocityAndPressureField.SetProperties(
			Drag,
			OuterDrag,
			Lift,
			OuterLift,
			FluidDensity / FMath::Cube(ClothingSimulationSolverConstant::WorldScale),  // Fluid density is given in kg/m^3. Need to convert to kg/cm^3 for solver.
			Pressure / ClothingSimulationSolverConstant::WorldScale);  // UI Pressure is in kg/m s^2. Need to convert to kg/cm s^2 for solver.
	}
}

const Softs::FVelocityAndPressureField& FClothingSimulationSolver::GetWindVelocityAndPressureField(uint32 GroupId) const
{
	if (Evolution)
	{
		// Return field from first active softbody in this group
		const TSet<int32>& ParticleRanges = Evolution->GetGroupActiveSoftBodies(GroupId);
		if (auto It = ParticleRanges.CreateConstIterator(); It)
		{
			const int32 ParticleRangeId = *It;
			const TSharedPtr<Softs::FVelocityAndPressureField>& VelocityField = GetClothConstraints(ParticleRangeId).GetVelocityAndPressureField();
			check(VelocityField);
			return *VelocityField.Get();
		}
		checkf(false, TEXT("No active velocity field found for this groupId %i"), GroupId);
		static Softs::FVelocityAndPressureField DummyField;
		return DummyField;
	}
	else
	{
		return PBDEvolution->GetVelocityAndPressureField(GroupId);
	}
}

void FClothingSimulationSolver::AddExternalForces(uint32 GroupId, bool bUseLegacyWind)
{
	if (PBDEvolution)
	{
		const FVec3& AngularVelocity = FictitiousAngularVelocities[GroupId];
		const FVec3& ReferenceSpaceLocation = ReferenceSpaceLocations[GroupId];
		const bool bHasFictitiousForces = !AngularVelocity.IsNearlyZero();

		static const FReal LegacyWindMultiplier = (FReal)25.;
		const FVec3 LegacyWindVelocity = WindVelocity * LegacyWindMultiplier;

		PBDEvolution->GetForceFunction(GroupId) =
			[this, bHasFictitiousForces, bUseLegacyWind, LegacyWindVelocity, AngularVelocity, ReferenceSpaceLocation](Softs::FSolverParticles& Particles, const FReal Dt, const int32 Index)
			{
				FVec3 Forces((FReal)0.);

				const TArray<FVector>& LinearVelocities = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearVelocity);
				if (!LinearVelocities.IsEmpty())
				{
					Forces += LinearVelocities[Index] * Particles.M(Index) / Dt;
				}

				const TArray<FVector>& LinearForces = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearForce);
				if (!LinearForces.IsEmpty())
				{
					Forces += LinearForces[Index];
				}

				if (bHasFictitiousForces)
				{
					const FVec3 X = Particles.X(Index) - ReferenceSpaceLocation;
					const FReal& M = Particles.M(Index);
#if 0
					// Coriolis + Centrifugal seems a bit overkilled, but let's keep the code around in case it's ever required
					const FVec3& V = Particles.V(Index);
					Forces -= (FVec3::CrossProduct(AngularVelocity, V) * 2.f + FVec3::CrossProduct(AngularVelocity, FVec3::CrossProduct(AngularVelocity, X))) * M;
#else
					// Centrifugal force
					Forces -= FVec3::CrossProduct(AngularVelocity, FVec3::CrossProduct(AngularVelocity, X)) * M;
#endif
				}
				
				if (bUseLegacyWind)
				{
					// Calculate wind velocity delta
					const FVec3 VelocityDelta = LegacyWindVelocity - Particles.V(Index);

					FVec3 Direction = VelocityDelta;
					if (Direction.Normalize())
					{
						// Scale by angle
						const FReal DirectionDot = FVec3::DotProduct(Direction, Normals[Index]);
						const FReal ScaleFactor = FMath::Min(1.f, FMath::Abs(DirectionDot) * LegacyWindAdaption);
						Forces += VelocityDelta * ScaleFactor * Particles.M(Index);
					}
				}

				Particles.Acceleration(Index) += FVector3f(Forces) * Particles.InvM(Index);
			};
	}
}

void FClothingSimulationSolver::ApplyPreSimulationTransforms()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_ApplyPreSimulationTransforms);
	check(FMath::IsNearlyEqual(OldLocalSpaceScale, LocalSpaceScale)); // Otherwise will need to apply a DeltaLocalScale.
	const Softs::FSolverVec3 DeltaLocalSpaceLocation((LocalSpaceLocation - OldLocalSpaceLocation) / LocalSpaceScale);  // LWC, note that LocalSpaceLocation is FReal based. but the delta can be stored in FSolverReal

	const FSolverReal MaxVelocitySquared = (ClothSolverMaxVelocity > 0.f) ? FMath::Square((FSolverReal)ClothSolverMaxVelocity) : TNumericLimits<FSolverReal>::Max();

	if (Evolution)
	{
		const TArray<uint32> ActiveGroups = Evolution->GetActiveGroupsArray();
		PhysicsParallelFor(ActiveGroups.Num(), [this, &ActiveGroups, &DeltaLocalSpaceLocation, MaxVelocitySquared](int32 ActiveGroupIndex)
			{
				const uint32 GroupId = ActiveGroups[ActiveGroupIndex];

				// Update particles
				const TSet<int32>& ActiveSoftBodies = Evolution->GetGroupActiveSoftBodies(GroupId);
				const Softs::FSolverRigidTransform3& GroupSpaceTransform = PreSimulationTransforms[GroupId];
				for (const int32 SoftBodyId : ActiveSoftBodies)
				{
					Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(SoftBodyId);
#if INTEL_ISPC
					if (bRealTypeCompatibleWithISPC && bChaos_PreSimulationTransforms_ISPC_Enabled)  // TODO: Make the ISPC works with both Single and Double depending on the FSolverReal type
					{
						if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
						{
							ispc::ApplyPreSimulationTransform(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(ispc::FVector3f*)Particles.GetV().GetData(),
								(ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FVector3f*)Particles.GetArrayView(OldAnimationPositions).GetData(),
								(ispc::FVector3f*)Particles.GetArrayView(AnimationVelocities).GetData(),
								Particles.GetInvM().GetData(),
								(const ispc::FVector3f*)Particles.GetConstArrayView(AnimationPositions).GetData(),
								(const ispc::FTransform3f&)GroupSpaceTransform,
								(const ispc::FVector3f&)DeltaLocalSpaceLocation,
								DeltaTime,
								Particles.GetRangeSize());
						}
						else
						{
							ispc::ApplyPreSimulationTransformAndClampVelocity(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(ispc::FVector3f*)Particles.GetV().GetData(),
								(ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FVector3f*)Particles.GetArrayView(OldAnimationPositions).GetData(),
								(ispc::FVector3f*)Particles.GetArrayView(AnimationVelocities).GetData(),
								Particles.GetInvM().GetData(),
								(const ispc::FVector3f*)Particles.GetConstArrayView(AnimationPositions).GetData(),
								(const ispc::FTransform3f&)GroupSpaceTransform,
								(ispc::FVector3f&)DeltaLocalSpaceLocation,
								DeltaTime,
								Particles.GetRangeSize(),
								MaxVelocitySquared);
						}
					}
					else
#endif
					{
						TArrayView<Softs::FSolverVec3> OldAnimationPositionsView = Particles.GetArrayView(OldAnimationPositions);
						TArrayView<Softs::FSolverVec3> AnimationVelocitiesView = Particles.GetArrayView(AnimationVelocities);
						TConstArrayView<Softs::FSolverVec3> AnimationPositionsView = Particles.GetConstArrayView(AnimationPositions);

						PhysicsParallelFor(Particles.GetRangeSize(),
							[this, &GroupSpaceTransform, &OldAnimationPositionsView, &AnimationVelocitiesView, &AnimationPositionsView, &DeltaLocalSpaceLocation, &Particles, MaxVelocitySquared](int32 Index)
							{

								// Update initial state for particles
								Particles.P(Index) = Particles.X(Index) = GroupSpaceTransform.TransformPositionNoScale(Particles.X(Index)) - DeltaLocalSpaceLocation;
								Particles.V(Index) = GroupSpaceTransform.TransformVector(Particles.V(Index));

								// Copy InvM over to PAndInvM
								Particles.PAndInvM(Index).InvM = Particles.InvM(Index);

								// Update anim initial state (target updated by skinning)
								OldAnimationPositionsView[Index] = GroupSpaceTransform.TransformPositionNoScale(OldAnimationPositionsView[Index]) - DeltaLocalSpaceLocation;

								// Update Animation velocities
								AnimationVelocitiesView[Index] = (AnimationPositionsView[Index] - OldAnimationPositionsView[Index]) / DeltaTime;

								// Clamp relative velocity
								const FSolverVec3 RelVelocity = Particles.V(Index) - AnimationVelocitiesView[Index];
								const FSolverReal RelVelocitySquaredLength = RelVelocity.SquaredLength();
								if (RelVelocitySquaredLength > MaxVelocitySquared)
								{
									Particles.V(Index) = AnimationVelocitiesView[Index] + RelVelocity * FMath::Sqrt(MaxVelocitySquared / RelVelocitySquaredLength);
								}
							}, Particles.GetRangeSize() < ClothSolverMinParallelBatchSize);
					}
				}

				// Update collision particles
				const TSet<int32>& ActiveCollisionRanges = Evolution->GetGroupActiveCollisionParticleRanges(GroupId);
				for (const int32 CollisionRangeId : ActiveCollisionRanges)
				{
					Softs::FSolverCollisionParticlesRange& CollisionParticles = Evolution->GetCollisionParticleRange(CollisionRangeId);
					TArrayView<Softs::FSolverRigidTransform3> OldCollisionTransformsView = CollisionParticles.GetArrayView(OldCollisionTransforms);
					for (int32 Index = 0; Index < CollisionParticles.GetRangeSize(); ++Index)
					{
						// Update initial state for collisions
						OldCollisionTransformsView[Index] = OldCollisionTransformsView[Index] * GroupSpaceTransform;
						OldCollisionTransformsView[Index].AddToTranslation(-DeltaLocalSpaceLocation);
						CollisionParticles.X(Index) = OldCollisionTransformsView[Index].GetTranslation();
						CollisionParticles.SetR(Index, OldCollisionTransformsView[Index].GetRotation());
					}
					if (FClothComplexColliders* const Colliders = GetComplexColliders(CollisionRangeId))
					{
						Colliders->ApplyPreSimulationTransforms(GroupSpaceTransform, DeltaLocalSpaceLocation, CollisionParticles.GetConstArrayView(OldCollisionTransforms),
							CollisionParticles.GetConstArrayView(CollisionTransforms), DeltaTime);
					}
				}
			}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPreUpdate);
	}
	else
	{
		const TPBDActiveView<Softs::FSolverParticles>& ParticlesActiveView = PBDEvolution->ParticlesActiveView();
		const TArray<uint32>& ParticleGroupIds = PBDEvolution->ParticleGroupIds();

		ParticlesActiveView.RangeFor(
			[this, &ParticleGroupIds, &DeltaLocalSpaceLocation, MaxVelocitySquared](Softs::FSolverParticles& Particles, int32 Offset, int32 Range)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_ParticlePreSimulationTransforms);
			SCOPE_CYCLE_COUNTER(STAT_ChaosClothParticlePreSimulationTransforms);

			const int32 RangeSize = Range - Offset;

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_PreSimulationTransforms_ISPC_Enabled)  // TODO: Make the ISPC works with both Single and Double depending on the FSolverReal type
			{
				if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
				{
					ispc::ApplyPreSimulationTransforms(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FVector3f*)Particles.GetV().GetData(),
						(ispc::FVector3f*)Particles.XArray().GetData(),
						(ispc::FVector3f*)OldAnimationPositions.GetData(),
						(ispc::FVector3f*)AnimationVelocities.GetData(),
						Particles.GetInvM().GetData(),
						(const ispc::FVector3f*)AnimationPositions.GetData(),
						ParticleGroupIds.GetData(),
						(ispc::FTransform3f*)PreSimulationTransforms.GetData(),
						(ispc::FVector3f&)DeltaLocalSpaceLocation,
						DeltaTime,
						Offset,
						Range);
				}
				else
				{
					ispc::ApplyPreSimulationTransformsAndClampVelocity(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FVector3f*)Particles.GetV().GetData(),
						(ispc::FVector3f*)Particles.XArray().GetData(),
						(ispc::FVector3f*)OldAnimationPositions.GetData(),
						(ispc::FVector3f*)AnimationVelocities.GetData(),
						Particles.GetInvM().GetData(),
						(const ispc::FVector3f*)AnimationPositions.GetData(),
						ParticleGroupIds.GetData(),
						(ispc::FTransform3f*)PreSimulationTransforms.GetData(),
						(ispc::FVector3f&)DeltaLocalSpaceLocation,
						DeltaTime,
						Offset,
						Range,
						MaxVelocitySquared);
				}
			}
			else
#endif
			{
				PhysicsParallelFor(RangeSize,
					[this, &ParticleGroupIds, &DeltaLocalSpaceLocation, &Particles, Offset, MaxVelocitySquared](int32 i)
				{
					const int32 Index = Offset + i;
					const Softs::FSolverRigidTransform3& GroupSpaceTransform = PreSimulationTransforms[ParticleGroupIds[Index]];

					// Update initial state for particles
					Particles.P(Index) = Particles.X(Index) = GroupSpaceTransform.TransformPositionNoScale(Particles.X(Index)) - DeltaLocalSpaceLocation;
					Particles.V(Index) = GroupSpaceTransform.TransformVector(Particles.V(Index));

					// Copy InvM over to PAndInvM
					Particles.PAndInvM(Index).InvM = Particles.InvM(Index);

					// Update anim initial state (target updated by skinning)
					OldAnimationPositions[Index] = GroupSpaceTransform.TransformPositionNoScale(OldAnimationPositions[Index]) - DeltaLocalSpaceLocation;

					// Update Animation velocities
					AnimationVelocities[Index] = (AnimationPositions[Index] - OldAnimationPositions[Index]) / DeltaTime;

					// Clamp relative velocity
					const FSolverVec3 RelVelocity = Particles.V(Index) - AnimationVelocities[Index];
					const FSolverReal RelVelocitySquaredLength = RelVelocity.SquaredLength();
					if (RelVelocitySquaredLength > MaxVelocitySquared)
					{
						Particles.V(Index) = AnimationVelocities[Index] + RelVelocity * FMath::Sqrt(MaxVelocitySquared / RelVelocitySquaredLength);
					}
				}, RangeSize < ClothSolverMinParallelBatchSize);
			}
		}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPreUpdate);

#if FRAMEPRO_ENABLED
		FRAMEPRO_CUSTOM_STAT("ChaosClothSolverMinParallelBatchSize", ClothSolverMinParallelBatchSize, "ChaosClothSolver", "Particles", FRAMEPRO_COLOUR(128, 0, 255));
		FRAMEPRO_CUSTOM_STAT("ChaosClothSolverParallelClothPreUpdate", bClothSolverParallelClothPreUpdate, "ChaosClothSolver", "Enabled", FRAMEPRO_COLOUR(128, 128, 64));
#endif

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_CollisionPreSimulationTransforms);
			SCOPE_CYCLE_COUNTER(STAT_ChaosClothCollisionPreSimulationTransforms);

			const TPBDActiveView<Softs::FSolverCollisionParticles>& CollisionParticlesActiveView = PBDEvolution->CollisionParticlesActiveView();
			const TArray<uint32>& CollisionParticleGroupIds = PBDEvolution->CollisionParticleGroupIds();

			CollisionParticlesActiveView.SequentialFor(  // There's unlikely to ever have enough collision particles for a parallel for
				[this, &CollisionParticleGroupIds, &DeltaLocalSpaceLocation](Softs::FSolverCollisionParticles& CollisionParticles, int32 Index)
			{
				const Softs::FSolverRigidTransform3& GroupSpaceTransform = PreSimulationTransforms[CollisionParticleGroupIds[Index]];

				// Update initial state for collisions
				OldCollisionTransforms[Index] = OldCollisionTransforms[Index] * GroupSpaceTransform;
				OldCollisionTransforms[Index].AddToTranslation(-DeltaLocalSpaceLocation);
				CollisionParticles.X(Index) = OldCollisionTransforms[Index].GetTranslation();
				CollisionParticles.SetR(Index, OldCollisionTransforms[Index].GetRotation());
			});
		}
	}

	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		if (Cloth && PreSimulationTransforms.IsValidIndex(Cloth->GetGroupId()))
		{
			Cloth->ApplyPreSimulationTransforms(this, DeltaLocalSpaceLocation, PreSimulationTransforms[Cloth->GetGroupId()], DeltaTime);
		}
	}
}

void FClothingSimulationSolver::PreSubstep( const Softs::FSolverReal InterpolationAlpha, bool bDetectSelfCollisions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_PreSubstep);

	if (Evolution)
	{
		const TArray<uint32> ActiveGroups = Evolution->GetActiveGroupsArray();
		PhysicsParallelFor(ActiveGroups.Num(), [this, &ActiveGroups, InterpolationAlpha, bDetectSelfCollisions](int32 ActiveGroupIndex)
		{
			const uint32 GroupId = ActiveGroups[ActiveGroupIndex];
			const TSet<int32>& ActiveSoftBodies = Evolution->GetGroupActiveSoftBodies(GroupId);

			bool bDoTriangleMeshUpdate = false;

			for (const int32 SoftBodyId : ActiveSoftBodies)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_ParticlePreSubstepKinematicInterpolation);
				SCOPE_CYCLE_COUNTER(STAT_ChaosClothParticlePreSubstepKinematicInterpolation);
				Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(SoftBodyId);

				FClothConstraints& Constraints = GetClothConstraints(SoftBodyId);

				Constraints.SetSkipSelfCollisionInit(!bDetectSelfCollisions);
				if (const TSharedPtr<Softs::FPBDSkinnedTriangleMeshCollisionConstraints>& TriMeshConstraint = Constraints.GetSkinnedTriangleCollisionsConstraint())
				{
					bDoTriangleMeshUpdate |= TriMeshConstraint->IsEnabled() && (!TriMeshConstraint->GetUseSelfCollisionSubstepsForSkinnedTriangleMeshes() || bDetectSelfCollisions);
				}
#if INTEL_ISPC
				if (bRealTypeCompatibleWithISPC && bChaos_PreSubstepInterpolation_ISPC_Enabled)  // TODO: Make the ISPC works with both Single and Double depending on the FSolverReal type
				{
					ispc::PreSubstepInterpolation(
						(ispc::FVector3f*)Particles.GetArrayView(InterpolatedAnimationPositions).GetData(),
						(ispc::FVector3f*)Particles.GetArrayView(InterpolatedAnimationNormals).GetData(),
						(const ispc::FVector3f*)Particles.GetConstArrayView(AnimationPositions).GetData(),
						(const ispc::FVector3f*)Particles.GetConstArrayView(OldAnimationPositions).GetData(),
						(const ispc::FVector3f*)Particles.GetConstArrayView(AnimationNormals).GetData(),
						(const ispc::FVector3f*)Particles.GetConstArrayView(OldAnimationNormals).GetData(),
						InterpolationAlpha,
						0,
						Particles.GetRangeSize());
				}
				else
#endif
				{
					TArrayView<Softs::FSolverVec3> InterpolatedAnimationPositionsView = Particles.GetArrayView(InterpolatedAnimationPositions);
					TArrayView<Softs::FSolverVec3> InterpolatedAnimationNormalsView = Particles.GetArrayView(InterpolatedAnimationNormals);
					TConstArrayView<Softs::FSolverVec3> AnimationPositionsView = Particles.GetConstArrayView(AnimationPositions);
					TConstArrayView<Softs::FSolverVec3> OldAnimationPositionsView = Particles.GetConstArrayView(OldAnimationPositions);
					TConstArrayView<Softs::FSolverVec3> AnimationNormalsView = Particles.GetConstArrayView(AnimationNormals);
					TConstArrayView<Softs::FSolverVec3> OldAnimationNormalsView = Particles.GetConstArrayView(OldAnimationNormals);
					PhysicsParallelFor(Particles.GetRangeSize(),
						[&InterpolatedAnimationPositionsView, &InterpolatedAnimationNormalsView, &AnimationPositionsView,
						&OldAnimationPositionsView, &AnimationNormalsView, &OldAnimationNormalsView, InterpolationAlpha](int32 Index)
					{
						InterpolatedAnimationPositionsView[Index] = InterpolationAlpha * AnimationPositionsView[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationPositionsView[Index];
						InterpolatedAnimationNormalsView[Index] = (InterpolationAlpha * AnimationNormalsView[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationNormalsView[Index]).GetSafeNormal();

					}, Particles.GetRangeSize() < ClothSolverMinParallelBatchSize);
				}
			}
			
			const TSet<int32>& ActiveCollisionRanges = Evolution->GetGroupActiveCollisionParticleRanges(GroupId);
			for (const int32 CollisionRangeId : ActiveCollisionRanges)
			{
				if (FClothComplexColliders* const Colliders = GetComplexColliders(CollisionRangeId))
				{
					Colliders->SetSkipSkinnedTriangleMeshKinematicUpdate(!bDoTriangleMeshUpdate);
				}
			}

		}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPreUpdate);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDClearCollidedArray);
			memset(Collided.GetData(), 0, Collided.Num() * sizeof(bool));
		}
		CollisionContacts.Reset();
		CollisionNormals.Reset();
		CollisionPhis.Reset();
	}
	else
	{
		const TPBDActiveView<Softs::FSolverParticles>& ParticlesActiveView = PBDEvolution->ParticlesActiveView();
		ParticlesActiveView.RangeFor(
			[this, InterpolationAlpha, bDetectSelfCollisions](Softs::FSolverParticles& Particles, int32 Offset, int32 Range)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_ParticlePreSubstepKinematicInterpolation);
			SCOPE_CYCLE_COUNTER(STAT_ChaosClothParticlePreSubstepKinematicInterpolation);

			GetClothConstraints(Offset).SetSkipSelfCollisionInit(!bDetectSelfCollisions);

			const int32 RangeSize = Range - Offset;
	
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_PreSubstepInterpolation_ISPC_Enabled)  // TODO: Make the ISPC works with both Single and Double depending on the FSolverReal type
			{
				ispc::PreSubstepInterpolation(
					(ispc::FVector3f*)InterpolatedAnimationPositions.GetData(),
					(ispc::FVector3f*)InterpolatedAnimationNormals.GetData(),
					(const ispc::FVector3f*)AnimationPositions.GetData(),
					(const ispc::FVector3f*)OldAnimationPositions.GetData(),
					(const ispc::FVector3f*)AnimationNormals.GetData(),
					(const ispc::FVector3f*)OldAnimationNormals.GetData(),
					InterpolationAlpha,
					Offset,
					Range);
			}
			else
#endif
			{
			PhysicsParallelFor(RangeSize,
					[this, Offset, InterpolationAlpha](int32 i)
				{
					const int32 Index = Offset + i;
					InterpolatedAnimationPositions[Index] = InterpolationAlpha * AnimationPositions[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationPositions[Index];
					InterpolatedAnimationNormals[Index] = (InterpolationAlpha * AnimationNormals[Index] + ((Softs::FSolverReal)1. - InterpolationAlpha) * OldAnimationNormals[Index]).GetSafeNormal();

				}, RangeSize < ClothSolverMinParallelBatchSize);
			}
		}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPreUpdate);
	}

	for (FClothingSimulationCloth* const Cloth : Cloths)
	{
		check(Cloth);
		Cloth->PreSubstep(this, InterpolationAlpha);
	}
}

void FClothingSimulationSolver::UpdateSolverField()
{
	if ((Evolution || PBDEvolution) && !PerSolverField.IsEmpty())
	{
		TArray<FVector>& SamplePositions = PerSolverField.GetSamplePositions();
		TArray<FFieldContextIndex>& SampleIndices = PerSolverField.GetSampleIndices();

		if (Evolution)
		{
			const uint32 NumParticles = Evolution->GetParticles().Size();
			const uint32 NumActiveParticles = Evolution->NumActiveParticles();

			SamplePositions.SetNum(NumParticles, EAllowShrinking::No);
			SampleIndices.SetNum(NumActiveParticles, EAllowShrinking::No);

			int32 SampleIndex = 0;
			for (const uint32 GroupId : Evolution->GetActiveGroupsArray())
			{
				for (const int32 SoftBodyId : Evolution->GetGroupActiveSoftBodies(GroupId))
				{
					const Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(SoftBodyId);
					for (int32 LocalParticleIndex = 0; LocalParticleIndex < Particles.GetRangeSize(); ++LocalParticleIndex)
					{
						const int32 GlobalParticleIndex = LocalParticleIndex + Particles.GetOffset();
						SamplePositions[GlobalParticleIndex] = LocalSpaceScale * FVector(Particles.X(LocalParticleIndex)) + LocalSpaceLocation;
						SampleIndices[SampleIndex] = FFieldContextIndex(GlobalParticleIndex, SampleIndex);
						++SampleIndex;
					}
				}
			}
		}
		else
		{
			const uint32 NumParticles = PBDEvolution->Particles().Size();
			const uint32 NumActiveParticles = PBDEvolution->ParticlesActiveView().GetActiveSize();

			SamplePositions.SetNum(NumParticles, EAllowShrinking::No);
			SampleIndices.SetNum(NumActiveParticles, EAllowShrinking::No);

			int32 SampleIndex = 0;
			PBDEvolution->ParticlesActiveView().SequentialFor(
				[this, &SamplePositions, &SampleIndices, &SampleIndex](Softs::FSolverParticles& Particles, int32 ParticleIndex)
			{
				SamplePositions[ParticleIndex] = LocalSpaceScale * FVector(Particles.X(ParticleIndex)) + LocalSpaceLocation;
				SampleIndices[SampleIndex] = FFieldContextIndex(ParticleIndex, SampleIndex);
				++SampleIndex;
			});
		}

		PerSolverField.ComputeFieldLinearImpulse(GetTime());
	}
	else
	{
		// Reset the outputs once the field isn't being processed anymore
		PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearVelocity).Reset();
		PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearForce).Reset();
	}
}

void FClothingSimulationSolver::Update(Softs::FSolverReal InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_Update);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdate);

#if !UE_BUILD_SHIPPING
	// Introduce artificial hitches for debugging any simulation jitter
	if (ClothSolverDebugHitchLength && ClothSolverDebugHitchInterval)
	{
		static int32 HitchCounter = 0;
		if (--HitchCounter < 0)
		{
			UE_LOGF(LogChaosCloth, Warning, "Hitching for %dms", ClothSolverDebugHitchLength);
			FPlatformProcess::Sleep((float)ClothSolverDebugHitchLength * 0.001f);
			HitchCounter = ClothSolverDebugHitchInterval;
		}
	}
#endif  // #if !UE_BUILD_SHIPPING

	// Update time step
	DeltaTime = InDeltaTime;

	// Update Cloths and cloth colliders
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_UpdateCloths);
		SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdateCloths);

		Swap(OldCollisionTransforms, CollisionTransforms);
		Swap(OldAnimationPositions, AnimationPositions);
		Swap(OldAnimationNormals, AnimationNormals);
		for (TPair<int32, TUniquePtr<FClothComplexColliders>>& Collider : ComplexColliders)
		{
			if (Collider.Value)
			{
				Collider.Value->SwapBuffersForFrameFlip();
			}
		}

		// Clear external collisions so that they can be re-added
		CollisionParticlesSize = 0;

		// Run sequential pre-updates first
		for (FClothingSimulationCloth* const Cloth : Cloths)
		{
			Cloth->PreUpdate(this);
		}

		// Run parallel update
		PhysicsParallelFor(Cloths.Num(), [this](int32 ClothIndex)
		{
			FClothingSimulationCloth* const Cloth = Cloths[ClothIndex];
			const uint32 GroupId = Cloth->GetGroupId();

			if (PBDEvolution)
			{
				// Pre-update overridable solver properties first
				PBDEvolution->SetGravity(Gravity, GroupId);
				PBDEvolution->GetVelocityAndPressureField(GroupId).SetVelocity(WindVelocity);
			}

			Cloth->Update(this);
		}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothUpdate);
	}

	// Pre solver step, apply group space transforms for teleport and linear/delta ratios, ...etc
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_UpdatePreSolverStep);
		SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdatePreSolverStep);

		ApplyPreSimulationTransforms();
		
		EventPreSolve.Broadcast(DeltaTime);
	}

	const bool bAdvanceTimeStep = (DeltaTime > Softs::FSolverReal(0.));
	if (bAdvanceTimeStep)
	{
		// Compute the solver field forces/velocities for future use in the solver force function
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_UpdateSolverFields);
			SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdateSolverFields);

			UpdateSolverField();
		}

		// Advance Sim
		if (bEnableSolver)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_UpdateSolverStep);
			SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdateSolverStep);
			SCOPE_CYCLE_COUNTER(STAT_ClothInternalSolve);

			check(Config);
			const Softs::FCollectionPropertyFacade& Properties = Config->GetProperties(SolverLOD);

			const int32 ConfigNumSubsteps = FMath::Max(GetNumSubsteps(Properties, ClothingSimulationSolverDefault::NumSubsteps),
				ClothingSimulationSolverDefault::MinNumSubsteps);

			const Softs::FSolverReal ConfigDynamicSubstepDeltaTime = GetDynamicSubstepDeltaTime(Properties, ClothingSimulationSolverDefault::DynamicSubstepDeltaTime); //  This is in ms

			// Calculate NumSubsteps for dynamic substepping.
			NumUsedSubsteps = ConfigDynamicSubstepDeltaTime > (Softs::FSolverReal)0.f ? FMath::Clamp(FMath::RoundToInt32(DeltaTime * 1000.f / ConfigDynamicSubstepDeltaTime), 1, ConfigNumSubsteps) : ConfigNumSubsteps;

			const bool bConfigEnableNumSelfCollisionSubsteps = GetEnableNumSelfCollisionSubsteps(Properties, ClothingSimulationSolverDefault::bEnableNumSelfCollisionSubsteps);

			const int32 ConfigNumSelfCollisionSubsteps = bConfigEnableNumSelfCollisionSubsteps ? FMath::Clamp(GetNumSelfCollisionSubsteps(Properties, ClothingSimulationSolverDefault::NumSelfCollisionSubsteps), ClothingSimulationSolverDefault::MinNumSubsteps, NumUsedSubsteps) : NumUsedSubsteps;

			const int32 NumSubstepsPerSelfCollisionSubstep = (NumUsedSubsteps - 1) / ConfigNumSelfCollisionSubsteps + 1;

			if (Evolution)
			{
				Evolution->SetSolverProperties(Properties);
				Evolution->SetDisableTimeDependentNumIterations(bClothSolverDisableTimeDependentNumIterations);
			}
			else
			{
				const int32 ConfigMaxNumIterations = FMath::Max(
					GetMaxNumIterations(Properties, ClothingSimulationSolverDefault::MaxNumIterations),
					ClothingSimulationSolverDefault::MinNumIterations);

				const int32 ConfigNumIterations = FMath::Clamp(
					GetNumIterations(Properties, ClothingSimulationSolverDefault::NumIterations),
					ClothingSimulationSolverDefault::MinNumIterations,
					ConfigMaxNumIterations);

				// Update solver time dependent parameters
				const Softs::FSolverReal SolverFrequency = (Softs::FSolverReal)ClothingSimulationSolverDefault::SolverFrequency;  // 60Hz default TODO: Should this become a solver property?

				const int32 TimeDependentNumIterations = bClothSolverDisableTimeDependentNumIterations ?
					ConfigNumIterations :
					FMath::RoundToInt32(SolverFrequency * DeltaTime * (Softs::FSolverReal)ConfigNumIterations);

				PBDEvolution->SetIterations(FMath::Clamp(TimeDependentNumIterations, 1, ConfigMaxNumIterations));
			}

			// Advance substeps
			const Softs::FSolverReal SubstepDeltaTime = DeltaTime / (Softs::FSolverReal)NumUsedSubsteps;
	
			for (int32 i = 0; i < NumUsedSubsteps; ++i)
			{
				PreSubstep(FMath::Clamp((Softs::FSolverReal)(i + 1) / (Softs::FSolverReal)NumUsedSubsteps, (Softs::FSolverReal)0., (Softs::FSolverReal)1.), i% NumSubstepsPerSelfCollisionSubstep == 0);
				if (Evolution)
				{
					Evolution->AdvanceOneTimeStep(SubstepDeltaTime, (Softs::FSolverReal)NumUsedSubsteps);
				}
				else
				{
					PBDEvolution->AdvanceOneTimeStep(SubstepDeltaTime);
				}
			}

			Time = Evolution ? Evolution->GetTime() : PBDEvolution->GetTime();
			UE_LOGF(LogChaosCloth, VeryVerbose, "DeltaTime: %.6f, Time = %.6f", DeltaTime, Time);
		}
	}

	// Post solver step, update normals, ...etc
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_UpdatePostSolverStep);
		SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverUpdatePostSolverStep);
		SCOPE_CYCLE_COUNTER(STAT_ClothComputeNormals);
		
		ExtremeDeformationReset();

		EventPostSolve.Broadcast(DeltaTime);

		PhysicsParallelFor(Cloths.Num(), [this](int32 ClothIndex)
		{
			FClothingSimulationCloth* const Cloth = Cloths[ClothIndex];
			Cloth->PostUpdate(this);
		}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPostUpdate);
	}

	// Save old space location for next update
	OldLocalSpaceLocation = LocalSpaceLocation;
}

void FClothingSimulationSolver::ExtremeDeformationReset()
{
	if (bClothSolverEnableResetIfExtremeEdgeDeformation)
	{
		if (Evolution)
		{
			const TArray<uint32> ActiveGroups = Evolution->GetActiveGroupsArray();
			PhysicsParallelFor(ActiveGroups.Num(), [this, &ActiveGroups](int32 ActiveGroupIndex)
				{
					const uint32 GroupId = ActiveGroups[ActiveGroupIndex];
					const TSet<int32>& ActiveSoftBodies = Evolution->GetGroupActiveSoftBodies(GroupId);
					for (const int32 SoftBodyId : ActiveSoftBodies)
					{
						if (const Softs::FPBDExtremeDeformationConstraints* const ExtremeDeformationConstraints = GetClothConstraints(SoftBodyId).GetExtremeDeformationConstraints().Get())
						{
							if (ExtremeDeformationConstraints->IsExtremelyDeformed(GetParticleXsView(SoftBodyId)))
							{
								ResetStartPose(SoftBodyId, Evolution->GetSoftBodyParticleNum(SoftBodyId));
							}
						}
					}
				});
		}
		else
		{
			const TPBDActiveView<Softs::FSolverParticles>& ParticlesActiveView = PBDEvolution->ParticlesActiveView();
			ParticlesActiveView.RangeFor(
				[this](Softs::FSolverParticles& Particles, int32 Offset, int32 Range)
			{
				if (const Softs::FPBDExtremeDeformationConstraints* const ExtremeDeformationConstraints = GetClothConstraints(Offset).GetExtremeDeformationConstraints().Get())
				{
					if (ExtremeDeformationConstraints->IsExtremelyDeformed(GetParticleXs()))
					{
						ResetStartPose(Offset, PBDEvolution->GetParticleRangeSize(Offset));
					}
				}
			}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPostUpdate);
		}
	}
}

void FClothingSimulationSolver::UpdateFromCache(const FClothingSimulationCacheData& CacheData)
{
	Chaos::Softs::FSolverParticles& SolverParticles = Evolution ? Evolution->GetParticles() : PBDEvolution->GetParticles();
	const int32 NumParticles = GetNumParticles();
	const int32 NumCachedParticles = CacheData.CacheIndices.Num();	
	const bool bHasVelocity = CacheData.CachedVelocities.Num() > 0;
	for (int32 CacheIndex = 0; CacheIndex < NumCachedParticles; ++CacheIndex)
	{
		const int32 ParticleIndex = CacheData.CacheIndices[CacheIndex];
		if (ensure(ParticleIndex < NumParticles))
		{
			SolverParticles.X(ParticleIndex) = CacheData.CachedPositions[CacheIndex];
			SolverParticles.V(ParticleIndex) = bHasVelocity ? (FSolverVec3)CacheData.CachedVelocities[CacheIndex] : FSolverVec3::ZeroVector;
		}
	}
	PhysicsParallelFor(Cloths.Num(), [this, &CacheData](int32 ClothIndex)
	{
		FClothingSimulationCloth* const Cloth = Cloths[ClothIndex];
		Cloth->UpdateFromCache(CacheData);
		Cloth->PostUpdate(this);
	}, /*bForceSingleThreaded =*/ !bClothSolverParallelClothPostUpdate);

}

int32 FClothingSimulationSolver::GetNumUsedIterations() const
{
	return Evolution ? Evolution->GetNumUsedIterations() : PBDEvolution->GetIterations();
}

int32 FClothingSimulationSolver::GetNumLinearSolverIterations(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetLastLinearSolveIterations(ParticleRangeId) : 0;
}

FRealSingle FClothingSimulationSolver::GetLinearSolverError(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetLastLinearSolveError(ParticleRangeId) : 0.f;
}

FBoxSphereBounds FClothingSimulationSolver::CalculateBounds() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClothingSimulationSolver_CalculateBounds);
	SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverCalculateBounds);

	if (Evolution)
	{
		// Calculate bounding box
		Softs::FSolverAABB3 BoundingBox = Softs::FSolverAABB3::EmptyAABB();
		const TArray<uint32> ActiveGroupsArray = Evolution->GetActiveGroupsArray();
		for (const uint32 GroupId : ActiveGroupsArray)
		{
			for (const int32 SoftBodyId : Evolution->GetGroupActiveSoftBodies(GroupId))
			{
				const Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(SoftBodyId);
#if INTEL_ISPC
				if (bRealTypeCompatibleWithISPC && bChaos_CalculateBounds_ISPC_Enabled)
				{
					Softs::FSolverVec3 NewMin = BoundingBox.Min();
					Softs::FSolverVec3 NewMax = BoundingBox.Max();

					ispc::CalculateBounds(
						(ispc::FVector3f&)NewMin,
						(ispc::FVector3f&)NewMax,
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						0,
						Particles.GetRangeSize());

					BoundingBox.GrowToInclude(Softs::FSolverAABB3(NewMin, NewMax));
				}
				else
#endif
				{
					for (const Softs::FSolverVec3& X : Particles.XArray())
					{
						BoundingBox.GrowToInclude(X);
					}
				}
			}
		}

		if (BoundingBox.IsEmpty())
		{
			return FBoxSphereBounds(LocalSpaceLocation, FVector(0.f), 0.f);
		}

		// Calculate (squared) radius
		const Softs::FSolverVec3 Center = BoundingBox.Center();
		Softs::FSolverReal SquaredRadius = (Softs::FSolverReal)0.;

		for (const uint32 GroupId : ActiveGroupsArray)
		{
			for (const int32 SoftBodyId : Evolution->GetGroupActiveSoftBodies(GroupId))
			{
				const Softs::FSolverParticlesRange& Particles = Evolution->GetSoftBodyParticles(SoftBodyId);
#if INTEL_ISPC
				if (bRealTypeCompatibleWithISPC && bChaos_CalculateBounds_ISPC_Enabled)
				{
					ispc::CalculateSquaredRadius(
						SquaredRadius,
						(const ispc::FVector3f&)Center,
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						0,
						Particles.GetRangeSize());
				}
				else
#endif
				{
					for (const Softs::FSolverVec3& X : Particles.XArray())
					{
						SquaredRadius = FMath::Max(SquaredRadius, (X - Center).SizeSquared());
					}
				}
			}
		}

		// Update bounds with this cloth
		return FBoxSphereBounds(LocalSpaceLocation + LocalSpaceScale * BoundingBox.Center(), LocalSpaceScale * FVector(BoundingBox.Extents() * 0.5f), LocalSpaceScale * FMath::Sqrt(SquaredRadius));
	}
	else
	{
		const TPBDActiveView<Softs::FSolverParticles>& ParticlesActiveView = PBDEvolution->ParticlesActiveView();

		if (ParticlesActiveView.HasActiveRange())
		{
			// Calculate bounding box
			Softs::FSolverAABB3 BoundingBox = Softs::FSolverAABB3::EmptyAABB();

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_CalculateBounds_ISPC_Enabled)
			{
				ParticlesActiveView.RangeFor(
					[&BoundingBox](Softs::FSolverParticles& Particles, int32 Offset, int32 Range)
				{
					Softs::FSolverVec3 NewMin = BoundingBox.Min();
					Softs::FSolverVec3 NewMax = BoundingBox.Max();

					ispc::CalculateBounds(
						(ispc::FVector3f&)NewMin,
						(ispc::FVector3f&)NewMax,
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						Offset,
						Range);

					BoundingBox.GrowToInclude(Softs::FSolverAABB3(NewMin, NewMax));
				});
			}
			else
#endif
			{
				ParticlesActiveView.SequentialFor(
					[&BoundingBox](Softs::FSolverParticles& Particles, int32 Index)
				{
					BoundingBox.GrowToInclude(Particles.X(Index));
				});
			}

			// Calculate (squared) radius
			const Softs::FSolverVec3 Center = BoundingBox.Center();
			Softs::FSolverReal SquaredRadius = (Softs::FSolverReal)0.;

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_CalculateBounds_ISPC_Enabled)
			{
				ParticlesActiveView.RangeFor(
					[&SquaredRadius, &Center](Softs::FSolverParticles& Particles, int32 Offset, int32 Range)
				{
					ispc::CalculateSquaredRadius(
						SquaredRadius,
						(const ispc::FVector3f&)Center,
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						Offset,
						Range);
				});
			}
			else
#endif
			{
				ParticlesActiveView.SequentialFor(
					[&SquaredRadius, &Center](Softs::FSolverParticles& Particles, int32 Index)
				{
					SquaredRadius = FMath::Max(SquaredRadius, (Particles.X(Index) - Center).SizeSquared());
				});
			}

			// Update bounds with this cloth
			return FBoxSphereBounds(LocalSpaceLocation + LocalSpaceScale * BoundingBox.Center(), LocalSpaceScale * FVector(BoundingBox.Extents() * 0.5f), LocalSpaceScale * FMath::Sqrt(SquaredRadius));
		}

		return FBoxSphereBounds(LocalSpaceLocation, FVector(0.f), 0.f);
	}
}


bool FClothingSimulationSolver::UsesBuoyancy()
{
	static IConsoleVariable* const UsesBuoyancy = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Cloth.EnableBuoyancy"));
	return UsesBuoyancy ? UsesBuoyancy->GetBool() && !IsLegacySolver() : false;
}

void FClothingSimulationSolver::PreSimulateBuoyancyField_GameThread(float dt, const UWorld* World, const FBox BoundingBox)
{
	auto& AllConstraints = ClothsConstraints;

	for (auto& Pair : AllConstraints)
	{
		int32 Key = Pair.Key;
		::Chaos::FClothConstraints* Constraints = Pair.Value.Get();

		if (Buoyancy::FBuoyancyField *BuoyancyField = Constraints->GetBuoyancyField().Get())
		{
			BuoyancyField->PreSimulate_GameThread(dt, World, BoundingBox);
		}
	}
}

uint32 FClothingSimulationSolver::GetNumParticles() const
{
	return Evolution ? Evolution->GetParticles().Size() : PBDEvolution->GetParticles().Size();
}

int32 FClothingSimulationSolver::GetNumActiveParticles() const
{
	return Evolution ? Evolution->NumActiveParticles() : PBDEvolution->ParticlesActiveView().GetActiveSize();
}

int32 FClothingSimulationSolver::GetGlobalParticleOffset(int32 ParticleRangeId) const
{
	return Evolution ? Evolution->GetSoftBodyParticles(ParticleRangeId).GetOffset() : ParticleRangeId;
}

} // End namespace Chaos
