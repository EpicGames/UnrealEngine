// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Transform.isph"

export void ApplyPreSimulationTransforms(uniform FVector4f ParticlesPandInvM[],
										uniform FVector3f ParticlesV[],
										uniform FVector3f ParticlesX[],
										uniform FVector3f OldAnimationPositions[],
										uniform FVector3f AnimationVelocities[],
										const uniform float InvM[],
										const uniform FVector3f AnimationPositions[],
										const uniform uint32 ParticleGroupIds[],
										const uniform FTransform3f PreSimulationTransforms[],
										const uniform FVector3f &DeltaLocalSpaceLocation,
										const uniform float DeltaTime,
										const uniform int32 Offset,
										const uniform int32 Range)
{
	foreach(i = Offset ... Range)
	{
		const varying uint32 ParticleGroupId = ParticleGroupIds[i];
		#pragma ignore warning(perf)
		const varying FTransform3f GroupSpaceTransform = PreSimulationTransforms[ParticleGroupId];

		const uniform int32 Index = extract(i, 0);

		const varying FVector3f ParticleX = VectorLoad(&ParticlesX[Index]);
		const varying FVector3f PX = TransformPosition(GroupSpaceTransform, ParticleX) - DeltaLocalSpaceLocation;

		VectorStore(&ParticlesPandInvM[Index], SetVector4(PX, InvM[i]));
		VectorStore(&ParticlesX[Index], PX);

		const varying FVector3f ParticleV = VectorLoad(&ParticlesV[Index]);
		const varying FVector3f V = TransformVector(GroupSpaceTransform, ParticleV);
		VectorStore(&ParticlesV[Index], V);

		const varying FVector3f OldAnimationPosition = VectorLoad(&OldAnimationPositions[Index]);
		const varying FVector3f AP = TransformPosition(GroupSpaceTransform, OldAnimationPosition) - DeltaLocalSpaceLocation;

		VectorStore(&OldAnimationPositions[Index], AP);

		const varying FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f AV = (AnimationPosition - AP) / DeltaTime;

		VectorStore(&AnimationVelocities[Index], AV);
	}
}

export void ApplyPreSimulationTransform(uniform FVector4f ParticlesPandInvM[],
										uniform FVector3f ParticlesV[],
										uniform FVector3f ParticlesX[],
										uniform FVector3f OldAnimationPositions[],
										uniform FVector3f AnimationVelocities[],
										const uniform float InvM[],
										const uniform FVector3f AnimationPositions[],
										const uniform FTransform3f& GroupSpaceTransform,
										const uniform FVector3f &DeltaLocalSpaceLocation,
										const uniform float DeltaTime,
										const uniform int32 NumParticles)
{
	foreach(i = 0 ... NumParticles)
	{
		const uniform int32 Index = extract(i, 0);

		const varying FVector3f ParticleX = VectorLoad(&ParticlesX[Index]);
		const varying FVector3f PX = TransformPosition(GroupSpaceTransform, ParticleX) - DeltaLocalSpaceLocation;

		VectorStore(&ParticlesPandInvM[Index], SetVector4(PX, InvM[i]));
		VectorStore(&ParticlesX[Index], PX);

		const varying FVector3f ParticleV = VectorLoad(&ParticlesV[Index]);
		const varying FVector3f V = TransformVector(GroupSpaceTransform, ParticleV);
		VectorStore(&ParticlesV[Index], V);

		const varying FVector3f OldAnimationPosition = VectorLoad(&OldAnimationPositions[Index]);
		const varying FVector3f AP = TransformPosition(GroupSpaceTransform, OldAnimationPosition) - DeltaLocalSpaceLocation;

		VectorStore(&OldAnimationPositions[Index], AP);

		const varying FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f AV = (AnimationPosition - AP) / DeltaTime;

		VectorStore(&AnimationVelocities[Index], AV);
	}
}

export void ApplyAnimationPreSimulationTransform(uniform FVector3f OldAnimationPositions[],
										uniform FVector3f AnimationVelocities[],
										const uniform FVector3f AnimationPositions[],
										const uniform FTransform3f& GroupSpaceTransform,
										const uniform FVector3f &DeltaLocalSpaceLocation,
										const uniform float DeltaTime,
										const uniform int32 NumParticles)
{
	foreach(i = 0 ... NumParticles)
	{
		const uniform int32 Index = extract(i, 0);

		const varying FVector3f OldAnimationPosition = VectorLoad(&OldAnimationPositions[Index]);
		const varying FVector3f AP = TransformPosition(GroupSpaceTransform, OldAnimationPosition) - DeltaLocalSpaceLocation;

		VectorStore(&OldAnimationPositions[Index], AP);

		const varying FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f AV = (AnimationPosition - AP) / DeltaTime;

		VectorStore(&AnimationVelocities[Index], AV);
	}
}

export void ApplyPreSimulationTransformAndClampVelocity(uniform FVector4f ParticlesPandInvM[],
										uniform FVector3f ParticlesV[],
										uniform FVector3f ParticlesX[],
										uniform FVector3f OldAnimationPositions[],
										uniform FVector3f AnimationVelocities[],
										const uniform float InvM[],
										const uniform FVector3f AnimationPositions[],
										const uniform FTransform3f& GroupSpaceTransform,
										const uniform FVector3f &DeltaLocalSpaceLocation,
										const uniform float DeltaTime,
										const uniform int32 NumParticles,
										const uniform float MaxVelocitySquared)
{
	foreach(i = 0 ... NumParticles)
	{
		const uniform int32 Index = extract(i, 0);

		const varying FVector3f ParticleX = VectorLoad(&ParticlesX[Index]);
		const varying FVector3f PX = TransformPosition(GroupSpaceTransform, ParticleX) - DeltaLocalSpaceLocation;

		VectorStore(&ParticlesPandInvM[Index], SetVector4(PX, InvM[i]));
		VectorStore(&ParticlesX[Index], PX);

		const varying FVector3f ParticleV = VectorLoad(&ParticlesV[Index]);
		varying FVector3f V = TransformVector(GroupSpaceTransform, ParticleV);

		const varying FVector3f OldAnimationPosition = VectorLoad(&OldAnimationPositions[Index]);
		const varying FVector3f AP = TransformPosition(GroupSpaceTransform, OldAnimationPosition) - DeltaLocalSpaceLocation;

		VectorStore(&OldAnimationPositions[Index], AP);

		const varying FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f AV = (AnimationPosition - AP) / DeltaTime;

		VectorStore(&AnimationVelocities[Index], AV);

		const varying FVector3f RelVelocity = V - AV;
		const varying float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			V = AV + RelVelocity * sqrt(MaxVelocitySquared/RelVelocitySquared);
		}

		VectorStore(&ParticlesV[Index], V);
	}
}

export void ApplyPreSimulationTransformsAndClampVelocity(uniform FVector4f ParticlesPandInvM[],
										uniform FVector3f ParticlesV[],
										uniform FVector3f ParticlesX[],
										uniform FVector3f OldAnimationPositions[],
										uniform FVector3f AnimationVelocities[],
										const uniform float InvM[],
										const uniform FVector3f AnimationPositions[],
										const uniform uint32 ParticleGroupIds[],
										const uniform FTransform3f PreSimulationTransforms[],
										const uniform FVector3f &DeltaLocalSpaceLocation,
										const uniform float DeltaTime,
										const uniform int32 Offset,
										const uniform int32 Range,
										const uniform float MaxVelocitySquared)
{
	foreach(i = Offset ... Range)
	{
		const varying uint32 ParticleGroupId = ParticleGroupIds[i];
		#pragma ignore warning(perf)
		const varying FTransform3f GroupSpaceTransform = PreSimulationTransforms[ParticleGroupId];

		const uniform int32 Index = extract(i, 0);

		const varying FVector3f ParticleX = VectorLoad(&ParticlesX[Index]);
		const varying FVector3f PX = TransformPosition(GroupSpaceTransform, ParticleX) - DeltaLocalSpaceLocation;

		VectorStore(&ParticlesPandInvM[Index], SetVector4(PX, InvM[i]));
		VectorStore(&ParticlesX[Index], PX);

		const varying FVector3f ParticleV = VectorLoad(&ParticlesV[Index]);
		varying FVector3f V = TransformVector(GroupSpaceTransform, ParticleV);

		const varying FVector3f OldAnimationPosition = VectorLoad(&OldAnimationPositions[Index]);
		const varying FVector3f AP = TransformPosition(GroupSpaceTransform, OldAnimationPosition) - DeltaLocalSpaceLocation;

		VectorStore(&OldAnimationPositions[Index], AP);

		const varying FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f AV = (AnimationPosition - AP) / DeltaTime;

		VectorStore(&AnimationVelocities[Index], AV);

		const varying FVector3f RelVelocity = V - AV;
		const varying float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			V = AV + RelVelocity * sqrt(MaxVelocitySquared/RelVelocitySquared);
		}

		VectorStore(&ParticlesV[Index], V);
	}
}

export void PreSubstepInterpolation( uniform FVector3f InterpolatedPositions[],
									uniform FVector3f InterpolatedNormals[],
									const uniform FVector3f AnimationPositions[],
									const uniform FVector3f OldAnimationPositions[],
									const uniform FVector3f AnimationNormals[],
									const uniform FVector3f OldAnimationNormals[],
									const uniform float InterpolationAlpha,
									const uniform int32 Offset,
									const uniform int32 Range)
{
	foreach(i=Offset ... Range)
	{
		const uniform int32 Index = extract(i, 0);
		const varying FVector3f Position = VectorLoad(&AnimationPositions[Index]);
		const varying FVector3f OldPosition = VectorLoad(&OldAnimationPositions[Index]);
		varying FVector3f InterpPosition = InterpolationAlpha * Position + ((uniform float)1.f - InterpolationAlpha) * OldPosition;
		VectorStore(&InterpolatedPositions[Index], InterpPosition);

		const varying FVector3f Normal = VectorLoad(&AnimationNormals[Index]);
		const varying FVector3f OldNormal = VectorLoad(&OldAnimationNormals[Index]);
		varying FVector3f InterpNormal = VectorGetSafeNormal(InterpolationAlpha * Normal + ((uniform float)1.f - InterpolationAlpha) * OldNormal);
		VectorStore(&InterpolatedNormals[Index], InterpNormal);
	}
}

export void CalculateBounds(uniform FVector3f &MMin,
							uniform FVector3f &MMax,
							const uniform FVector3f ParticlesX[],
							const uniform int32 Offset,
							const uniform int32 Range)
{
	varying FVector3f Min = MMin;
	varying FVector3f Max = MMax;

	foreach(i = Offset ... Range)
	{
		const uniform int32 Index = extract(i, 0);
		const varying FVector3f X = VectorLoad(&ParticlesX[Index]);

		Min = VectorMin(Min, X);
		Max = VectorMax(Max, X);
	}

	MMin = VectorReduceMin(Min);
	MMax = VectorReduceMax(Max);
}

export void CalculateSquaredRadius(uniform float &SquaredRadius,
									const uniform FVector3f &Center,
									const uniform FVector3f ParticlesX[],
									const uniform int32 Offset,
									const uniform int32 Range)
{
	varying float vSquaredRadius = SquaredRadius;

	foreach(i = Offset ... Range)
	{
		const uniform int32 Index = extract(i, 0);
		const varying FVector3f X = VectorLoad(&ParticlesX[Index]);

		vSquaredRadius = max(vSquaredRadius, VectorSizeSquared(X - Center));
	}

	SquaredRadius = reduce_max(vSquaredRadius);
}
