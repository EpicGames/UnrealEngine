// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothPhysicalMeshData.h"
#include "ChaosWeightMapTarget.generated.h"


/** Targets for painted weight maps (aka masks). */
UENUM()
enum class EChaosWeightMapTarget : uint8
{
	None               = (uint8)EWeightMapTargetCommon::None,
	MaxDistance        = (uint8)EWeightMapTargetCommon::MaxDistance,
	BackstopDistance   = (uint8)EWeightMapTargetCommon::BackstopDistance,
	BackstopRadius     = (uint8)EWeightMapTargetCommon::BackstopRadius,
	AnimDriveStiffness = (uint8)EWeightMapTargetCommon::AnimDriveStiffness,
	TetherEndsMask     = (uint8)EWeightMapTargetCommon::TetherEndsMask,

	// Add Chaos specific maps below this line
	AnimDriveDamping   = (uint8)EWeightMapTargetCommon::AnimDriveDamping_DEPRECATED,
	TetherStiffness    = (uint8)EWeightMapTargetCommon::FirstUserTarget,
	TetherScale,
	Drag,
	Lift,
	EdgeStiffness,
	BendingStiffness,
	AreaStiffness,
	BucklingStiffness,
	Pressure,
	FlatnessRatio,
	OuterDrag,
	OuterLift,

	// Add Chaos specific maps above this line
	MAX UMETA(Hidden)
};
static_assert((uint8)EChaosWeightMapTarget::MAX <= (uint8)EWeightMapTargetCommon::LastUserTarget);
