// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && INTEL_ISPC

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestChaosClothingSimulationSolverAABB, "Ispc.Physics.ChaosClothingSimulationSolver.AABB", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestChaosClothingSimulationSolverAABB::RunTest(const FString& Parameters)
{
	using namespace Chaos;
	
	bool InitialState = bChaos_CalculateBounds_ISPC_Enabled;

	const int32 NumParticles = 100;

	bChaos_CalculateBounds_ISPC_Enabled = true;
	FBoxSphereBounds ISPCAABB;
	{
		FClothingSimulationSolver ClothingSimulationSolver;
		ClothingSimulationSolver.AddParticles(NumParticles, 0);
		ClothingSimulationSolver.EnableParticles(0, true);

		for (int32 i = 0; i < NumParticles; ++i)
		{
			double Value = (i % 2) ? (double)i * 100. : (double)(-i) * 100.;

			Softs::FSolverVec3* Solver = ClothingSimulationSolver.GetParticleXs(i);
			if (Solver != nullptr)
			{
				*Solver = FSolverVec3(Value);
			}
		}

		ISPCAABB = ClothingSimulationSolver.CalculateBounds();
	}

	bChaos_CalculateBounds_ISPC_Enabled = false;
	FBoxSphereBounds CPPAABB;
	{
		FClothingSimulationSolver ClothingSimulationSolver;
		ClothingSimulationSolver.AddParticles(NumParticles, 0);
		ClothingSimulationSolver.EnableParticles(0, true);

		for (int32 i = 0; i < NumParticles; ++i)
		{
			double Value = (i % 2) ? (double)i * 100. : (double)(-i) * 100.;

			Softs::FSolverVec3* Solver = ClothingSimulationSolver.GetParticleXs(i);
			if (Solver != nullptr)
			{
				*Solver = FSolverVec3(Value);
			}
		}

		CPPAABB = ClothingSimulationSolver.CalculateBounds();
	}

	bChaos_CalculateBounds_ISPC_Enabled = InitialState;

	TestTrue(TEXT("Origin"), ISPCAABB.Origin.Equals(CPPAABB.Origin));
	TestTrue(TEXT("Extent"), ISPCAABB.BoxExtent.Equals(CPPAABB.BoxExtent));
	TestEqual(TEXT("Radius"), ISPCAABB.SphereRadius, CPPAABB.SphereRadius);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcTestChaosClothingSimulationSolverPreSimulation, "Ispc.Physics.ChaosClothingSimulationSolver.PreSimulation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcTestChaosClothingSimulationSolverPreSimulation::RunTest(const FString& Parameters)
{
	using namespace Chaos;
	
	bool InitialState = bChaos_PreSimulationTransforms_ISPC_Enabled;

	const int32 NumParticles = 100;

	const Softs::FSolverReal InvMValue = .5f;
	Softs::FPAndInvM FPInvMData;
	FPInvMData.P = Softs::FSolverVec3(0.f);
	FPInvMData.InvM = InvMValue;

	const Softs::FSolverVec3 XSolverVec(5.f);
	const Softs::FSolverVec3 VSolverVec(2.f);
	const Softs::FSolverVec3 CurrentAnimationVec(1.f);
	const Softs::FSolverVec3 OldAnimationVec(5.f);

	const FQuat4d Rotation(1., 1., 1., 1.);
	// Debug complains that the rotation isn't normalized.
	const FQuat4d NormalizedRotation(Rotation.GetNormalized());

	constexpr uint32 GroupId = 0;
	const FRigidTransform3 OldTransform(Softs::FSolverVec3(1.), NormalizedRotation);
	const FRigidTransform3 NewTransform(Softs::FSolverVec3(10.), NormalizedRotation);
	TVec3<FReal> ReferenceVelocity(0.);
	TVec3<FReal> ReferenceAngularVelocity(0.);
	const TVec3<FRealSingle> LinearVelocity(10.f);
	const TVec3<FRealSingle> MaxLinearVelocity(TNumericLimits<FRealSingle>::Max());
	const TVec3<FRealSingle> MaxLinearAcceleration(TNumericLimits<FRealSingle>::Max());
	const FRealSingle AngularVelocityScale = 1.f;
	const FRealSingle MaxAngularVelocity = TNumericLimits<FRealSingle>::Max();
	const FRealSingle MaxAngularAcceleration = TNumericLimits<FRealSingle>::Max();
	const FRealSingle FicticiousAngularScale = 1.f;
	constexpr FRealSingle MaxVelocityScale = 1.f;

	bChaos_PreSimulationTransforms_ISPC_Enabled = true;
	FClothingSimulationSolver ISPCSimulation;
	{
		ISPCSimulation.AddParticles(NumParticles, 0);
		ISPCSimulation.EnableParticles(0, true);

		for (int32 i = 0; i < NumParticles; ++i)
		{
			Softs::FSolverVec3* XSolver = ISPCSimulation.GetParticleXs(i);
			Softs::FSolverVec3* VSolver = ISPCSimulation.GetParticleVs(i);
			Softs::FSolverVec3* CurrentAnimation = ISPCSimulation.GetAnimationPositions(i);
			Softs::FSolverVec3* OldAnimation = ISPCSimulation.GetOldAnimationPositions(i);
			Softs::FPAndInvM* InvM = ISPCSimulation.GetParticlePandInvMs(i);
			if ((XSolver != nullptr) &&
				(VSolver != nullptr) &&
				(CurrentAnimation != nullptr) &&
				(OldAnimation != nullptr) &&
				(InvM != nullptr))
			{
				*XSolver = XSolverVec;
				*VSolver = VSolverVec;
				*CurrentAnimation = CurrentAnimationVec;
				*OldAnimation = OldAnimationVec;
				InvM->P = FPInvMData.P;
				InvM->InvM = FPInvMData.InvM;
			}
		}

		ISPCSimulation.ResetStartPose(0, NumParticles);

		ISPCSimulation.SetReferenceVelocityScale(
			GroupId,
			OldTransform,
			NewTransform,
			ReferenceVelocity,
			ReferenceAngularVelocity,
			EChaosSoftsSimulationSpace::ReferenceBoneSpace,
			LinearVelocity,
			MaxLinearVelocity,
			MaxLinearAcceleration,
			AngularVelocityScale,
			MaxAngularVelocity,
			MaxAngularAcceleration,
			FicticiousAngularScale,
			MaxVelocityScale,
			FClothingSimulationSolver::EFictitiousForcesModel::None);

		ISPCSimulation.Update(.5); // Calls ApplyPreSimulationTransform()
	}

	bChaos_PreSimulationTransforms_ISPC_Enabled = false;
	FClothingSimulationSolver CPPSimulation;
	{
		CPPSimulation.AddParticles(NumParticles, 0);
		CPPSimulation.EnableParticles(0, true);

		for (int32 i = 0; i < NumParticles; ++i)
		{
			Softs::FSolverVec3* XSolver = CPPSimulation.GetParticleXs(i);
			Softs::FSolverVec3* VSolver = CPPSimulation.GetParticleVs(i);
			Softs::FSolverVec3* CurrentAnimation = CPPSimulation.GetAnimationPositions(i);
			Softs::FSolverVec3* OldAnimation = CPPSimulation.GetOldAnimationPositions(i);
			Softs::FPAndInvM* InvM = CPPSimulation.GetParticlePandInvMs(i);
			if ((XSolver != nullptr) && 
				(VSolver != nullptr) && 
				(CurrentAnimation != nullptr) &&
				(OldAnimation != nullptr) &&
				(InvM != nullptr))
			{
				*XSolver = XSolverVec;
				*VSolver = VSolverVec;
				*CurrentAnimation = CurrentAnimationVec;
				*OldAnimation = OldAnimationVec;
				InvM->P = FPInvMData.P;
				InvM->InvM = FPInvMData.InvM;
			}
		}

		CPPSimulation.ResetStartPose(0, NumParticles);

		CPPSimulation.SetReferenceVelocityScale(
			GroupId,
			OldTransform,
			NewTransform,
			ReferenceVelocity,
			ReferenceAngularVelocity,
			EChaosSoftsSimulationSpace::ReferenceBoneSpace,
			LinearVelocity,
			MaxLinearVelocity,
			MaxLinearAcceleration,
			AngularVelocityScale,
			MaxAngularVelocity,
			MaxAngularAcceleration,
			FicticiousAngularScale,
			MaxVelocityScale,
			FClothingSimulationSolver::EFictitiousForcesModel::None);

		CPPSimulation.Update(.5); // Calls ApplyPreSimulationTransform()
	}

	bChaos_PreSimulationTransforms_ISPC_Enabled = InitialState;

	for (int32 i = 0; i < NumParticles; ++i)
	{
		{
			Softs::FSolverVec3& ISPCXSolver = *ISPCSimulation.GetParticleXs(i);
			Softs::FSolverVec3& CPPXSolver = *CPPSimulation.GetParticleXs(i);
			const FString Message(FString::Format(TEXT("X Solver {0}: ISPC {1}, CPP {2}"), { i, ISPCXSolver.ToString(), CPPXSolver.ToString() }));
			TestTrue(*Message, ISPCXSolver.Equals(CPPXSolver));
		}

		{
			Softs::FSolverVec3& ISPCVSolver = *ISPCSimulation.GetParticleVs(i);
			Softs::FSolverVec3& CPPVSolver = *CPPSimulation.GetParticleVs(i);
			const FString Message(FString::Format(TEXT("V Solver {0}: ISPC {1}, CPP {2}"), { i, ISPCVSolver.ToString(), CPPVSolver.ToString() }));
			TestTrue(*Message, ISPCVSolver.Equals(CPPVSolver));
		}

		{
			Softs::FPAndInvM& ISPCInvM = *ISPCSimulation.GetParticlePandInvMs(i);
			Softs::FPAndInvM& CPPInvM = *CPPSimulation.GetParticlePandInvMs(i);

			const FString PMessage(FString::Format(TEXT("P Vec {0}: ISPC {1}, CPP {2}"), { i, ISPCInvM.P.ToString(), CPPInvM.P.ToString() }));
			const FString InvMMessage(FString::Format(TEXT("InvM {0}: ISPC {1}, CPP {2}"), { i, ISPCInvM.InvM, CPPInvM.InvM }));
			TestTrue(*PMessage, ISPCInvM.P.Equals(CPPInvM.P));
			TestEqual(*InvMMessage, ISPCInvM.InvM, CPPInvM.InvM);
		}
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
