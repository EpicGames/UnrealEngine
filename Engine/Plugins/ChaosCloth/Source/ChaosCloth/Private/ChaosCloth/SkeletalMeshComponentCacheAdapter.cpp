// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCloth/SkeletalMeshComponentCacheAdapter.h"
#include "ChaosCloth/ChaosClothingCacheSchema.h"
#include "ChaosCloth/ChaosClothingSimulationFactory.h"
#include "ChaosCloth/ChaosClothPrivate.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ChaosCache.h"
#include "Chaos/CacheManagerActor.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "Components/SkeletalMeshComponent.h"
#include "ClothingSimulationInstance.h"

namespace Chaos
{
	FComponentCacheAdapter::SupportType FSkeletalMeshCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
	{
		const UClass* Desired = GetDesiredClass();
		if(InComponentClass == Desired)
		{
			return FComponentCacheAdapter::SupportType::Direct;
		}
		else if(InComponentClass->IsChildOf(Desired))
		{
			return FComponentCacheAdapter::SupportType::Derived;
		}

		return FComponentCacheAdapter::SupportType::None;
	}

	UClass* FSkeletalMeshCacheAdapter::GetDesiredClass() const
	{
		return USkeletalMeshComponent::StaticClass();
	}

	uint8 FSkeletalMeshCacheAdapter::GetPriority() const
	{
		return EngineAdapterPriorityBegin;
	}

	void FSkeletalMeshCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComponent, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const
	{
		if( const FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			FClothingCacheSchema::RecordPostSolve(*ClothSolver, OutFrame, InTime);			
		}
	}

	void FSkeletalMeshCacheAdapter::Playback_PreSolve(UPrimitiveComponent* InComponent, UChaosCache* InCache, Chaos::FReal InTime, FPlaybackTickRecord& TickRecord, TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const
	{
		check(InCache);
		if (FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			FClothingCacheSchema::PlaybackPreSolve(*InCache, InTime, TickRecord, *ClothSolver);
		}
	}

	FGuid FSkeletalMeshCacheAdapter::GetGuid() const
	{
		FGuid NewGuid;
		checkSlow(FGuid::Parse(TEXT("4328BED8C52D4E07BE0CA2433940AF6D"), NewGuid));
		return NewGuid;
	}

	bool FSkeletalMeshCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
	{
		// If we have a skel mesh we can play back any cache as long as it has one or more tracks
		const USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent);
		return MeshComp && MeshComp->GetSkinnedAsset() && Chaos::FClothingCacheSchema::CacheIsValidForPlayback(InCache);
	}

	FClothingSimulationSolver* FSkeletalMeshCacheAdapter::GetClothSolver(UPrimitiveComponent* InComponent) const
	{
		if(InComponent)
		{
			if( USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent) )
			{
				if (MeshComp->GetClothingSimulationInstances().Num() > 1)
				{
					static bool bErroredOnce = false;
					UE_CLOGF(!bErroredOnce, LogChaosCloth, Error,
						"SkeletalMeshComponent [%ls] has more than one clothing simulation running, but cloth caching can only deal with a single one.", *MeshComp->GetName());
					bErroredOnce = true;
				}

				for (FClothingSimulationInstance& ClothingSimulationInstance : MeshComp->GetClothingSimulationInstances())
				{
					if (ClothingSimulationInstance.GetClothingSimulation() && ClothingSimulationInstance.GetClothingSimulationFactory())
					{
						// TODO: Manage multiple solvers and solver types as to process all instances
						if (ClothingSimulationInstance.GetClothingSimulationFactory()->GetClass() == UChaosClothingSimulationFactory::StaticClass())
						{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
							return StaticCast<FClothingSimulation*>(ClothingSimulationInstance.GetClothingSimulation())->Solver.Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
						}
					}
				}
			}
		}
		return nullptr;
	}

	FPhysicsSolverEvents* FSkeletalMeshCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
	{
		if(USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent))
		{
			// Initialize the physics solver at the beginning of the play/record
			MeshComp->RecreatePhysicsState();
			MeshComp->RecreateClothingActors();

			return GetClothSolver(InComponent);
		}
		return nullptr;
	}
	
	FPhysicsSolver* FSkeletalMeshCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
	{
		return nullptr;
	}
	
	void FSkeletalMeshCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, FReal InTime) const
	{
		if (!InCache || InCache->GetDuration() == 0.0f)
		{
			return;
		}

		if(USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent))
		{
			InitializeForRestState(InComponent);

			if( MeshComp->GetClothingSimulationContext())
			{
				FClothingSimulationCacheData& SimulationContextData =
						StaticCast<FClothingSimulationContext*>(MeshComp->GetClothingSimulationContext())->CacheData;

				FClothingCacheSchema::LoadCacheData(InCache, InTime, SimulationContextData);
			}
		}
	}

	void FSkeletalMeshCacheAdapter::InitializeForRestState(UPrimitiveComponent* InComponent) const
	{
		if(USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent))
		{
#if WITH_EDITOR
			if(!MeshComp->GetUpdateClothInEditor() || !MeshComp->GetUpdateAnimationInEditor())
			{
				MeshComp->SetUpdateAnimationInEditor(true);
				MeshComp->SetUpdateClothInEditor(true);
			}
#endif
			// Initialize the physics solver if the cloth/animation in editor are not set
			MeshComp->RecreatePhysicsState();
			MeshComp->RecreateClothingActors();
		}

		if( FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			ClothSolver->SetEnableSolver(false);
		}
	}

	bool FSkeletalMeshCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache)
	{
		if( FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			ClothSolver->SetEnableSolver(true);
		}
		return true;
	}

	bool FSkeletalMeshCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime)
	{
		EnsureIsInGameThreadContext();
		
		if( FClothingSimulationSolver* ClothSolver = GetClothSolver(InComponent))
		{
			ClothSolver->SetEnableSolver(false);
		}
		return true;
	}

	void FSkeletalMeshCacheAdapter::WaitForSolverTasks(UPrimitiveComponent* InComponent) const
	{
		EnsureIsInGameThreadContext();

		if (USkeletalMeshComponent* MeshComp = CastChecked<USkeletalMeshComponent>(InComponent))
		{
			MeshComp->WaitForExistingParallelClothSimulation_GameThread();
		}
	}
}    // namespace Chaos
