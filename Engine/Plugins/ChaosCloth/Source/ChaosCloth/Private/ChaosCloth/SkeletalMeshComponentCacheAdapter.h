// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Adapters/CacheAdapter.h"

namespace Chaos
{
	class FClothingSimulationSolver;
	
	/**
	 * Skeletal mesh cache adapter to be able to cache cloth simulation datas through the chaos cache system
	 */
	class FSkeletalMeshCacheAdapter : public FComponentCacheAdapter
	{
	public:
		
		virtual ~FSkeletalMeshCacheAdapter() = default;

		// ~Begin FComponentCacheAdapter interface
		virtual SupportType            SupportsComponentClass(UClass* InComponentClass) const override;
		virtual UClass*                GetDesiredClass() const override;
		virtual uint8                  GetPriority() const override;
		virtual FGuid                  GetGuid() const override;
		virtual bool                   ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const override;
		virtual Chaos::FPhysicsSolver* GetComponentSolver(UPrimitiveComponent* InComponent) const override;
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const override;
		virtual void				   SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const override;
		virtual bool                   InitializeForRecord(UPrimitiveComponent* InComponent, UChaosCache* InCache) override;
		virtual bool                   InitializeForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache, float InTime) override;
		virtual void                   Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const override;
		virtual void                   Playback_PreSolve(UPrimitiveComponent* InComponent,
										 UChaosCache*										InCache,
										 Chaos::FReal                                       InTime,
										 FPlaybackTickRecord&								TickRecord,
										 TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const override;
		virtual void                   WaitForSolverTasks(UPrimitiveComponent* InComponent) const override;
		// ~End FComponentCacheAdapter interface

	private :

		/** Return the adapter component cloth solver
		 * @param InComponent skelmesh component from which the cloth solver will be extracted
		 * @return Cloth solver used by the skelmesh component
		 */
		Chaos::FClothingSimulationSolver* GetClothSolver(UPrimitiveComponent* InComponent) const;

		/** Initialize the skelmesh adapter when changing the rest state 
		* @param InComponent skelmesh component from which the cloth solver will be extracted
		*/
		void InitializeForRestState(UPrimitiveComponent* InComponent) const;

	};
}    // namespace Chaos
