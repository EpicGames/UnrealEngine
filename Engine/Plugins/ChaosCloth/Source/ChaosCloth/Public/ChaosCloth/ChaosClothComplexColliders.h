// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollection.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"

namespace Chaos
{
	namespace Softs
	{
		struct FParticleRangeIndex;
		struct FPBDComplexColliderBoneData;
	}

	class FClothComplexColliders final
	{
	public:
		FClothComplexColliders(
			Softs::FEvolution* InEvolution,
			int32 InCollisionRangeId);

		~FClothComplexColliders() = default;

		int32 GetCollisionRangeId() const { return CollisionRangeId; }

		// --- Solver interface ----
		void SwapBuffersForFrameFlip();
		void KinematicUpdate(const Softs::FSolverCollisionParticlesRange& Particles, const Softs::FSolverReal Dt, const Softs::FSolverReal Alpha);
		void ApplyPreSimulationTransforms(const Softs::FSolverRigidTransform3& PreSimulationTransform, const Softs::FSolverVec3& DeltaLocalSpaceLocation,
			const TConstArrayView<Softs::FSolverRigidTransform3>& OldParticleTransforms, const TConstArrayView<Softs::FSolverRigidTransform3>& ParticleTransforms, const Softs::FSolverReal Dt);
		void SetSkipSkinnedTriangleMeshKinematicUpdate(bool bSkip)
		{
			bSkipSkinnedTriangleMeshKinematicUpdate = bSkip;
		}
		// ---- End Solver interface ----

		// ---- Collider interface ----
		// Warning: changing subbone indices can cause any extracted FPBDComplexColliderBoneData to go stale. You must re-extract after calling these methods.
		void Reset();
		void AddSubBoneIndices(const TArray<int32>& SubBoneIndices);

		void AddSkinnedLevelSet(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& SkinnedLevelSet);
		void AddMLLevelSet(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& MLLevelSet);
		void AddSkinnedTriangleMesh(int32 Index, const TArray<int32>& MappedSubBones, const FImplicitObjectPtr& SkinnedTriangleMesh);
		void Update(const Softs::FSolverTransform3& ComponentToLocalSpace, const TArray<FTransform>& BoneTransforms, const TConstArrayView<Softs::FSolverRigidTransform3>& CollisionRangeTransforms);
		void ResetStartPose();

		void SetMinLODSize(const Softs::FSolverReal InMinLODSize)
		{
			MinLODSize = InMinLODSize;
		}
		// ---- End of Collider interface ----

		// -- Constraints interface ----
		CHAOSCLOTH_API void ExtractComplexColliderBoneData(TMap<Softs::FParticleRangeIndex, Softs::FPBDComplexColliderBoneData>& Data) const;
		int32 GetNumSkinnedTriangleMeshes() const
		{
			return SkinnedTriangleMeshes.Num();
		}
		const FImplicitObjectPtr& GetSkinnedTriangleMesh(int32 Index) const
		{
			return SkinnedTriangleMeshes[Index].SkinnedTriangleMesh;
		}
		int32 GetSkinnedTriangleMeshIndex(int32 Index) const 
		{
			return SkinnedTriangleMeshes[Index].Index;
		}
		const TArray<Softs::FSolverVec3>& GetSkinnedTriangleMeshVelocities(int32 Index) const
		{
			return SkinnedTriangleMeshes[Index].SkinnedPositions.SolverSpaceVelocities;
		}
		// -- End of Constraints interface ----
	private:

		Softs::FEvolution* Evolution;
		int32 CollisionRangeId;

		struct FCollisionSubBones : public TArrayCollection
		{
			FCollisionSubBones()
			{
				TArrayCollection::AddArray(&BoneIndices);
				TArrayCollection::AddArray(&BaseTransforms);
				TArrayCollection::AddArray(&OldTransforms);
				TArrayCollection::AddArray(&Transforms);
				TArrayCollection::AddArray(&X);
				TArrayCollection::AddArray(&V);
				TArrayCollection::AddArray(&R);
				TArrayCollection::AddArray(&W);
			}

			void Reset()
			{
				ResizeHelper(0);
			}

			void AddSubBones(int32 Num)
			{
				check(Num >= 0);
				ResizeHelper(Size() + Num);
			}

			TArrayCollectionArray<int32> BoneIndices;
			TArrayCollectionArray<Softs::FSolverRigidTransform3> BaseTransforms;
			TArrayCollectionArray<Softs::FSolverRigidTransform3> OldTransforms;
			TArrayCollectionArray<Softs::FSolverRigidTransform3> Transforms;
			TArrayCollectionArray<Softs::FSolverVec3> X;
			TArrayCollectionArray<Softs::FSolverVec3> V;
			TArrayCollectionArray<Softs::FSolverRotation3> R;
			TArrayCollectionArray<Softs::FSolverVec3> W;
		};
		FCollisionSubBones CollisionSubBones;

		struct FSkinnedLevelSetData
		{
			int32 Index;
			TArray<int32> MappedSubBones;
			FImplicitObjectPtr SkinnedLevelSet;
		};
		TArray<FSkinnedLevelSetData> SkinnedLevelSets;

		struct FMLLevelSetData
		{
			int32 Index;
			TArray<int32> MappedSubBones;
			FImplicitObjectPtr MLLevelSet;
		};
		TArray<FMLLevelSetData> MLLevelSets;

		struct FSkinnedTriangleMeshData
		{
			int32 Index;
			TArray<int32> MappedSubBones;
			FImplicitObjectPtr SkinnedTriangleMesh; // Note: SkinnedTriangleMesh LocalPositions will be in solver/particle space, not collision space.
			struct FSkinnedPositions : public TArrayCollection
			{
				FSkinnedPositions()
				{
					TArrayCollection::AddArray(&OldPositions);
					TArrayCollection::AddArray(&Positions);
					TArrayCollection::AddArray(&SolverSpaceVelocities);
				}

				void SetNum(int32 Num)
				{
					ResizeHelper(Num);
				}

				TArrayCollectionArray<Softs::FSolverVec3> OldPositions;
				TArrayCollectionArray<Softs::FSolverVec3> Positions;
				TArrayCollectionArray<Softs::FSolverVec3> SolverSpaceVelocities;
				// Interpolated positions are on the SkinnedTriangleMesh
			} SkinnedPositions;
		};
		TArray<FSkinnedTriangleMeshData> SkinnedTriangleMeshes;
		bool bSkipSkinnedTriangleMeshKinematicUpdate = false;
		Softs::FSolverReal MinLODSize = 0.f;

	};
} // namespace Chaos
