// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/SoftsSimulationSpace.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "ClothConfig.h"
#include "CoreMinimal.h"
#include "ChaosClothConfig.generated.h"

USTRUCT()
struct FChaosClothWeightedValue
{
	GENERATED_BODY()

	/**
	 * Parameter value corresponding to the lower bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "Low Weight", ChaosClothShortName = "Lo"))
	float Low = 0.f;

	/**
	 * Parameter value corresponding to the upper bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "High Weight", ChaosClothShortName = "Hi"))
	float High = 1.f;
};

/**
 * Long range attachment options.
 * Deprecated.
 */
UENUM()
enum class EChaosClothTetherMode : uint8
{
	// Fast Tether Fast Length: Use fast euclidean methods to both setup the tethers and calculate their lengths. Fast initialization and simulation times, but is very prone to artifacts.
	FastTetherFastLength,
	// Accurate Tether Fast Length: Use the accurate geodesic method to setup the tethers and a fast euclidean method to calculate their lengths. Slow initialization times and fast simulation times, but can still be prone to artifacts.
	AccurateTetherFastLength,
	// Accurate Tether Accurate Length: Use accurate geodesic method to both setup the tethers and calculate their lengths. Slow initialization and simulation times, but this is the most accurate setting showing the less artifacts.
	AccurateTetherAccurateLength UMETA(Hidden),
	MaxChaosClothTetherMode UMETA(Hidden)
};

/** Holds initial, asset level config for clothing actors. */
// Hiding categories that will be used in the future
UCLASS(HideCategories = (Collision), MinimalAPI)
class UChaosClothConfig : public UClothConfigCommon
{
	GENERATED_BODY()
public:
	CHAOSCLOTH_API UChaosClothConfig();
	CHAOSCLOTH_API virtual ~UChaosClothConfig() override;

	/** Migrate from the legacy FClothConfig structure. */
	CHAOSCLOTH_API virtual void MigrateFrom(const FClothConfig_Legacy&) override;

	/** Migrate from shared config. */
	CHAOSCLOTH_API virtual void MigrateFrom(const UClothSharedConfigCommon* ClothSharedConfig) override;

	/** Serialize override used to set the current custom version. */
	CHAOSCLOTH_API virtual void Serialize(FArchive& Ar) override;

	/** PostLoad override used to deal with updates/changes in properties. */
	CHAOSCLOTH_API virtual void PostLoad() override;

	virtual bool NeedsSelfCollisionData() const override { return bUseSelfCollisionSpheres && SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier > 0.f; }

	/** Return wherether to pre-compute Inverse Masses. */
	virtual bool NeedsInverseMasses() const override { return false; }  // TODO: Chaos Cloth uses the mass mode enum, and this will require a little refactor to work

	/** Return wherether to pre-compute the Long Range Attachment tethers. */
	virtual bool NeedsTethers() const override { return TetherStiffness.Low > 0.f || TetherStiffness.High > 0.f; }

	/** Return the self collision radius to precomute self collision data. */
	virtual float GetSelfCollisionRadius() const override { return SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier; }

	/** Return whether tethers need to be calculated using geodesic distances instead of eclidean. */
	virtual bool TethersUseGeodesicDistance() const override { return bUseGeodesicDistance; }

	/** Return the mass value, from whichever mass mode (Density, UniformMass, or TotalMass) is selected. */
	CHAOSCLOTH_API float GetMassValue() const;

	/**
	 * How cloth particle mass is determined
	 * -	Uniform Mass: Every particle's mass will be set to the value specified in the UniformMass setting. Mostly to avoid as it can cause some serious issues with irregular tessellations.
	 * -	Total Mass: The total mass is distributed equally over all the particles. Useful when referencing a specific garment size and feel.
	 * -	Density: A constant mass density is used. Density is usually the preferred way of setting mass since it allows matching real life materials' density values.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass Properties")
	EClothMassMode MassMode = EClothMassMode::Density;

	/** The value used when the Mass Mode is set to Uniform Mass. */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", meta = (UIMin = "0.000001", UIMax = "0.001", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::UniformMass"))
	float UniformMass = 0.00015f;

	/** The value used when Mass Mode is set to TotalMass. */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", meta = (UIMin = "0.001", UIMax = "10", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::TotalMass"))
	float TotalMass = 0.5f;

	/**
	 * The value used when Mass Mode is set to Density.
	 * Melton Wool: 0.7
	 * Heavy leather: 0.6
	 * Polyurethane: 0.5
	 * Denim: 0.4
	 * Light leather: 0.3
	 * Cotton: 0.2
	 * Silk: 0.1
	 */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", meta = (UIMin = "0.001", UIMax = "1", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::Density"))
	float Density = 0.35f;

	/** This is a lower bound to cloth particle masses. */
	UPROPERTY()
	float MinPerParticleMass = 0.0001f;

	/**
	 * The Stiffness of segments constraints. Increase the iteration count for stiffer materials.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Edge Stiffness" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	 */
	UPROPERTY(EditAnywhere, Category = "Material Properties", DisplayName = "Edge Stiffness", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue EdgeStiffnessWeighted = { 1.f, 1.f };

	/**
	 * The Stiffness of cross segments and bending elements constraints. Increase the iteration count for stiffer materials.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Bend Stiffness" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	 */
	UPROPERTY(EditAnywhere, Category = "Material Properties", DisplayName = "Bending Stiffness", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue BendingStiffnessWeighted = { 1.f, 1.f };

	/** Enable the more accurate bending element constraints instead of the faster cross-edge spring constraints used for controlling bending stiffness. */
	UPROPERTY(EditAnywhere, Category = "Material Properties")
	bool bUseBendingElements = false;

	/**
	* Once the element has bent such that it's folded more than this ratio from its rest angle ("buckled"), switch to using Buckling Stiffness instead of Bending Stiffness.
	* When Buckling Ratio = 0, the Buckling Stiffness will never be used. When BucklingRatio = 1, the Buckling Stiffness will be used as soon as its bent past its rest configuration.
	*/
	UPROPERTY(EditAnywhere, Category = "Material Properties", DisplayName = "Buckling Ratio", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "bUseBendingElements"))
	float BucklingRatio = 0.f;

	/**
	* Bending will use this stiffness instead of Bending Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	* Typically, Buckling Stiffness is set to be less than Bending Stiffness. Buckling Ratio determines the switch point between using Bending Stiffness and Buckling Stiffness.
	* If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Buckling Stiffness" is added to the cloth, 
	* then both the Low and High values will be used in conjunction with the per particle Weight stored in the Weight Map to interpolate the final value from them.
	* Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	*/
	UPROPERTY(EditAnywhere, Category = "Material Properties", DisplayName = "Buckling Stiffness", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "bUseBendingElements"))
	FChaosClothWeightedValue BucklingStiffnessWeighted = { 1.f, 1.f };

	/**
	* Calculate rest angles as a ratio between completely flat and whatever is the 3D rest angle.
	* When FlatnessRatio = 0, the rest angle will match the input mesh..
	* When FlatnessRatio = 1, the rest angle will be 0 (completely flat).
	* If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Flatness Ratio" is added to the cloth, 
	* then both the Low and High values will be used in conjunction with the per particle Weight stored in the Weight Map to interpolate the final value from them.
	* Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	*/
	UPROPERTY(EditAnywhere, Category = "Material Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "bUseBendingElements"))
	FChaosClothWeightedValue FlatnessRatio = { 0.f, 1.f };

	/**
	 * The stiffness of the surface area preservation constraints. Increase the iteration count for stiffer materials.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Bend Stiffness" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	 */
	UPROPERTY(EditAnywhere, Category = "Material Properties", DisplayName = "Area Stiffness", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue AreaStiffnessWeighted = { 1.f, 1.f };

	/** The stiffness of the volume preservation constraints. */
	UPROPERTY()
	float VolumeStiffness = 0.f;

	/**
	 * The tethers' stiffness of the long range attachment constraints.
	 * The long range attachment connects each of the cloth particles to its closest fixed point with a spring constraint.
	 * This can be used to compensate for a lack of stretch resistance when the iterations count is kept low for performance reasons.
	 * Can lead to an unnatural pull string puppet like behavior.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Tether Stiffness" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	 * Use 0, 0 to disable.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue TetherStiffness = { 1.f, 1.f };

	/**
	 * The limit scale of the long range attachment constraints (aka tether limit).
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Tether Scale" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the tethers' scale.
	*/
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment", meta = (UIMin = "1.", UIMax = "1.1", ClampMin = "0.01", ClampMax = "10"))
	FChaosClothWeightedValue TetherScale = { 1.f, 1.f };

	/**
	 * Use geodesic instead of euclidean distance calculations for the Long Range Attachment constraint,
	 * which is slower at setup but more accurate at establishing the correct position and length of the tethers,
	 * and therefore is less prone to artifacts during the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment")
	bool bUseGeodesicDistance = true;

	/** The stiffness of the shape target constraints. */
	UPROPERTY()
	float ShapeTargetStiffness = 0.f;

	/** The added thickness of collision shapes. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	float CollisionThickness = 1.0f;

	/** Friction coefficient for cloth - collider interaction. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10"))
	float FrictionCoefficient = 0.8f;

	/**
	 * Use continuous collision detection (CCD) to prevent any missed collisions between fast moving particles and colliders.
	 * This has a negative effect on performance compared to when resolving collision without using CCD.
	 */
	UPROPERTY(EditAnywhere, Category = "Collision Properties")
	bool bUseCCD = false;

	/** Enable self collision repulsion forces (point-face). */
	UPROPERTY(EditAnywhere, Category = "Collision Properties")
	bool bUseSelfCollisions = false;

	/** The radius of the spheres used in self collision. (i.e., offset per side. total thickness of cloth is 2x this value) */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", EditCondition = "bUseSelfCollisions"))
	float SelfCollisionThickness = 2.0f;

	/**Friction coefficient for cloth - cloth interaction. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "bUseSelfCollisions"))
	float SelfCollisionFriction = 0.0f;

	/** Enable self intersection resolution. This will try to fix any cloth intersections that are not handled by collision repulsions. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (EditCondition = "bUseSelfCollisions"))
	bool bUseSelfIntersections = false;

	/** Enable sphere-based self collision repulsion forces. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties",meta = (EditCondition = "!bUseSelfCollisions"))
	bool bUseSelfCollisionSpheres = false;

	/** The radius of the spheres used in self collision centered at each vertex. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (EditCondition = "!bUseSelfCollisions && bUseSelfCollisionSpheres", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	float SelfCollisionSphereRadius = 0.5f;

	/** The stiffness of the springs used to control self collision. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (EditCondition = "!bUseSelfCollisions && bUseSelfCollisionSpheres", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	float SelfCollisionSphereStiffness = 1.f;

	/** 
	 * Multiplier for culling the self collision spheres. Spheres are seeded on every vertex,
	 * and culled based on SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier.
	 */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", meta = (EditCondition = "!bUseSelfCollisions && bUseSelfCollisionSpheres", ClampMin = "1"))
	float SelfCollisionSphereRadiusCullMultiplier = 1.f;

	/**
	 * This parameter is automatically set by the migration code. It can be overridden here to use the old way of authoring the backstop distances.
	 * The legacy backstop requires the sphere radius to be included within the painted distance mask, making it difficult to author correctly. In this case the backstop distance is the distance from the animated mesh to the center of the corresponding backstop collision sphere.
	 * The non legacy backstop automatically adds the matching sphere's radius to the distance calculations at runtime to make for a simpler authoring of the backstop distances. In this case the backstop distance is the distance from the animated mesh to the surface of the backstop collision sphere.
	 * In both cases, a positive backstop distance goes against the corresponding animated mesh's normal, and a negative backstop distance goes along the corresponding animated mesh's normal.
	 */
	UPROPERTY(EditAnywhere, Category = "Collision Properties")
	bool bUseLegacyBackstop = false;

	/**
	 * The amount of global damping applied to the cloth velocities, also known as point damping.
	 * Point damping improves simulation stability, but can also cause an overall slow-down effect and therefore is best left to very small percentage amounts.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float DampingCoefficient = 0.01f;

	/**
	 * The amount of local damping applied to the cloth velocities.
	 * This type of damping only damps individual deviations of the particles velocities from the global motion.
	 * It makes the cloth deformations more cohesive and reduces jitter without affecting the overall movement.
	 * It can also produce synchronization artifacts where part of the cloth mesh are disconnected (which might well be desirable, or not), and is more expensive than global damping.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float LocalDampingCoefficient = 0.f;

	/**
	 * This parameter is automatically set by the migration code. It can be overridden here to use the old deprecated "Legacy" wind model in order to preserve behavior with previous versions of the engine.
	 * The old wind model is not an accurate aerodynamic model and as such should be avoided. Being point based, it doesn't take into account the surface area that gets hit by the wind.
	 * Using this model makes the simulation slightly slower, disables the aerodynamically accurate wind model, and prevents the cloth to interact with the surrounding environment (air, water, ...etc.) even when there is no wind.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties")
	bool bUsePointBasedWindModel = false;

	/**
	 * The aerodynamic coefficient of drag applying on each particle. 
	 * When "Outer Drag" is enabled, this acts as the "Inner Drag", i.e., drag applied when the air velocity is
	 * moving in the mesh normal direction.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Drag" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the aerodynamic drag.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "!bUsePointBasedWindModel"))
	FChaosClothWeightedValue Drag = { 0.035f, 1.f };

	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (InlineEditConditionToggle))
	bool bEnableOuterDrag = false;

	/**
	 * The aerodynamic coefficient of drag applying on each particle when the air velocity is moving
	 * against the mesh normal direction.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Outer Drag" is added to the cloth,
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the aerodynamic drag.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "bEnableOuterDrag"))
	FChaosClothWeightedValue OuterDrag = { 0.035f, 1.f };

	/**
	 * The aerodynamic coefficient of lift applying on each particle.
	 * When "Outer Lift" is enabled, this acts as the "Inner lift", i.e., lift applied when the air velocity is
	 * moving in the mesh normal direction.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Lift" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the aerodynamic lift.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "!bUsePointBasedWindModel"))
	FChaosClothWeightedValue Lift = { 0.035f, 1.f };

	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (InlineEditConditionToggle))
	bool bEnableOuterLift = false;

	/**
	 * The aerodynamic coefficient of lift applying on each particle when the air velocity is moving
	 * against the mesh normal direction.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Outer Lift" is added to the cloth,
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the aerodynamic lift.
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "bEnableOuterLift"))
	FChaosClothWeightedValue OuterLift = { 0.035f, 1.f };

	// Use the config gravity value instead of world gravity.
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (InlineEditConditionToggle))
	bool bUseGravityOverride = false;

	// Scale factor applied to the world gravity and also to the clothing simulation interactor gravity. Does not affect the gravity if set using the override below.
	UPROPERTY(EditAnywhere, Category = "Environmental Properties")
	float GravityScale = 1.f;

	// The gravitational acceleration vector [cm/s^2]
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (EditCondition = "bUseGravityOverride"))
	FVector Gravity = { 0.f, 0.f, -980.665f };

	/** 
	 * Pressure force strength applied in the normal direction(use negative value to push toward backface)
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Pressure" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to set the pressure.	 
	 */
	UPROPERTY(EditAnywhere, Category = "Environmental Properties", meta = (UIMin = "-10", UIMax = "10", ClampMin = "-100", ClampMax = "100"))
	FChaosClothWeightedValue Pressure = { 0.0f, 1.f };

	/**
	 * The strength of the constraint driving the cloth towards the animated goal mesh.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Anim Drive Stiffness" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is meaningful and sufficient to enable this constraint.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue AnimDriveStiffness = { 0.f, 1.f };

	/**
	 * The damping amount of the anim drive.
	 * If an enabled Weight Map (Mask with values in the range [0;1]) targeting the "Anim Drive Damping" is added to the cloth, 
	 * then both the Low and High values will be used in conjunction with the per particle Weight stored
	 * in the Weight Map to interpolate the final value from them.
	 * Otherwise only the Low value is sufficient to work on this constraint.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothWeightedValue AnimDriveDamping = { 0.f, 1.f };

	/**
	 * All vector properties in this section (e.g., Linear Velocity Scale, Max Linear Acceleration)
	 * will be evaluated in this space.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties")
	EChaosSoftsSimulationSpace VelocityScaleSpace = EChaosSoftsSimulationSpace::ReferenceBoneSpace;
	/**
	 * The amount of linear velocities sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FVector LinearVelocityScale = { 0.75f, 0.75f, 0.75f };

	/**
	 * Enable linear velocity clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (InlineEditConditionToggle))
	bool bEnableLinearVelocityClamping = false;

	/**
	 * The maximum amount of linear velocity sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (ClampMin = "0", EditCondition = "bEnableLinearVelocityClamping"))
	FVector3f MaxLinearVelocity = { 1000.f, 1000.f, 1000.f }; // Approx 22mph or 36kph per direction

	/**
	 * Enable linear acceleration clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (InlineEditConditionToggle))
	bool bEnableLinearAccelerationClamping = false;

	/**
	 * The maximum amount of linear acceleration sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (ClampMin = "0", EditCondition = "bEnableLinearAccelerationClamping"))
	FVector3f MaxLinearAcceleration = { 60000.f, 60000.f, 60000.f };

	/**
	 * The amount of angular velocities sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float AngularVelocityScale = 0.75f;

	/**
	 * Enable angular velocity clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (InlineEditConditionToggle))
	bool bEnableAngularVelocityClamping = false;

	/**
	 * The maximum amount of angular velocity sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (ClampMin = "0", EditCondition = "bEnableAngularVelocityClamping"))
	float MaxAngularVelocity = 200.f;

	/**
	 * Enable angular acceleration clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (InlineEditConditionToggle))
	bool bEnableAngularAccelerationClamping = false;

	/**
	 * The maximum amount of angular acceleration sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (ClampMin = "0", EditCondition = "bEnableAngularAccelerationClamping"))
	float MaxAngularAcceleration = 12000.f;

	/**
	 * Fictitious forces model.
	 * Decide how fictitious forces (e.g. centrifugal force) are applied to the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties")
	EChaosClothingSimulationSolverFictitiousForcesModel FictitiousForcesModel = EChaosClothingSimulationSolverFictitiousForcesModel::FullScale;

	/**
	 * The portion of the angular velocity that is used to calculate the strength of all fictitious forces (e.g. centrifugal force).
	 * In Legacy model, this parameter is only having an effect on the portion of the reference bone's angular velocity that has been
	 * removed from the simulation via the Angular Velocity Scale parameter. This means it has no effect when AngularVelocityScale is
	 * set to 1 and Angular Velocity and Acceleration clamps are disabled, in which case the cloth is simulated with full world space
	 * angular velocities and subjected to the true physical world inertial forces.
	 * In FullScale model, this parameter has also an effect when the cloth is simulated with full world space angular velocities, and
	 * isn't affected by the Angular Velocity Scale parameter.
	 * Values range from 0 to 2, with 0 showing no centrifugal effect, 1 full centrifugal effect, and 2 an overdriven centrifugal effect.
	 */
	UPROPERTY(EditAnywhere, Category = "Animation Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "2", EditCondition = "FictitiousForcesModel != EChaosClothingSimulationSolverFictitiousForcesModel::None"))
	float FictitiousAngularScale = 1.f;

	/** Enable tetrahedral constraints. */
	UPROPERTY()
	bool bUseTetrahedralConstraints = false;

	/** Enable thin shell volume constraints. */
	UPROPERTY()
	bool bUseThinShellVolumeConstraints = false;

	/** Enable continuous collision detection. */
	UPROPERTY()
	bool bUseContinuousCollisionDetection = false;

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float EdgeStiffness_DEPRECATED = 1.f;

	/** The Stiffness of the bending constraints. Increase the iteration count for stiffer materials. Increase the iteration count for stiffer materials. */
	UPROPERTY()
	float BendingStiffness_DEPRECATED = 1.f;

	/** The stiffness of the area preservation constraints. Increase the iteration count for stiffer materials. */
	UPROPERTY()
	float AreaStiffness_DEPRECATED = 1.f;

	UPROPERTY()
	EChaosClothTetherMode TetherMode_DEPRECATED = EChaosClothTetherMode::MaxChaosClothTetherMode;

	UPROPERTY()
	float LimitScale_DEPRECATED = 1.f;

	UPROPERTY()
	float DragCoefficient_DEPRECATED = 0.07f;

	UPROPERTY()
	float LiftCoefficient_DEPRECATED = 0.035f;

	UPROPERTY()
	float AnimDriveSpringStiffness_DEPRECATED = 0.f;

	UPROPERTY()
	float StrainLimitingStiffness_DEPRECATED = 0.5f;
#endif
};

/**
 * Chaos config settings shared between all instances of a skeletal mesh.
 * Unlike UChaosClothConfig, these settings contain common cloth simulation
 * parameters that cannot change between the various clothing assets assigned
 * to a specific skeletal mesh. @seealso UChaosClothConfig.
 */
UCLASS(MinimalAPI)
class UChaosClothSharedSimConfig : public UClothSharedConfigCommon
{
	GENERATED_BODY()
public:
	CHAOSCLOTH_API UChaosClothSharedSimConfig();
	CHAOSCLOTH_API virtual ~UChaosClothSharedSimConfig() override;

	/** Migrate from the legacy FClothConfig structure. */
	CHAOSCLOTH_API virtual void MigrateFrom(const FClothConfig_Legacy& ClothConfig) override;

	/** Serialize override used to set the current custom version. */
	CHAOSCLOTH_API virtual void Serialize(FArchive& Ar) override;

	/** PostLoad override used to deal with updates/changes in properties. */
	CHAOSCLOTH_API virtual void PostLoad() override;

#if WITH_EDITOR
	/** Called after changes in any of the asset properties. */
	CHAOSCLOTH_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& ChainEvent) override;
#endif

	/**
	 * The number of time step dependent solver iterations. This sets iterations at 60fps. 
	 * This will increase the stiffness of all constraints but will increase the CPU cost.
	 * If the frame rate is higher, the actual number of iterations used by the solver might be less, if the frame rate is lower it might be more.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100"))
	int32 IterationCount = 1;

	/**
	 * The maximum number of solver iterations.
	 * This is the upper limit of the number of iterations set in solver, when the frame rate is lower than 60fps.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100"))
	int32 MaxIterationCount = 10;

	/**
	 * The number of solver substeps.
	 * This will increase the precision of the collision inputs and help with constraint resolutions but will increase the CPU cost.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100"))
	int32 SubdivisionCount = 1;

#if WITH_EDITORONLY_DATA
	// The radius of the spheres used in self collision 
	UPROPERTY()
	float SelfCollisionThickness_DEPRECATED  = 2.0f;

	// The radius of cloth points when considering collisions against collider shapes.
	UPROPERTY()
	float CollisionThickness_DEPRECATED = 1.0f;

	// Use shared config damping rather than per cloth damping.
	UPROPERTY()
	bool bUseDampingOverride_DEPRECATED = true;

	// The amount of cloth damping. Override the per cloth damping coefficients.
	UPROPERTY()
	float Damping_DEPRECATED = 0.01f;

	// Use the config gravity value instead of world gravity.
	UPROPERTY()
	bool bUseGravityOverride_DEPRECATED = false;

	// Scale factor applied to the world gravity and also to the clothing simulation interactor gravity. Does not affect the gravity if set using the override below.
	UPROPERTY()
	float GravityScale_DEPRECATED = 1.f;

	// The gravitational acceleration vector [cm/s^2]
	UPROPERTY()
	FVector Gravity_DEPRECATED = { 0.f, 0.f, -980.665f };
#endif  // #if WITH_EDITORONLY_DATA


	// Enable local space simulation to help with jitter due to floating point precision errors if the character is far away from the world origin
	UPROPERTY()
	bool bUseLocalSpaceSimulation = true;

	// Enable the XPBD constraints that resolve stiffness independently from the number of iterations
	// Experimental, this feature might be removed without warning, not for production use
	UPROPERTY()
	bool bUseXPBDConstraints = false;
};
