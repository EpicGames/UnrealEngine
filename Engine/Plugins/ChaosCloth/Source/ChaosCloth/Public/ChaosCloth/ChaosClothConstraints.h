// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/PBDCollisionSpringConstraintsBase.h"
#include "Chaos/XPBDPlanarConstraints.h"
#include "Chaos/PBDExtremeDeformationConstraints.h"

namespace Buoyancy
{
	class FBuoyancyField;
}

struct FManagedArrayCollection;

namespace Chaos
{

	struct FClothingPatternData;
	struct FClothingAccessoryMeshData;
	class FClothComplexColliders;

	class FClothConstraints final
	{
	public:
		FClothConstraints();
		~FClothConstraints();

		// ---- Solver interface ----
		// Force-based solver
		void Initialize(
			Softs::FEvolution* InEvolution,
			FPerSolverFieldSystem* InPerSolverField,
			const TArray<Softs::FSolverVec3>& InInterpolatedAnimationPositions,
			const TArray<Softs::FSolverVec3>& InInterpolatedAnimationNormals,
			const TArray<Softs::FSolverVec3>& InAnimationVelocities,
			const TArray<Softs::FSolverVec3>& InNormals,
			const TArray<Softs::FSolverRigidTransform3>& InLastSubframeCollisionTransformsCCD,
			TArray<bool>& InCollisionParticleCollided,
			TArray<Softs::FSolverVec3>& InCollisionContacts,
			TArray<Softs::FSolverVec3>& InCollisionNormals,
			TArray<Softs::FSolverReal>& InCollisionPhis,
			int32 InParticleRangeId);

		// Force-based solver
		void UpdateFromSolver(const Softs::FSolverVec3& SolverGravity, bool bPerClothGravityOverrideEnabled,
			const Softs::FSolverVec3& FictitiousAngularVelocity, const Softs::FSolverVec3& ReferenceSpaceLocation,
			const Softs::FSolverVec3& SolverWindVelocity, const Softs::FSolverReal LegacyWindAdaptation);

		// PBD solver
		void Initialize(
			Softs::FPBDEvolution* InEvolution,
			const TArray<Softs::FSolverVec3>& InInterpolatedAnimationPositions,
			const TArray<Softs::FSolverVec3>& /*InOldAnimationPositions*/, // deprecated
			const TArray<Softs::FSolverVec3>& InInterpolatedAnimationNormals,
			const TArray<Softs::FSolverVec3>& InAnimationVelocities,
			int32 InParticleOffset,
			int32 InNumParticles);

		void SetSkipSelfCollisionInit(bool bValue) { bSkipSelfCollisionInit = bValue; }

		void OnCollisionRangeRemoved(int32 CollisionRangeId);
		// ---- End of Solver interface ----

		// ---- Cloth interface ----
		void AddRules(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const TMap<FString, const TSet<int32>*>& FaceSets,
			const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
			const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& Tethers,
			Softs::FSolverReal MeshScale,
			bool bEnabled,
			const TArray<const FClothComplexColliders*>& ComplexColliders = TArray<const FClothComplexColliders*>(),
			const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection = TSharedPtr<const FManagedArrayCollection>(),
			const TMap<FName, FClothingAccessoryMeshData>* AccessoryMeshes = nullptr);

		UE_DEPRECATED(5.8, "MultiRes constraint is no longer supported. Use other version of AddRules instead.")
		void AddRules(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const TMap<FString, const TSet<int32>*>& FaceSets,
			const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
			const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& Tethers,
			Softs::FSolverReal MeshScale,
			bool bEnabled,
			const FTriangleMesh* MultiResCoarseLODMesh = nullptr,
			const int32 MultiResCoarseLODParticleRangeId = INDEX_NONE,
			const TSharedPtr<Softs::FMultiResConstraints>& FineLODMultiResConstraint = TSharedPtr<Softs::FMultiResConstraints>(nullptr),
			const TArray<const FClothComplexColliders*>& ComplexColliders = TArray<const FClothComplexColliders*>(),
			const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection = TSharedPtr<const FManagedArrayCollection>(),
			const TMap<FName, FClothingAccessoryMeshData>* AccessoryMeshes = nullptr)
		{
			AddRules(ConfigProperties, TriangleMesh, PatternData, WeightMaps, VertexSets, FaceSets, FaceIntMaps, Tethers, MeshScale, bEnabled, ComplexColliders, ManagedArrayCollection, AccessoryMeshes);
		}

		void Update(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const TMap<FString, const TSet<int32>*>& FaceSets,
			const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
			Softs::FSolverReal MeshScale,
			Softs::FSolverReal MaxDistancesScale = (Softs::FSolverReal)1.,
			const FRotation3& LocalSpaceRotation = FRotation3::Identity,
			const FRotation3& ReferenceSpaceRotation = FRotation3::Identity,
			const FReal LocalSpaceScale = (Softs::FSolverReal)1.,
			const FSolverVec3 LocalSpaceLocation = FSolverVec3::ZeroVector);

		// NOTE: this only does something if using the PBDSolver. Force-based solver constraints are activated automatically
		// when activating a particle range.
		void Enable(bool bEnable);

		void ResetRestLengths(
			const TConstArrayView<Softs::FSolverVec3>& NewRestLengthPositions,
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);
		// ---- End of Cloth interface ----

		// ---- Debug functions ----
		const TSharedPtr<Softs::FPBDEdgeSpringConstraints>& GetEdgeSpringConstraints() const { return EdgeConstraints; }
		const TSharedPtr<Softs::FXPBDEdgeSpringConstraints>& GetXEdgeSpringConstraints() const { return XEdgeConstraints; }
		const TSharedPtr<Softs::FXPBDStretchBiasElementConstraints>& GetXStretchBiasConstraints() const { return XStretchBiasConstraints; }
		const TSharedPtr<Softs::FXPBDAnisotropicSpringConstraints>& GetXAnisoSpringConstraints() const { return XAnisoSpringConstraints; }
		const TSharedPtr<Softs::FPBDBendingSpringConstraints>& GetBendingSpringConstraints() const { return BendingConstraints; }
		const TSharedPtr<Softs::FXPBDBendingSpringConstraints>& GetXBendingSpringConstraints() const { return XBendingConstraints; }
		const TSharedPtr<Softs::FPBDBendingConstraints>& GetBendingElementConstraints() const { return BendingElementConstraints; }
		const TSharedPtr<Softs::FXPBDBendingConstraints>& GetXBendingElementConstraints() const { return XBendingElementConstraints; }
		const TSharedPtr<Softs::FXPBDAnisotropicBendingConstraints>& GetXAnisoBendingElementConstraints() const { return XAnisoBendingElementConstraints; }
		const TSharedPtr<Softs::FPBDExtremeDeformationConstraints>& GetExtremeDeformationConstraints() const { return ExtremeDeformationConstraints; }
		const TSharedPtr<Softs::FPBDAreaSpringConstraints>& GetAreaSpringConstraints() const { return AreaConstraints; }
		const TSharedPtr<Softs::FXPBDAreaSpringConstraints>& GetXAreaSpringConstraints() const { return XAreaConstraints; }
		const TSharedPtr<Softs::FPBDLongRangeConstraints>& GetLongRangeConstraints() const { return LongRangeConstraints; }
		const TSharedPtr<Softs::FPBDSphericalConstraint>& GetMaximumDistanceConstraints() const { return MaximumDistanceConstraints; }
		const TSharedPtr<Softs::FPBDSphericalBackstopConstraint>& GetBackstopConstraints() const { return BackstopConstraints; }
		const TSharedPtr<Softs::FPBDAnimDriveConstraint>& GetAnimDriveConstraints() const { return AnimDriveConstraints; }
		const TSharedPtr<Softs::FPBDCollisionSpringConstraints>& GetSelfCollisionConstraints() const { return SelfCollisionConstraints; }
		const TSharedPtr<Softs::FPBDTriangleMeshIntersections>& GetSelfIntersectionConstraints() const { return SelfIntersectionConstraints; }
		const TSharedPtr<Softs::FPBDTriangleMeshCollisions>& GetSelfCollisionInit() const { return SelfCollisionInit; }
		const TSharedPtr<Softs::FPBDSelfCollisionSphereConstraints>& GetSelfCollisionSphereConstraints() const
		{ return SelfCollisionSphereConstraints; }
		const TSharedPtr<Softs::FVelocityAndPressureField>& GetVelocityAndPressureField() const { return VelocityAndPressureField; }
		
		const TSharedPtr<Buoyancy::FBuoyancyField>& GetBuoyancyField() const { return VelocityPressureAndBuoyancyField; }

		const TSharedPtr<Softs::FExternalForces>& GetExternalForces() const { return ExternalForces; }
		const TSharedPtr<Softs::FPBDSoftBodyCollisionConstraint>& GetCollisionConstraint() const { return CollisionConstraint; }
		const TSharedPtr<Softs::FPBDSkinnedTriangleMeshCollisionConstraints>& GetSkinnedTriangleCollisionsConstraint() const { return SkinnedTriangleCollisionsConstraint; }
		UE_DEPRECATED(5.8, "MultiResConstraints have been deprecated")
		const TSharedPtr<Softs::FMultiResConstraints>& GetMultiResConstraints() const { static TSharedPtr<Softs::FMultiResConstraints> Dummy; return Dummy; }
		const TSharedPtr<Softs::FXPBDVertexConstraints>& GetClothVertexSpringConstraints() const { return ClothVertexSpringConstraints; };
		const TSharedPtr<Softs::FXPBDVertexFaceConstraints>& GetClothVertexFaceSpringConstraints() const { return ClothVertexFaceSpringConstraints; }
		const TSharedPtr<Softs::FXPBDFaceConstraints>& GetClothFaceSpringConstraints() const { return ClothFaceSpringConstraints; }
		const TSharedPtr<Softs::FPBDVertexFaceRepulsionConstraints>& GetRepulsionConstraints() const { return RepulsionConstraints; }
		// ---- End of debug functions ----

	private:
		void CreateSelfCollisionConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const TMap<FString, const TSet<int32>*>& FaceSets,
			const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
			const FTriangleMesh& TriangleMesh);
		void CreateStretchConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData,
			const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection,
			Softs::FSolverReal MeshScale);
		void CreateBendingConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData);
		void CreateExtremeDeformationConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData);
		void CreateAreaConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FTriangleMesh& TriangleMesh,
			const FClothingPatternData* PatternData);
		void CreateLongRangeConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& Tethers,
			Softs::FSolverReal MeshScale);
		void CreateMaxDistanceConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			Softs::FSolverReal MeshScale);
		void CreateBackstopConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			Softs::FSolverReal MeshScale,
			const TMap<FName, FClothingAccessoryMeshData>* AccessoryMeshes);
		void CreateAnimDriveConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);
		void CreateVelocityAndPressureField(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FTriangleMesh& TriangleMesh
		);
		void CreateVelocityPressureAndBuoyancyField(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FTriangleMesh& TriangleMesh
		);

		void CreateExternalForces(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps
		);
		void CreateCollisionConstraint(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			Softs::FSolverReal MeshScale,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const TArray<const FClothComplexColliders*>& ComplexColliders);
		void CreateClothClothConstraints(
			const Softs::FCollectionPropertyConstFacade& ConfigProperties,
			const TSharedPtr<const FManagedArrayCollection>& ManagedArrayCollection);

		void CreateForceBasedRules();
		void CreatePBDRules();

		TSharedPtr<Softs::FPBDEdgeSpringConstraints> EdgeConstraints;
		TSharedPtr<Softs::FXPBDEdgeSpringConstraints> XEdgeConstraints;
		TSharedPtr<Softs::FXPBDStretchBiasElementConstraints> XStretchBiasConstraints;
		TSharedPtr<Softs::FXPBDAnisotropicSpringConstraints> XAnisoSpringConstraints;
		TSharedPtr<Softs::FPBDBendingSpringConstraints> BendingConstraints;
		TSharedPtr<Softs::FXPBDBendingSpringConstraints> XBendingConstraints;
		TSharedPtr<Softs::FPBDBendingConstraints> BendingElementConstraints;
		TSharedPtr<Softs::FXPBDBendingConstraints> XBendingElementConstraints;
		TSharedPtr<Softs::FXPBDAnisotropicBendingConstraints> XAnisoBendingElementConstraints;
		TSharedPtr<Softs::FPBDAreaSpringConstraints> AreaConstraints;
		TSharedPtr<Softs::FXPBDAreaSpringConstraints> XAreaConstraints;
		TSharedPtr<Softs::FPBDLongRangeConstraints> LongRangeConstraints; 
		TSharedPtr<Softs::FPBDSphericalConstraint> MaximumDistanceConstraints;
		TSharedPtr<Softs::FPBDSphericalBackstopConstraint> BackstopConstraints;
		TSharedPtr<Softs::FPBDAnimDriveConstraint> AnimDriveConstraints;
		TSharedPtr<Softs::FPBDTriangleMeshCollisions> SelfCollisionInit;
		TSharedPtr<Softs::FPBDCollisionSpringConstraints> SelfCollisionConstraints;
		TSharedPtr<Softs::FPBDTriangleMeshIntersections> SelfIntersectionConstraints;
		TSharedPtr<Softs::FPBDSelfCollisionSphereConstraints> SelfCollisionSphereConstraints;
		TSharedPtr<Softs::FPBDExtremeDeformationConstraints> ExtremeDeformationConstraints;
		//~ Begin Force-based solver only constraints
		Softs::FSolverVec3 SolverWindVelocity; // Set from solver and added to wind from the config
		TSharedPtr<Softs::FVelocityAndPressureField> VelocityAndPressureField;
		TSharedPtr<Buoyancy::FBuoyancyField> VelocityPressureAndBuoyancyField;
		TSharedPtr<Softs::FExternalForces> ExternalForces;
		TSharedPtr<Softs::FPBDSoftBodyCollisionConstraint> CollisionConstraint;
		TSharedPtr<Softs::FPBDSkinnedTriangleMeshCollisionConstraints> SkinnedTriangleCollisionsConstraint;
		TSharedPtr<Softs::FXPBDVertexConstraints> ClothVertexSpringConstraints;
		TSharedPtr<Softs::FXPBDVertexFaceConstraints> ClothVertexFaceSpringConstraints;
		TSharedPtr<Softs::FXPBDFaceConstraints> ClothFaceSpringConstraints;
		TSharedPtr<Softs::FPBDVertexFaceRepulsionConstraints> RepulsionConstraints;
		//~ End Force-based solver only constraints

		// Exactly one of these should be non-null
		Softs::FEvolution* Evolution;
		Softs::FPBDEvolution* PBDEvolution;

		const TArray<Softs::FSolverVec3>* AnimationPositions;
		const TArray<Softs::FSolverVec3>* AnimationNormals;
		const TArray<Softs::FSolverVec3>* AnimationVelocities;

		int32 ParticleOffset;
		int32 ParticleRangeId;
		int32 NumParticles;

		int32 NumConstraintInits;
		int32 NumConstraintRules;
		int32 NumPostCollisionConstraintRules;
		int32 NumPostprocessingConstraintRules;

		bool bSkipSelfCollisionInit = false;

		//~ Begin Force-based solver only fields
		FPerSolverFieldSystem* PerSolverField;
		const TArray<Softs::FSolverVec3>* Normals;
		const TArray<Softs::FSolverRigidTransform3>* LastSubframeCollisionTransformsCCD;
		TArray<bool>* CollisionParticleCollided;
		TArray<Softs::FSolverVec3>* CollisionContacts;
		TArray<Softs::FSolverVec3>* CollisionNormals;
		TArray<Softs::FSolverReal>* CollisionPhis;

		int32 NumPreSubstepInits;
		int32 NumExternalForceRules;
		int32 NumPreSubstepConstraintRules;
		int32 NumCollisionConstraintRules;
		int32 NumUpdateLinearSystemRules;
		int32 NumUpdateLinearSystemCollisionsRules;
		//~ End Force-based solver only fields

		//~ Begin PBD solver only fields
		int32 ConstraintInitOffset;
		int32 ConstraintRuleOffset;
		int32 PostCollisionConstraintRuleOffset;
		int32 PostprocessingConstraintRuleOffset;
		//~ End PBD solver only fields

		class FRuleCreator;
	};
} // namespace Chaos
