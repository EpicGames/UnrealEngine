// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define	CHAOS_CLOTH_MODULE_NAME "ChaosCloth"

//////////////////////////////////////////////////////////////////////////
// IChaosClothModuleInterface

class IChaosClothModuleInterface : public IModuleInterface
{
};
