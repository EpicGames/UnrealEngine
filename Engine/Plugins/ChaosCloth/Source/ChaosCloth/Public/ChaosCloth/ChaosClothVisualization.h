// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Build.h"      // For CHAOS_DEBUG_DRAW
#include "Chaos/Declares.h"  //
#include "Misc/CoreMiscDefines.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"
#endif  // #if WITH_EDITOR

class FCanvas;
class FSceneView;
class FPrimitiveDrawInterface;
class UMaterial;

namespace Chaos
{
	class FClothingSimulationSolver;

	class FClothVisualizationNoGC
	{
	public:
		CHAOSCLOTH_API explicit FClothVisualizationNoGC(const ::Chaos::FClothingSimulationSolver* InSolver = nullptr);
		CHAOSCLOTH_API virtual ~FClothVisualizationNoGC();

		struct FDrawFilter
		{
			FString VertexSet;
			int32 SingleVertex = INDEX_NONE;
		};

		struct FDrawContext
		{
			explicit FDrawContext(FPrimitiveDrawInterface* InPDI = nullptr)
				: PDI(InPDI)
			{
			}
			FPrimitiveDrawInterface* PDI = nullptr;
			FDrawFilter Filter;
		};

		struct FDrawTextsContext
		{
			explicit FDrawTextsContext(FCanvas* InCanvas = nullptr, const FSceneView* InSceneView = nullptr)
				: Canvas(InCanvas), SceneView(InSceneView)
			{
			}
			FCanvas* Canvas = nullptr;
			const FSceneView* SceneView = nullptr;
			FDrawFilter Filter;
		};

#if CHAOS_DEBUG_DRAW
		// Editor & runtime functions
		CHAOSCLOTH_API void SetSolver(const ::Chaos::FClothingSimulationSolver* InSolver);

		CHAOSCLOTH_API void DrawParticleIndices(const FDrawTextsContext& Context = FDrawTextsContext()) const;
		CHAOSCLOTH_API void DrawElementIndices(const FDrawTextsContext& Context = FDrawTextsContext()) const;
		CHAOSCLOTH_API void DrawMaxDistanceValues(const FDrawTextsContext& Context = FDrawTextsContext()) const;
		CHAOSCLOTH_API void DrawLocalSpaceBoneNames(const FDrawTextsContext& Context = FDrawTextsContext()) const;

		CHAOSCLOTH_API void DrawPhysMeshWired(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawAnimMeshWired(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawAnimNormals(const FDrawContext& Context = FDrawContext(), const FReal Length = 20.) const;
		CHAOSCLOTH_API void DrawAnimVelocities(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawOpenEdges(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawPointNormals(const FDrawContext& Context = FDrawContext(), const FReal Length = 20.) const;
		CHAOSCLOTH_API void DrawPointVelocities(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawCollision(const FDrawContext& Context = FDrawContext(), const bool bWireframe = false) const;
		CHAOSCLOTH_API void DrawBackstops(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawBackstopDistances(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawMaxDistances(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawAnimDrive(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawEdgeConstraint(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawBendingConstraint(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawLongRangeConstraint(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawWindAndPressureForces(const FDrawContext& Context = FDrawContext(), const FReal LengthScale = 10.) const;
		CHAOSCLOTH_API void DrawWindVelocity(const FDrawContext& Context = FDrawContext(), const FReal LengthScale = 0.1) const;
		CHAOSCLOTH_API void DrawLocalSpace(const FDrawContext& Context = FDrawContext(), const FReal LengthScale = 1.) const;
		CHAOSCLOTH_API void DrawVelocityScale(const FDrawContext& Context = FDrawContext(), const FReal LengthScale = 0.1) const;
		CHAOSCLOTH_API void DrawSelfCollision(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawSelfIntersection(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawSelfCollisionThickness(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawClothCollisionThickness(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawInnerCollisionThickness(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawKinematicColliderWired(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawBounds(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawGravity(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawFictitiousAngularForces(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawClothClothConstraints(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawTeleportReset(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawExtremlyDeformedEdges(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawAccessoryMesh(const FDrawContext& Context = FDrawContext(), const FName& AccessoryMeshName = NAME_None) const;
		CHAOSCLOTH_API void DrawAccessoryMeshNormals(const FDrawContext& Context = FDrawContext(), const FName& AccessoryMeshName = NAME_None, const FReal Length = 20.) const;

		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawParticleIndices(FCanvas* Canvas, const FSceneView* SceneView = nullptr) const
		{
			return DrawParticleIndices(FDrawTextsContext(Canvas,SceneView));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawElementIndices(FCanvas* Canvas, const FSceneView* SceneView = nullptr) const
		{
			return DrawElementIndices(FDrawTextsContext(Canvas, SceneView));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawMaxDistanceValues(FCanvas* Canvas, const FSceneView* SceneView = nullptr) const
		{
			return DrawMaxDistanceValues(FDrawTextsContext(Canvas, SceneView));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLocalSpaceBoneNames(FCanvas* Canvas, const FSceneView* SceneView = nullptr) const
		{
			return DrawLocalSpaceBoneNames(FDrawTextsContext(Canvas, SceneView));
		}

		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPhysMeshWired(FPrimitiveDrawInterface* PDI) const
		{
			return DrawPhysMeshWired(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimMeshWired(FPrimitiveDrawInterface* PDI) const
		{
			return DrawAnimMeshWired(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimNormals(FPrimitiveDrawInterface* PDI, const FReal Length) const
		{
			return DrawAnimNormals(FDrawContext(PDI), Length);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimVelocities(FPrimitiveDrawInterface* PDI) const
		{
			return DrawAnimVelocities(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawOpenEdges(FPrimitiveDrawInterface* PDI) const
		{
			return DrawOpenEdges(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPointNormals(FPrimitiveDrawInterface* PDI, const FReal Length) const
		{
			return DrawPointNormals(FDrawContext(PDI), Length);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPointVelocities(FPrimitiveDrawInterface* PDI) const
		{
			return DrawPointVelocities(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawCollision(FPrimitiveDrawInterface* PDI, bool bWireframe) const
		{
			return DrawCollision(FDrawContext(PDI), bWireframe);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBackstops(FPrimitiveDrawInterface* PDI) const
		{
			return DrawBackstops(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBackstopDistances(FPrimitiveDrawInterface* PDI) const
		{
			return DrawBackstopDistances(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawMaxDistances(FPrimitiveDrawInterface* PDI) const
		{
			return DrawMaxDistances(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimDrive(FPrimitiveDrawInterface* PDI) const
		{
			return DrawAnimDrive(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawEdgeConstraint(FPrimitiveDrawInterface* PDI) const
		{
			return DrawEdgeConstraint(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBendingConstraint(FPrimitiveDrawInterface* PDI) const
		{
			return DrawBendingConstraint(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLongRangeConstraint(FPrimitiveDrawInterface* PDI) const
		{
			return DrawLongRangeConstraint(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindAndPressureForces(FPrimitiveDrawInterface* PDI, const FReal LengthScale) const
		{
			return DrawWindAndPressureForces(FDrawContext(PDI), LengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindVelocity(FPrimitiveDrawInterface* PDI, const FReal LengthScale) const
		{
			return DrawWindVelocity(FDrawContext(PDI), LengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLocalSpace(FPrimitiveDrawInterface* PDI, const FReal LengthScale) const
		{
			return DrawLocalSpace(FDrawContext(PDI), LengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawVelocityScale(FPrimitiveDrawInterface* PDI, const FReal LengthScale) const
		{
			return DrawVelocityScale(FDrawContext(PDI), LengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfCollision(FPrimitiveDrawInterface* PDI) const
		{
			return DrawSelfCollision(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfIntersection(FPrimitiveDrawInterface* PDI) const
		{
			return DrawSelfIntersection(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfCollisionThickness(FPrimitiveDrawInterface* PDI) const
		{
			return DrawSelfCollisionThickness(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawKinematicColliderWired(FPrimitiveDrawInterface* PDI) const
		{
			return DrawKinematicColliderWired(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBounds(FPrimitiveDrawInterface* PDI) const
		{
			return DrawBounds(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawGravity(FPrimitiveDrawInterface* PDI) const
		{
			return DrawGravity(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawFictitiousAngularForces(FPrimitiveDrawInterface* PDI) const
		{
			return DrawFictitiousAngularForces(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "MultiResConstraints have been deprecated.")
		void DrawMultiResConstraint(FPrimitiveDrawInterface* PDI = nullptr) const
		{
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawClothClothConstraints(FPrimitiveDrawInterface* PDI) const
		{
			return DrawClothClothConstraints(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawTeleportReset(FPrimitiveDrawInterface* PDI) const
		{
			return DrawTeleportReset(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawExtremlyDeformedEdges(FPrimitiveDrawInterface* PDI) const
		{
			return DrawExtremlyDeformedEdges(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAccessoryMesh(FPrimitiveDrawInterface* PDI, const FName& AccessoryMeshName = NAME_None) const
		{
			return DrawAccessoryMesh(FDrawContext(PDI), AccessoryMeshName);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAccessoryMeshNormals(FPrimitiveDrawInterface* PDI, const FName& AccessoryMeshName = NAME_None, const FReal Length = 20.) const
		{
			return DrawAccessoryMeshNormals(FDrawContext(PDI), AccessoryMeshName, Length);
		}
	
#else  // #if CHAOS_DEBUG_DRAW
		void SetSolver(const ::Chaos::FClothingSimulationSolver* /*InSolver*/) {}
		void DrawParticleIndices(const FDrawTextsContext& /*Context*/ = FDrawTextsContext()) const {}
		void DrawElementIndices(const FDrawTextsContext& /*Context*/ = FDrawTextsContext()) const {}
		void DrawMaxDistanceValues(const FDrawTextsContext& /*Context*/ = FDrawTextsContext()) const {}
		void DrawLocalSpaceBoneNames(const FDrawTextsContext& /*Context*/ = FDrawTextsContext()) const {}
		void DrawPhysMeshWired(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawAnimMeshWired(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawAnimNormals(const FDrawContext& /*Context*/ = FDrawContext(), const FReal /*Length*/ = 20.) const {}
		void DrawOpenEdges(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawPointNormals(const FDrawContext& /*Context*/ = FDrawContext(), const FReal /*Length*/ = 20.) const {}
		void DrawPointVelocities(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawCollision(const FDrawContext& /*Context*/ = FDrawContext(), bool /*bWireframe*/ = false) const {}
		void DrawBackstops(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawBackstopDistances(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawMaxDistances(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawAnimDrive(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawEdgeConstraint(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawBendingConstraint(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawLongRangeConstraint(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawWindAndPressureForces(const FDrawContext& /*Context*/ = FDrawContext(), const FReal /*ForceLengthScale*/ = 10.) const {}
		void DrawWindVelocity(const FDrawContext& /*Context*/ = FDrawContext(), const FReal /*LengthScale*/ = 0.1) const {}
		void DrawLocalSpace(const FDrawContext& /*Context*/ = FDrawContext(), FReal /*LocalSpaceLengthScale*/ = 1.) const {}
		void DrawVelocityScale(const FDrawContext& /*Context*/ = FDrawContext(), const FReal /*LengthScale*/ = 0.1) const {}
		void DrawSelfCollision(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawSelfIntersection(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawSelfCollisionThickness(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawClothCollisionThickness(const FDrawContext & /*Context*/ = FDrawContext()) const {}
		void DrawInnerCollisionThickness(const FDrawContext & /*Context*/ = FDrawContext()) const {}
		void DrawKinematicColliderWired(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawBounds(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawGravity(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawFictitiousAngularForces(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawClothClothConstraints(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawExtremlyDeformedEdges(const FDrawContext& /*Context*/ = FDrawContext()) const {}
		void DrawAccessoryMesh(const FDrawContext& /*Context*/ = FDrawContext(), const FName& /*AccessoryMeshName*/ = NAME_None) const {}
		void DrawAccessoryMeshNormals(const FDrawContext& /*Context*/ = FDrawContext(), const FName& /*AccessoryMeshName*/ = NAME_None, const FReal /*Length*/ = 20.) const;

		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawParticleIndices(FCanvas* /*Canvas*/, const FSceneView* /*SceneView*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawElementIndices(FCanvas* /*Canvas*/, const FSceneView* /*SceneView*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawMaxDistanceValues(FCanvas* /*Canvas*/, const FSceneView* /*SceneView*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLocalSpaceBoneNames(FCanvas* /*Canvas*/, const FSceneView* /*SceneView*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPhysMeshWired(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimMeshWired(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimNormals(FPrimitiveDrawInterface* /*PDI*/, const FReal /*Length*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawOpenEdges(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPointNormals(FPrimitiveDrawInterface* /*PDI*/, const FReal /*Length*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPointVelocities(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawCollision(FPrimitiveDrawInterface* /*PDI*/, bool /*bWireframe*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBackstops(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBackstopDistances(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawMaxDistances(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimDrive(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawEdgeConstraint(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBendingConstraint(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLongRangeConstraint(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindAndPressureForces(FPrimitiveDrawInterface* /*PDI*/, const FReal /*ForceLengthScale*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindVelocity(FPrimitiveDrawInterface* /*PDI*/, const FReal /*LengthScale*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLocalSpace(FPrimitiveDrawInterface* /*PDI*/, FReal /*LocalSpaceLengthScale*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawVelocityScale(FPrimitiveDrawInterface* /*PDI*/, const FReal /*LengthScale*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfCollision(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfIntersection(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfCollisionThickness(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawKinematicColliderWired(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawBounds(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawGravity(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawFictitiousAngularForces(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "MultiRes Constraints have been deprecated.")
		void DrawMultiResConstraint(FPrimitiveDrawInterface* /*PDI*/ = nullptr) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawClothClothConstraints(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawExtremlyDeformedEdges(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAccessoryMesh(FPrimitiveDrawInterface* /*PDI*/, const FName& /*AccessoryMeshName*/ = NAME_None) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAccessoryMeshNormals(FPrimitiveDrawInterface* /*PDI*/, const FName& /*AccessoryMeshName*/ = NAME_None, const FReal /*Length*/ = 20.) const;
#endif  // #if CHAOS_DEBUG_DRAW

		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawAnimNormals(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultLength = 20.;
			return DrawAnimNormals(FDrawContext(PDI), DefaultLength);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPointNormals(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultLength = 20.;
			return DrawPointNormals(FDrawContext(PDI), DefaultLength);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawCollision(FPrimitiveDrawInterface* PDI) const
		{
			constexpr bool bDefaultWireframe = false;
			return DrawCollision(FDrawContext(PDI), bDefaultWireframe);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindAndPressureForces(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultForceLengthScale = 10.;
			return DrawWindAndPressureForces(FDrawContext(PDI), DefaultForceLengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWindVelocity(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultLengthScale = .1;
			return DrawWindVelocity(FDrawContext(PDI), DefaultLengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawVelocityScale(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultLengthScale = .1;
			return DrawVelocityScale(FDrawContext(PDI), DefaultLengthScale);
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawLocalSpace(FPrimitiveDrawInterface* PDI) const
		{
			constexpr FReal DefaultLengthScale = 1.;
			return DrawLocalSpace(FDrawContext(PDI), DefaultLengthScale);
		}

#if WITH_EDITOR && CHAOS_DEBUG_DRAW
		// Editor only functions
		CHAOSCLOTH_API void DrawPhysMeshShaded(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawSelfCollisionLayers(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawInpaintWeightsMatched(const FDrawContext& Context = FDrawContext()) const;
		CHAOSCLOTH_API void DrawKinematicColliderShaded(const FDrawContext& Context = FDrawContext()) const;
		/** When Name is empty, the string stored by the cvar p.ChaosClothVisualization.WeightMapName is used instead */
		CHAOSCLOTH_API void DrawWeightMapWithName(const FDrawContext& Context = FDrawContext(), const FString& Name = "") const;
		/** Will draw current morph target (if any) if no name or an invalid name is specified */
		CHAOSCLOTH_API void DrawSimMorphTarget(const FDrawContext& Context = FDrawContext(), const FString& Name = "") const;
		CHAOSCLOTH_API TArray<FString> GetAllWeightMapNames() const;
		CHAOSCLOTH_API TArray<FString> GetAllMorphTargetNames() const;
		CHAOSCLOTH_API TArray<FString> GetAllVertexSetNames() const;
		CHAOSCLOTH_API TArray<FName> GetAllAccessoryMeshNames() const;

		// Editor only functions
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPhysMeshShaded(FPrimitiveDrawInterface* PDI) const
		{
			FDrawContext Context;
			return DrawPhysMeshShaded(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use DrawWeightMapWithName instead.")
		void DrawWeightMap(FPrimitiveDrawInterface* PDI) const
		{
			return DrawWeightMapWithName(FDrawContext(PDI), FString());
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSelfCollisionLayers(FPrimitiveDrawInterface* PDI) const
		{
			return DrawSelfCollisionLayers(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawInpaintWeightsMatched(FPrimitiveDrawInterface* PDI) const
		{
			return DrawInpaintWeightsMatched(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawKinematicColliderShaded(FPrimitiveDrawInterface* PDI) const
		{
			return DrawKinematicColliderShaded(FDrawContext(PDI));
		}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawWeightMapWithName(FPrimitiveDrawInterface* PDI, const FString& Name) const
		{
			return DrawWeightMapWithName(FDrawContext(PDI), Name);
		}
		/** Will draw current morph target (if any) if no name or an invalid name is specified */
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawSimMorphTarget(FPrimitiveDrawInterface* PDI, const FString& Name = "") const
		{
			return DrawSimMorphTarget(FDrawContext(PDI), Name);
		}
#else  // #if WITH_EDITOR && CHAOS_DEBUG_DRAW
		void DrawPhysMeshShaded(const FDrawContext& Context = FDrawContext()) const {}
		void DrawKinematicColliderShaded(const FDrawContext& Context = FDrawContext()) const {}

		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawPhysMeshShaded(FPrimitiveDrawInterface* /*PDI*/) const {}
		UE_DEPRECATED(5.8, "Use Context version of this method instead.")
		void DrawKinematicColliderShaded(FPrimitiveDrawInterface* /*PDI*/) const {}
#endif  // #if WITH_EDITOR && CHAOS_DEBUG_DRAW

#if CHAOS_DEBUG_DRAW
	private:
		// Simulation objects
		const ::Chaos::FClothingSimulationSolver* Solver;
#endif  // #if CHAOS_DEBUG_DRAW

		class FMaterials;
	};
} // End namespace Chaos
