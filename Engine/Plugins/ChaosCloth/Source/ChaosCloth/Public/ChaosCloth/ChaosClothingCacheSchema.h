// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"

struct FClothingSimulationCacheData;
class UChaosCache;
struct FPendingFrameWrite;
struct FPlaybackTickRecord;

namespace Chaos
{
class FClothingSimulationSolver;

struct FClothingCacheSchema
{
	// This should only be called from the ClothSolver as a PostSolve callback (when it is safe to read directly from ClothSolver data)
	static void CHAOSCLOTH_API RecordPostSolve(const FClothingSimulationSolver& ClothSolver, FPendingFrameWrite& OutFrame, FReal InTime);
	// This should only be called from the ClothSolver as a PreSolve callback (when it is safe to write directly to ClothSolver data)
	static void CHAOSCLOTH_API PlaybackPreSolve(UChaosCache& InCache, FReal InTime, FPlaybackTickRecord& TickRecord, FClothingSimulationSolver& ClothSolver);
	static void CHAOSCLOTH_API LoadCacheData(UChaosCache* InCache, FReal InTime, FClothingSimulationCacheData& CacheData);

	static bool CHAOSCLOTH_API CacheIsValidForPlayback(UChaosCache* InCache);

private:
	inline static const FName VelocityXName = TEXT("VelocityX");
	inline static const FName VelocityYName = TEXT("VelocityY");
	inline static const FName VelocityZName = TEXT("VelocityZ");
	inline static const FName PositionXName = TEXT("PositionX");
	inline static const FName PositionYName = TEXT("PositionY");
	inline static const FName PositionZName = TEXT("PositionZ");

	inline static const FName ReferenceTransformsName = TEXT("ReferenceTransform");
};

} // namespace Chaos