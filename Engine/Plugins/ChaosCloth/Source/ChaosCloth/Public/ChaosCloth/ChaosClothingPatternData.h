// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/TriangleMesh.h"
#include "Containers/ContainersFwd.h"
namespace Chaos
{

struct FClothingPatternData
{
	FTriangleMesh PatternTriangleMesh;

	TConstArrayView<FVector2f> PatternPositions;
	TConstArrayView<uint32> PatternToWeldedIndices;
	TArray<TVec3<FVec2f>> WeldedFaceVertexPatternPositions; // This will be empty if there are no PatternPositions

	CHAOSCLOTH_API FClothingPatternData(
		int32 InNumParticles,
		const TConstArrayView<uint32>& InIndices,
		const TConstArrayView<FVector2f>& InPatternPositions,
		const TConstArrayView<uint32>& InPatternIndices,
		const TConstArrayView<uint32>& InPatternToWeldedIndices);

private:
	void Reset();

	void GenerateDerivedPatternData(
		int32 InNumParticles,
		const TConstArrayView<uint32>& InIndices,
		const TConstArrayView<uint32>& InPatternIndices);
};
} // namespace Chaos
