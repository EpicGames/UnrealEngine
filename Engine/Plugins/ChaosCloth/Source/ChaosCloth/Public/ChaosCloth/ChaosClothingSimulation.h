// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothingSimulation.h"
#include "ClothingAsset.h"
#include "ClothCollisionData.h"
#include "ChaosCloth/ChaosClothConfig.h"
#include "ChaosCloth/ChaosClothVisualization.h"
#include "Templates/Atomic.h"
#include "Templates/UniquePtr.h"

class USkeletalMeshComponent;
class FClothingSimulationContextCommon;

namespace Chaos
{
	class FTriangleMesh;
	class FClothingSimulationSolver;
	class FClothingSimulationMesh;
	class FClothingSimulationCloth;
	class FClothingSimulationCollider;
	class FClothingSimulationConfig;
	class FSkeletalMeshCacheAdapter;

	typedef FClothingSimulationContextCommon FClothingSimulationContext;

	class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") FClothingSimulation  // TODO: 5.9 switch to IClothingSimulationInterface
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: public FClothingSimulationCommon
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	public:
		FClothingSimulation();
		virtual ~FClothingSimulation() override;

		friend FSkeletalMeshCacheAdapter;

	protected:
		//~ Begin IClothingSimulationInterface Interface
		virtual void Initialize() override;
		virtual void Shutdown() override;

		virtual IClothingSimulationContext* CreateContext() override;
		virtual void DestroyContext(IClothingSimulationContext* InContext) override { delete InContext; }

		virtual void CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* InAsset, int32 SimDataIndex) override;
		virtual void EndCreateActor() override;
		virtual void DestroyActors() override;

		virtual void FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly) override;
		virtual bool ShouldSimulateLOD(int32 OwnerLODIndex) const override;
		virtual void Simulate_AnyThread(const IClothingSimulationContext* InContext) override;
		virtual void ForceClothNextUpdateTeleportAndReset_AnyThread() override;
		virtual void HardResetSimulation(const IClothingSimulationContext* InContext) override
		{
			RefreshClothConfig(InContext);
		}
		virtual void AppendSimulationData(TMap<int32, FClothSimulData>& OutData, const USkeletalMeshComponent* InOwnerComponent, const USkinnedMeshComponent* InOverrideComponent) const override;

		// Return bounds in local space (or in world space if InOwnerComponent is null).
		virtual FBoxSphereBounds GetBounds(const USkeletalMeshComponent* InOwnerComponent) const override;

		virtual void AddExternalCollisions(const FClothCollisionData& InData) override;
		virtual void ClearExternalCollisions() override;
		//~ End IClothingSimulationInterface Interface

	public:
		void SetGravityOverride(const FVector& InGravityOverride);
		void DisableGravityOverride();

		// Function to be called if any of the assets' configuration parameters have changed
		void RefreshClothConfig(const IClothingSimulationContext* InContext);
		// Function to be called if any of the assets' physics assets changes (colliders)
		// This seems to only happen when UPhysicsAsset::RefreshPhysicsAssetChange is called with
		// bFullClothRefresh set to false during changes created using the viewport manipulators.
		void RefreshPhysicsAsset();

		//~ Begin IClothingSimulationInterface Interface
		virtual int32 GetNumCloths() const override { return NumCloths; }
		virtual int32 GetNumKinematicParticles() const override { return NumKinematicParticles; }
		virtual int32 GetNumDynamicParticles() const override { return NumDynamicParticles; }
		virtual int32 GetNumIterations() const override { return NumIterations; }
		virtual int32 GetNumSubsteps() const override { return NumSubsteps; }
		virtual float GetSimulationTime() const override { return SimulationTime; }
		virtual bool IsTeleported() const override { return bIsTeleported; }
		//~ End IClothingSimulationInterface Interface

		FClothingSimulationCloth* GetCloth(int32 ClothId);

		FClothingSimulationSolver* GetSolver() { return Solver.Get(); }

#if WITH_EDITOR
		// Editor only debug draw function
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawPhysMeshShaded(FPrimitiveDrawInterface* PDI) const { Visualization.DrawPhysMeshShaded(FClothVisualizationNoGC::FDrawContext(PDI)); }
#endif  // #if WITH_EDITOR

#if CHAOS_DEBUG_DRAW
		// Editor & runtime debug draw functions
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawParticleIndices(FCanvas* Canvas = nullptr, const FSceneView* SceneView = nullptr) const { Visualization.DrawParticleIndices(FClothVisualizationNoGC::FDrawTextsContext(Canvas, SceneView)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawElementIndices(FCanvas* Canvas = nullptr, const FSceneView* SceneView = nullptr) const { Visualization.DrawElementIndices(FClothVisualizationNoGC::FDrawTextsContext(Canvas, SceneView)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawMaxDistanceValues(FCanvas* Canvas = nullptr, const FSceneView* SceneView = nullptr) const { Visualization.DrawMaxDistanceValues(FClothVisualizationNoGC::FDrawTextsContext(Canvas, SceneView)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawLocalSpaceBoneNames(FCanvas* Canvas = nullptr, const FSceneView* SceneView = nullptr) const { Visualization.DrawLocalSpaceBoneNames(FClothVisualizationNoGC::FDrawTextsContext(Canvas, SceneView)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawPhysMeshWired(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawPhysMeshWired(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawAnimMeshWired(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawAnimMeshWired(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawAnimNormals(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawAnimNormals(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawAnimVelocities(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawAnimVelocities(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawPointNormals(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawPointNormals(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawPointVelocities(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawPointVelocities(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawCollision(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawCollision(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawBackstops(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawBackstops(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawBackstopDistances(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawBackstopDistances(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawMaxDistances(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawMaxDistances(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawAnimDrive(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawAnimDrive(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawEdgeConstraint(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawEdgeConstraint(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawBendingConstraint(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawBendingConstraint(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawLongRangeConstraint(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawLongRangeConstraint(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawWindAndPressureForces(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawWindAndPressureForces(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawWindVelocity(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawWindVelocity(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawLocalSpace(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawLocalSpace(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawSelfCollision(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawSelfCollision(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawSelfIntersection(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawSelfIntersection(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawBounds(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawBounds(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawGravity(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawGravity(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawTeleportReset(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawTeleportReset(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawExtremlyDeformedEdges(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawExtremlyDeformedEdges(FClothVisualizationNoGC::FDrawContext(PDI)); }
		UE_DEPRECATED(5.8, "Call debug draw methods directly from the GetClothVisualization() instead.")
		void DebugDrawVelocityScale(FPrimitiveDrawInterface* PDI = nullptr) const { Visualization.DrawVelocityScale(FClothVisualizationNoGC::FDrawContext(PDI)); }
#endif  // #if CHAOS_DEBUG_DRAW

		/** Return the visualization object for this simulation. */
		const FClothVisualizationNoGC* GetClothVisualization() const
		{
			return &Visualization;
		}

	private:
		void ResetStats();
		void UpdateStats(const FClothingSimulationCloth* Cloth);

		void UpdateSimulationFromSharedSimConfig();

		const FClothingSimulationContext* GetClothingSimulationContext(const USkeletalMeshComponent* SkeletalMeshComponent) const;

	private:
		// Visualization object
		FClothVisualizationNoGC Visualization;

		// Simulation objects
		TUniquePtr<FClothingSimulationSolver> Solver;  // Default solver
		TArray<TUniquePtr<FClothingSimulationMesh>> Meshes;
		TArray<TUniquePtr<FClothingSimulationCloth>> Cloths;
		TArray<TUniquePtr<FClothingSimulationCollider>> Colliders;
		TArray<TUniquePtr<FClothingSimulationConfig>> Configs;

		// External collision Data
		FClothCollisionData ExternalCollisionData;

		// Shared cloth config
		const UChaosClothSharedSimConfig* ClothSharedSimConfig;

		TBitArray<> LODHasAnyRenderClothMappingData; // Used by ShouldSimulateLOD

		// Properties that must be readable from all threads
		TAtomic<int32> NumCloths;
		TAtomic<int32> NumKinematicParticles;
		TAtomic<int32> NumDynamicParticles;
		TAtomic<int32> NumIterations;
		TAtomic<int32> NumSubsteps;
		TAtomic<float> SimulationTime;
		TAtomic<bool> bIsTeleported;

		// Overrides
		bool bUseLocalSpaceSimulation;
		bool bUseGravityOverride;
		FVector GravityOverride;
		FReal MaxDistancesMultipliers;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		int32 StepCount;
		int32 ResetCount;
#endif
		mutable bool bHasInvalidReferenceBoneTransforms;
	};
} // namespace Chaos

#if !defined(CHAOS_GET_SIM_DATA_ISPC_ENABLED_DEFAULT)
#define CHAOS_GET_SIM_DATA_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_GetSimData_ISPC_Enabled = INTEL_ISPC && CHAOS_GET_SIM_DATA_ISPC_ENABLED_DEFAULT;
#else
extern bool bChaos_GetSimData_ISPC_Enabled;
#endif
