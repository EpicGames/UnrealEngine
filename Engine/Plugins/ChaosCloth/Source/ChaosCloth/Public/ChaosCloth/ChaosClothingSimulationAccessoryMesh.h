// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Containers/ContainersFwd.h"
#include "EngineDefines.h"

#define UE_API CHAOSCLOTH_API

struct FClothVertBoneData;
struct FManagedArrayCollection;

namespace Chaos
{
	class FClothingSimulationSolver;
	class FClothingSimulationMesh;

	// Accessory mesh used as an input to a simulation. They are expected to share topology (e.g., Indices, PatternIndices) with their associated FClothingSimulationMesh
	class FClothingSimulationAccessoryMesh
	{
	public:
		UE_API FClothingSimulationAccessoryMesh(const FClothingSimulationMesh& InMesh, const FName& InAccessoryMeshName);

		UE_API virtual ~FClothingSimulationAccessoryMesh();

		FClothingSimulationAccessoryMesh(const FClothingSimulationAccessoryMesh&) = delete;
		FClothingSimulationAccessoryMesh(FClothingSimulationAccessoryMesh&&) = delete;
		FClothingSimulationAccessoryMesh& operator=(const FClothingSimulationAccessoryMesh&) = delete;
		FClothingSimulationAccessoryMesh& operator=(FClothingSimulationAccessoryMesh&&) = delete;

		const FName& GetAccessoryMeshName() const { return AccessoryMeshName; }

		/* Return the number of points. */
		virtual int32 GetNumPoints() const = 0;

		/* Return the number of pattern points (2d, unwelded). */
		virtual int32 GetNumPatternPoints() const = 0;

		/* Return the source mesh positions (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetPositions() const = 0;

		/* Return the source mesh 2d pattern positions. */
		virtual TConstArrayView<FVector2f> GetPatternPositions() const = 0;

		/* Return the source mesh normals (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetNormals() const = 0;

		/* Return the bone data containing bone weights and influences. */
		virtual TConstArrayView<FClothVertBoneData> GetBoneData() const = 0;

		/** Return the MorphTargetIndex for a given Morph Target Name, if it exists. NOTE: currently only the DefaultAccessoryMesh supports Morph Targets.*/
		virtual int32 FindMorphTargetByName(const FString& Name) const
		{
			return INDEX_NONE;
		}

		/** Get a list of all MorphTargets. (Index matches FindMorphTargetByName) */
		virtual TConstArrayView<FString> GetAllMorphTargetNames() const
		{
			return TConstArrayView<FString>();
		}

		/** Get all Morph Target position deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Positions via MorphTargetIndices */
		virtual TConstArrayView<FVector3f> GetMorphTargetPositionDeltas(int32 MorphTargetIndex) const
		{
			return TConstArrayView<FVector3f>();
		}

		/** Get all Morph Target tangent z (normal) deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Normals via MorphTargetIndices */
		virtual TConstArrayView<FVector3f> GetMorphTargetTangentZDeltas(int32 MorphTargetIndex) const
		{
			return TConstArrayView<FVector3f>();
		}

		/** Get all Morph Target indices for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). These indices can map MorphTargetDeltas back to Positions. */
		virtual TConstArrayView<int32> GetMorphTargetIndices(int32 MorphTargetIndex) const
		{
			return TConstArrayView<int32>();
		}

		/* Update the mesh for the next solver step, doing skinning and matching the shapes during LOD changes. */
		UE_API void Update(
			const FClothingSimulationSolver* Solver,
			const TArrayView<Softs::FSolverVec3>& OutPositions,
			const TArrayView<Softs::FSolverVec3>& OutNormals,
			int32 ActiveMorphTargetIndex = INDEX_NONE,
			float ActiveMorphTargetWeight = 0.f) const;

		UE_API void ApplyMorphTarget(
			int32 ActiveMorphTargetIndex,
			float ActiveMorphTargetWeight,
			const TArrayView<Softs::FSolverVec3>& InOutPositions,
			const TArrayView<Softs::FSolverVec3>& InOutNormals) const;
		// ---- End of the Cloth interface ----

	private:
		void SkinPhysicsMesh(
			int32 ActiveMorphTargetIndex,
			float ActiveMorphTargetWeight,
			const FReal LocalSpaceScale,
			const FVec3& LocalSpaceLocation,
			const TArrayView<Softs::FSolverVec3>& OutPositions,
			const TArrayView<Softs::FSolverVec3>& OutNormals) const;

	protected:
		const FClothingSimulationMesh& Mesh;
		const FName AccessoryMeshName;
	};
} // namespace Chaos

#if !defined(CHAOS_SKIN_PHYSICS_MESH_ISPC_ENABLED_DEFAULT)
#define CHAOS_SKIN_PHYSICS_MESH_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_SkinPhysicsMesh_ISPC_Enabled = INTEL_ISPC && CHAOS_SKIN_PHYSICS_MESH_ISPC_ENABLED_DEFAULT;
#else
extern bool bChaos_SkinPhysicsMesh_ISPC_Enabled;
#endif

#undef UE_API
