// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/Transform.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/AABB.h"
#include "Containers/ContainersFwd.h"
#include "ChaosCloth/ChaosClothConstraints.h"

struct FManagedArrayCollection;
struct FClothingSimulationCacheData;

namespace Chaos
{
	class FClothingSimulationSolver;
	class FClothingSimulationMesh;
	class FClothingSimulationCollider;
	class FClothingSimulationConfig;

	// Cloth simulation node
	class FClothingSimulationCloth final
	{
	public:
		enum EMassMode
		{
			UniformMass,
			TotalMass,
			Density
		};

		CHAOSCLOTH_API FClothingSimulationCloth(
			FClothingSimulationConfig* InConfig,
			FClothingSimulationMesh* InMesh,
			TArray<FClothingSimulationCollider*>&& InColliders,
			uint32 InGroupId);

		CHAOSCLOTH_API ~FClothingSimulationCloth();

		FClothingSimulationCloth(const FClothingSimulationCloth&) = delete;
		FClothingSimulationCloth(FClothingSimulationCloth&&) = delete;
		FClothingSimulationCloth& operator=(const FClothingSimulationCloth&) = delete;
		FClothingSimulationCloth& operator=(FClothingSimulationCloth&&) = delete;

		uint32 GetGroupId() const { return GroupId; }
		uint32 GetLODIndex(const FClothingSimulationSolver* Solver) const { return LODIndices.FindChecked(Solver); }

		int32 GetNumActiveKinematicParticles() const { return NumActiveKinematicParticles; }
		int32 GetNumActiveDynamicParticles() const { return NumActiveDynamicParticles; }

		// ---- Animatable property setters ----
		void SetMaxDistancesMultiplier(FRealSingle InMaxDistancesMultiplier) { MaxDistancesMultiplier = InMaxDistancesMultiplier; }

		void Reset() { bNeedsReset = true; }
		void Teleport() { bNeedsTeleport = true; }
		void ResetRestLengthsWithMorphTarget(const FString& MorphTargetName) 
		{
			bResetRestLengthsFromMorphTarget = true; 
			ResetRestLengthsMorphTargetName = MorphTargetName; 
		}
		// ---- End of the animatable property setters ----

		// ---- Node property getters/setters
		FClothingSimulationMesh* GetMesh() const { return Mesh; }
		CHAOSCLOTH_API void SetMesh(FClothingSimulationMesh* InMesh);

		FClothingSimulationConfig* GetConfig() const { return Config; }
		CHAOSCLOTH_API void SetConfig(FClothingSimulationConfig* InConfig);

		const TArray<FClothingSimulationCollider*>& GetColliders() const { return Colliders; }
		CHAOSCLOTH_API void SetColliders(TArray<FClothingSimulationCollider*>&& InColliders);
		CHAOSCLOTH_API void AddCollider(FClothingSimulationCollider* InCollider);
		CHAOSCLOTH_API void RemoveCollider(FClothingSimulationCollider* InCollider);
		CHAOSCLOTH_API void RemoveColliders();
		// ---- End of the Node property getters/setters

		// ---- Debugging/visualization functions
		// Return the solver's input positions for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationPositions(const FClothingSimulationSolver* Solver) const;
		// Return the solver's input positions for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetOldAnimationPositions(const FClothingSimulationSolver* Solver) const;
		// Return the solver's input normals for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationNormals(const FClothingSimulationSolver* Solver) const;
		// Return the solver's input velocities for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationVelocities(const FClothingSimulationSolver* Solver) const;
		CHAOSCLOTH_API bool AccessoryMeshExists(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const;
		// Return the solver's input positions for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAccessoryMeshPositions(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const;
		// Return the solver's input positions for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetOldAccessoryMeshPositions(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const;
		// Return the solver's input normals for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAccessoryMeshNormals(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const;
		// Return the solver's input velocities for this cloth source current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAccessoryMeshVelocities(const FClothingSimulationSolver* Solver, const FName& AccessoryMeshName) const;
		// Return the solver's positions for this cloth current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetParticlePositions(const FClothingSimulationSolver* Solver) const;
		// Return the solver's velocities for this cloth current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetParticleVelocities(const FClothingSimulationSolver* Solver) const;
		// Return the solver's normals for this cloth current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetParticleNormals(const FClothingSimulationSolver* Solver) const;
		// Return the solver's inverse masses for this cloth current LOD, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverReal> GetParticleInvMasses(const FClothingSimulationSolver* Solver) const;
		// Return the current gravity as applied by the solver using the various overrides, not thread safe, call must be done right after the solver update.
		// Does not have GravityScale applied when using Force-based solver (Get the per-particle value directly from cloth constraints' external forces).
		CHAOSCLOTH_API TVec3<FRealSingle> GetGravity(const FClothingSimulationSolver* Solver) const;
		// Return the current bounding box based on a given solver, not thread safe, call must be done right after the solver update.
		CHAOSCLOTH_API FAABB3 CalculateBoundingBox(const FClothingSimulationSolver* Solver) const;
		// Return the current LOD ParticleRangeId, or INDEX_NONE if no LOD is currently selected.
		CHAOSCLOTH_API int32 GetParticleRangeId(const FClothingSimulationSolver* Solver) const;
		UE_DEPRECATED(5.4, "Offset has been renamed ParticleRangeId to reflect that it is no longer an offset.")
		int32 GetOffset(const FClothingSimulationSolver* Solver) const { return GetParticleRangeId(Solver); }
		// Return the current LOD num particles, or 0 if no LOD is currently selected.
		CHAOSCLOTH_API int32 GetNumParticles(const FClothingSimulationSolver* Solver) const;
		// Return the current LOD mesh.
		CHAOSCLOTH_API const FTriangleMesh& GetTriangleMesh(const FClothingSimulationSolver* Solver) const;
		// Return the weight map of the specified name if available on the current LOD, or an empty array view otherwise.
		CHAOSCLOTH_API TConstArrayView<FRealSingle> GetWeightMapByName(const FClothingSimulationSolver* Solver, const FString& Name) const;
		// Return the weight map of the specified property name if it exists and is available on the current LOD, or an empty array view otherwise.
		CHAOSCLOTH_API TConstArrayView<FRealSingle> GetWeightMapByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const;
		template<typename StringType UE_REQUIRES(Chaos::Softs::TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName property version of this method instead")
		TConstArrayView<FRealSingle> GetWeightMapByProperty(const FClothingSimulationSolver* Solver, const StringType Property) const
		{
			return GetWeightMapByProperty(Solver, FName(Property));
		}
		// Return list of weight map names available across all LODs
		CHAOSCLOTH_API TSet<FString> GetAllWeightMapNames() const;
		// Return the face int map of the specified name if available on the current LOD, or an empty array view otherwise.
		CHAOSCLOTH_API TConstArrayView<int32> GetFaceIntMapByName(const FClothingSimulationSolver* Solver, const FString& Name) const;
		// Return the face int map of the specified property name if it exists and is available on the current LOD, or an empty array view otherwise.
		CHAOSCLOTH_API TConstArrayView<int32> GetFaceIntMapByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const;
		template<typename StringType UE_REQUIRES(Chaos::Softs::TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName property version of this method instead")
		TConstArrayView<int32> GetFaceIntMapByProperty(const FClothingSimulationSolver* Solver, const StringType Property) const
		{
			return GetFaceIntMapByProperty(Solver, FName(Property));
		}
		CHAOSCLOTH_API TSet<FString> GetAllVertexSetNames() const;
		// Return the vertex set of the specified name if available on the current LOD.
		CHAOSCLOTH_API const TSet<int32>* GetVertexSetByName(const FClothingSimulationSolver* Solver, const FString& Name) const;
		// Return the vertex set  of the specified property name if it exists and is available on the current LOD.
		CHAOSCLOTH_API const TSet<int32>* GetVertexSetByProperty(const FClothingSimulationSolver* Solver, const FName& Property) const;
		CHAOSCLOTH_API FString GetPropertyString(const FClothingSimulationSolver* Solver, const FName& Name) const;
		// Return the current LOD tethers.
		CHAOSCLOTH_API const TArray<TConstArrayView<TTuple<int32, int32, float>>>& GetTethers(const FClothingSimulationSolver* Solver) const;
		// Return the reference bone index for this cloth.
		CHAOSCLOTH_API int32 GetReferenceBoneIndex() const;
		// Return the local reference space transform for this cloth.
		const FRigidTransform3& GetReferenceSpaceTransform() const { return ReferenceSpaceTransform;  }
		CHAOSCLOTH_API int32 GetCurrentMorphTargetIndex(const FClothingSimulationSolver* Solver) const;
		CHAOSCLOTH_API FRealSingle GetCurrentMorphTargetWeight(const FClothingSimulationSolver* Solver) const;
		CHAOSCLOTH_API TSet<FString> GetAllMorphTargetNames() const;
		CHAOSCLOTH_API TSet<FName> GetAllAccessoryMeshNames() const;
#if CHAOS_DEBUG_DRAW
		FRealSingle GetTimeSinceLastTeleport() const { return TimeSinceLastTeleport; }
		FRealSingle GetTimeSinceLastReset() const { return TimeSinceLastReset; }
#endif
		// ---- End of the debugging/visualization functions

		// ---- Solver interface ----
		CHAOSCLOTH_API void Add(FClothingSimulationSolver* Solver);
		CHAOSCLOTH_API void Remove(FClothingSimulationSolver* Solver);

		CHAOSCLOTH_API void PreUpdate(FClothingSimulationSolver* Solver);
		CHAOSCLOTH_API void Update(FClothingSimulationSolver* Solver);
		CHAOSCLOTH_API void PostUpdate(FClothingSimulationSolver* Solver);

		CHAOSCLOTH_API void UpdateFromCache(const FClothingSimulationCacheData& CacheData);

		CHAOSCLOTH_API void ApplyPreSimulationTransforms(FClothingSimulationSolver* Solver, const Softs::FSolverVec3& DeltaLocalSpaceLocation, const Softs::FSolverRigidTransform3& GroupSpaceTransform, const Softs::FSolverReal DeltaTime);
		CHAOSCLOTH_API void PreSubstep(FClothingSimulationSolver* Solver, const Softs::FSolverReal InterpolationAlpha);
		// ---- End of the Solver interface ----


		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseGeodesicTethers, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MassMode, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MinPerParticleMass, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MassValue, float);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseGravityOverride, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(GravityOverride, FVector3f);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(GravityScale, float);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(VelocityScaleSpace, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(LinearVelocityScale, FVector3f);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AngularVelocityScale, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxVelocityScale, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxLinearVelocity, FVector3f);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxLinearAcceleration, FVector3f);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxAngularVelocity, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxAngularAcceleration, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(FictitiousAngularScale, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(FictitiousForcesModel, int32);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UsePointBasedWindModel, bool);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DampingCoefficient, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(LocalDampingCoefficient, float);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(CollisionThickness, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(FrictionCoefficient, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseCCD, bool);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(KinematicVertices3D, bool); // Selection set name string property; bool value is not actually used.

	private:
		CHAOSCLOTH_API int32 GetNumParticles(int32 InLODIndex) const;
		CHAOSCLOTH_API int32 GetParticleRangeId(const FClothingSimulationSolver* Solver, int32 InLODIndex) const;

	private:
		struct FLODData;

		// Cloth parameters
		FClothingSimulationMesh* Mesh = nullptr;
		FClothingSimulationConfig* Config = nullptr;
		TArray<FClothingSimulationCollider*> Colliders;
		uint32 GroupId = 0;

		TSharedPtr<FManagedArrayCollection> PropertyCollection;  // Used for backward compatibility only, otherwise the properties are owned by the Config

		FRealSingle MaxDistancesMultiplier = 1.f;  // Legacy multiplier

		bool bUseLODIndexOverride = false;
		int32 LODIndexOverride = 0;
		bool bNeedsReset = false;
		bool bNeedsTeleport = false;

		/** Reset restlengths from morph target. */
		bool bResetRestLengthsFromMorphTarget = false;
		FString ResetRestLengthsMorphTargetName;

#if CHAOS_DEBUG_DRAW
		FRealSingle TimeSinceLastTeleport = 0.f;
		FRealSingle TimeSinceLastReset = 0.f;
#endif

		// Reference space transform
		FRigidTransform3 ReferenceSpaceTransform;  // TODO: Add override in the style of LODIndexOverride
		FVec3 AppliedReferenceSpaceVelocity; // After scaling/clamping
		FVec3 AppliedReferenceSpaceAngularVelocity; // After scaling/clamping

		// LOD data
		TArray<TUniquePtr<FLODData>> LODData;
		TMap<FClothingSimulationSolver*, int32> LODIndices;

		// Stats
		int32 NumActiveKinematicParticles = 0;
		int32 NumActiveDynamicParticles = 0;
	};
} // namespace Chaos
