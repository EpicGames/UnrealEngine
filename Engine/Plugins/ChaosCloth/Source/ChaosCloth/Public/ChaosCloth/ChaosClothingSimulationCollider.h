// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothCollisionData.h"
#include "Containers/ContainersFwd.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Levelset.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Misc/EnumRange.h"

class USkeletalMeshComponent;
class UClothingAssetCommon;
class UPhysicsAsset;
class FClothingSimulationContextCommon;
struct FReferenceSkeleton;

namespace Chaos
{
	class FClothingSimulationSolver;
	class FClothingSimulationCloth;
	template<typename T> class TWeightedLatticeImplicitObject;

	struct FLevelSetCollisionData
	{
		const TSharedPtr<Chaos::FLevelSet, ESPMode::ThreadSafe> LevelSet;
		FTransform Transform;
		int32 BoneIndex;

		FLevelSetCollisionData(const TSharedPtr<Chaos::FLevelSet, ESPMode::ThreadSafe>& InLevelSet, const FTransform& InTransform, int32 InBoneIndex)
			: LevelSet(InLevelSet)
			, Transform(InTransform)
			, BoneIndex(InBoneIndex)
		{}
	};

	struct FSkinnedLevelSetCollisionData
	{
		const TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>> WeightedLevelSet;
		int32 BoneIndex;
		TArray<int32> MappedSkinnedBones;

		FSkinnedLevelSetCollisionData(const TRefCountPtr<Chaos::TWeightedLatticeImplicitObject<Chaos::FLevelSet>>& InWeightedLevelSet, int32 InBoneIndex, TArray<int32>&& InMappedSkinnedBones)
			: WeightedLevelSet(InWeightedLevelSet)
			, BoneIndex(InBoneIndex)
			, MappedSkinnedBones(MoveTemp(InMappedSkinnedBones))
		{}
	};

	struct FMLLevelSetCollisionData
	{
		const TSharedPtr<Chaos::FMLLevelSet> MLLevelSet;
		int32 BoneIndex;
		TArray<int32> MappedActiveBoneIndices;

		FMLLevelSetCollisionData(const TSharedPtr<Chaos::FMLLevelSet>& InMLLevelSet, int32 InBoneIndex, TArray<int32>&& InMappedActiveBoneIndices)
			: MLLevelSet(InMLLevelSet)
			, BoneIndex(InBoneIndex)
			, MappedActiveBoneIndices(MoveTemp(InMappedActiveBoneIndices))
		{}
	};

	struct FSkinnedTriangleMeshCollisionData
	{
		const TRefCountPtr<Chaos::FSkinnedTriangleMesh> SkinnedTriangleMesh;
		int32 BoneIndex;
		TArray<int32> MappedSkinnedBones;

		FSkinnedTriangleMeshCollisionData(const TRefCountPtr<Chaos::FSkinnedTriangleMesh>& InSkinnedTriangleMesh, int32 InBoneIndex, TArray<int32>&& InMappedSkinnedBones)
			: SkinnedTriangleMesh(InSkinnedTriangleMesh)
			, BoneIndex(InBoneIndex)
			, MappedSkinnedBones(MoveTemp(InMappedSkinnedBones))
		{}
	};

	struct FClothCollisionDataExtended
	{
		TArray<FLevelSetCollisionData> LevelSets;
		TArray<FSkinnedLevelSetCollisionData> SkinnedLevelSets;
		TArray<FMLLevelSetCollisionData> MLLevelSets;
		TArray<FSkinnedTriangleMeshCollisionData> SkinnedTriangleMeshes;

		void Reset()
		{
			LevelSets.Reset();
			SkinnedLevelSets.Reset();
			MLLevelSets.Reset();
			SkinnedTriangleMeshes.Reset();
		}
	};

	// Collider simulation node
	class FClothingSimulationCollider final
	{
	public:
		enum class ECollisionDataType : int32
		{
			LODless = 0,  // Global LODless collision slot filled with physics collisions
			External,  // External collision slot added/removed at every frame
			LODs,  // LODIndex based start slot for LODs collisions
			Count
		};

		CHAOSCLOTH_API FClothingSimulationCollider(const UPhysicsAsset* InPhysicsAsset, const FReferenceSkeleton* InReferenceSkeleton);
		CHAOSCLOTH_API ~FClothingSimulationCollider();

		FClothingSimulationCollider(const FClothingSimulationCollider&) = delete;
		FClothingSimulationCollider(FClothingSimulationCollider&&) = delete;
		FClothingSimulationCollider& operator=(const FClothingSimulationCollider&) = delete;
		FClothingSimulationCollider& operator=(FClothingSimulationCollider&&) = delete;

		CHAOSCLOTH_API int32 GetNumGeometries() const;

		// Return source (untransformed) collision data for LODless, external and active LODs.
		CHAOSCLOTH_API FClothCollisionData GetCollisionData(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth) const;

		// ---- Animatable property setters ----
		// Set external collision data, will only get updated when used as a Solver Collider TODO: Subclass collider?
		void SetCollisionData(const FClothCollisionData* InCollisionData) { CollisionData = InCollisionData; }
		// ---- End of the animatable property setters ----

		// ---- Cloth interface ----
		CHAOSCLOTH_API void Add(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);
		CHAOSCLOTH_API void Remove(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);

		CHAOSCLOTH_API void PreUpdate(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);
		CHAOSCLOTH_API void Update(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);
		CHAOSCLOTH_API void ResetStartPose(FClothingSimulationSolver* Solver, FClothingSimulationCloth* Cloth);
		// ---- End of the Cloth interface ----

		// ---- Debugging and visualization functions ----
		// Return current active LOD collision particles translations, not thread safe, to use after solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetCollisionTranslations(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;

		// Return current active LOD collision particles rotations, not thread safe, to use after solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRotation3> GetCollisionRotations(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;

		// Return current active LOD previous frame collision particles transforms, not thread safe, to use after solver update.
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRigidTransform3> GetOldCollisionTransforms(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;

		// Return current active LOD collision geometries, not thread safe, to use after solver update.
		CHAOSCLOTH_API TConstArrayView<FImplicitObjectPtr> GetCollisionGeometry(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;

		// Return whether the collision has been hit by a particle during CCD.
		CHAOSCLOTH_API TConstArrayView<bool> GetCollisionStatus(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;
		// ---- End of the debugging and visualization functions ----

		// Extract all collisions from the physics asset into a more cloth friendly format
		CHAOSCLOTH_API static void ExtractPhysicsAssetCollision(
			const UPhysicsAsset* PhysicsAsset,
			const FReferenceSkeleton* ReferenceSkeleton,
			FClothCollisionData& ClothCollisionData,
			FClothCollisionDataExtended& ClothCollisionDataExtended,
			TArray<int32>& UsedBoneIndices, // Bones that correspond with bodies
			TArray<int32>& UsedSubBoneIndices, // Bones used by extended type's internal bones, e.g., FSkinnedTriangleMeshCollisionData::MappedSkinnedBones
			bool bUseSphylOnly = false,
			bool bSkipMissingBones = false);

		// Return the collision particle range ID for the specified type if valid. If ECollisionDataType::LODs is asked, then the offset returned is for the current LOD.
		CHAOSCLOTH_API int32 GetCollisionRangeId(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType) const;

	private:	
		CHAOSCLOTH_API int32 GetNumGeometries(int32 InSlotIndex) const;

		// Return the collision particle range ID for the specified slot being LODLess, external, or any of the LODs collision.
		CHAOSCLOTH_API int32 GetCollisionRangeId(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, int32 InSlotIndex) const;

		// Return the collision particle  range ID and number of geometries for the specified type if valid. If ECollisionDataType::LODs is asked, then the offset returned is for the current LOD.
		CHAOSCLOTH_API bool GetCollisionRangeIdAndNumGeometries(const FClothingSimulationSolver* Solver, const FClothingSimulationCloth* Cloth, ECollisionDataType CollisionDataType, int32& OutCollisionRangeId, int32& OutNumGeometries) const;

	private:
		typedef TPair<const FClothingSimulationSolver*, const FClothingSimulationCloth*> FSolverClothPair;

		struct FLODData;

		const UPhysicsAsset* PhysicsAsset = nullptr;
		const FReferenceSkeleton* ReferenceSkeleton = nullptr;
		const FClothCollisionData* CollisionData = nullptr;  // External collision data

		bool bHasExternalCollisionChanged = false;

		// Collision primitives
		TArray<TUniquePtr<FLODData>> LODData;  // Actual LODs start at ECollisionDataType::LODs
		TMap<FSolverClothPair, int32> LODIndices;

		// Initial scale
		FReal Scale = 1.f;
	};
} // namespace Chaos

ENUM_RANGE_BY_COUNT(Chaos::FClothingSimulationCollider::ECollisionDataType, Chaos::FClothingSimulationCollider::ECollisionDataType::Count);
