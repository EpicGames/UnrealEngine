// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

struct FManagedArrayCollection;
class UChaosClothConfig;
class UChaosClothSharedSimConfig;

namespace Chaos
{
	namespace Softs
	{
		class FCollectionPropertyConstFacade;
		class FCollectionPropertyFacade;
		class FCollectionPropertyMutableFacade;
	}

	// Cloth simulation properties
	class FClothingSimulationConfig final
	{
	public:
		CHAOSCLOTH_API FClothingSimulationConfig();
		CHAOSCLOTH_API FClothingSimulationConfig(const TArray<TSharedPtr<const FManagedArrayCollection>>& InPropertyCollections);

		CHAOSCLOTH_API ~FClothingSimulationConfig();

		FClothingSimulationConfig(const FClothingSimulationConfig&) = delete;
		FClothingSimulationConfig(FClothingSimulationConfig&&) = delete;
		FClothingSimulationConfig& operator=(const FClothingSimulationConfig&) = delete;
		FClothingSimulationConfig& operator=(FClothingSimulationConfig&&) = delete;

		/**
		 * Initialize configuration from cloth config UObjects.
		 * @param ClothConfig The cloth config UObject.
		 * @param ClothSharedConfig The cloth solver shared config UObject.
		 * @param bUseLegacyConfig Whether to make the config a legacy cloth config, so that the constraints disable themselves with missing masks, ...etc.
		 */
		CHAOSCLOTH_API void Initialize(const UChaosClothConfig* ClothConfig, const UChaosClothSharedSimConfig* ClothSharedConfig, bool bUseLegacyConfig = false);

		/**
		 * Initialize config from an array of property collections (one per LOD).
		 * @param InPropertyCollections The cloth source property collections to initialize from.
		 * @param bForceDirty Whether to force the initialization even if the source property collections haven't changed.
		 * @param LODIndex The index of the LOD to replace, or INDEX_NONE to replace all LODs.
		 */
		CHAOSCLOTH_API void Initialize(const TArray<TSharedPtr<const FManagedArrayCollection>>& InPropertyCollections, bool bForceDirty = false, int32 LODIndex = INDEX_NONE);

		/** Return a property collection facade for reading properties from this configuration. */
		CHAOSCLOTH_API const Softs::FCollectionPropertyConstFacade& GetProperties(int32 LODIndex) const;
		const Softs::FCollectionPropertyConstFacade& GetProperties() const 
		{
			ensureMsgf(bIsLegacySingleLOD, TEXT("LODIndex-less version should only be used with legacy single LOD configs."));
			return GetProperties(0); 
		}

		/** Return a property collection facade for setting properties for this configuration. */
		CHAOSCLOTH_API Softs::FCollectionPropertyFacade& GetProperties(int32 LODIndex);
		Softs::FCollectionPropertyFacade& GetProperties() 
		{
			ensureMsgf(bIsLegacySingleLOD, TEXT("LODIndex-less version should only be used with legacy single LOD configs."));
			return GetProperties(0); 
		}

		/** Return this configuration's internal property collection. */
		TSharedPtr<const FManagedArrayCollection> GetPropertyCollection(int32 LODIndex) const 
		{
			return bIsLegacySingleLOD ? TSharedPtr<const FManagedArrayCollection>(PropertyCollections[0]) : 
				TSharedPtr<const FManagedArrayCollection>(PropertyCollections[LODIndex]);
		}
		TSharedPtr<const FManagedArrayCollection> GetPropertyCollection() const 
		{
			ensureMsgf(bIsLegacySingleLOD, TEXT("LODIndex-less version should only be used with legacy single LOD configs."));
			return GetPropertyCollection(0); 
		}

		bool IsLegacySingleLOD() const { return bIsLegacySingleLOD; }
		int32 GetNumLODs() const { return PropertyCollections.Num(); }
		bool IsValidLOD(int32 LODIndex) const
		{
			return bIsLegacySingleLOD ? true : Properties.IsValidIndex(LODIndex);
		}

		template<typename FunctionType>
		void ForAllProperties(FunctionType Func)
		{
			for (TUniquePtr<Softs::FCollectionPropertyMutableFacade>& Property : Properties)
			{
				Func((Softs::FCollectionPropertyFacade&)(*Property));
			}
		}

		template<typename FunctionType>
		void ForAllProperties(FunctionType Func) const
		{
			for (const TUniquePtr<Softs::FCollectionPropertyMutableFacade>& Property : Properties)
			{
				Func((Softs::FCollectionPropertyConstFacade&)(*Property));
			}
		}

		inline static const FName UseLegacyConfigName = TEXT("UseLegacyConfig");

	private:
		bool bIsLegacySingleLOD = false;
		TArray<TSharedPtr<FManagedArrayCollection>> PropertyCollections;
		TArray<TUniquePtr<Softs::FCollectionPropertyMutableFacade>> Properties;
	};
} // namespace Chaos
