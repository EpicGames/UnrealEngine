// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothingSimulationAccessoryMesh.h"

#define UE_API CHAOSCLOTH_API

namespace Chaos
{
	class FClothingSimulationMesh;

	// Wrapper around a FClothingSimulationMesh's LOD Data so it can share methods with accessory meshes.
	class FClothingSimulationDefaultAccessoryMesh final : public FClothingSimulationAccessoryMesh
	{
	public:
		UE_API FClothingSimulationDefaultAccessoryMesh(const FClothingSimulationMesh& InMesh, const int32 InLODIndex);

		UE_API virtual ~FClothingSimulationDefaultAccessoryMesh();

		const FClothingSimulationMesh& GetMesh() const
		{
			return Mesh;
		}

		int32 GetLODIndex() const
		{ 
			return LODIndex;
		}

		/* Return the number of points. */
		UE_API virtual int32 GetNumPoints() const override;

		/* Return the number of pattern points (2d, unwelded). */
		UE_API virtual int32 GetNumPatternPoints() const override;

		/* Return the source mesh positions (pre-skinning). */
		UE_API virtual TConstArrayView<FVector3f> GetPositions() const override;

		/* Return the source mesh 2d pattern positions. */
		UE_API virtual TConstArrayView<FVector2f> GetPatternPositions() const override;

		/* Return the source mesh normals (pre-skinning). */
		UE_API virtual TConstArrayView<FVector3f> GetNormals() const override;

		/* Return the bone data containing bone weights and influences. */
		UE_API virtual TConstArrayView<FClothVertBoneData> GetBoneData() const override;

		/** Return the MorphTargetIndex for a given Morph Target Name, if it exists. */
		UE_API virtual int32 FindMorphTargetByName(const FString& Name) const override;

		/** Get a list of all MorphTargets. (Index matches FindMorphTargetByName) */
		UE_API virtual TConstArrayView<FString> GetAllMorphTargetNames() const override;

		/** Get all Morph Target position deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Positions via MorphTargetIndices */
		UE_API virtual TConstArrayView<FVector3f> GetMorphTargetPositionDeltas(int32 MorphTargetIndex) const override;

		/** Get all Morph Target tangent z (normal) deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Normals via MorphTargetIndices */
		UE_API virtual TConstArrayView<FVector3f> GetMorphTargetTangentZDeltas(int32 MorphTargetIndex) const override;

		/** Get all Morph Target indices for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). These indices can map MorphTargetDeltas back to Positions. */
		UE_API virtual TConstArrayView<int32> GetMorphTargetIndices(int32 MorphTargetIndex) const override;

	private:
		static const FName DefaultAccessoryMeshName;
		const int32 LODIndex;
	};
} // namespace Chaos

#undef UE_API
