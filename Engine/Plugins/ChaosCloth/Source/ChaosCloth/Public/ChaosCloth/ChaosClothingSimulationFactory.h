// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationFactory.h"

#include "ChaosClothingSimulationFactory.generated.h"

UCLASS(MinimalAPI)
class UChaosClothingSimulationFactory final : public UClothingSimulationFactory
{
	GENERATED_BODY()
public:
	CHAOSCLOTH_API virtual IClothingSimulationInterface* CreateSimulation() const override;
	CHAOSCLOTH_API virtual void DestroySimulation(IClothingSimulationInterface* InSimulation) const override;
	CHAOSCLOTH_API virtual bool SupportsAsset(const UClothingAssetBase* InAsset) const override;

	CHAOSCLOTH_API virtual bool SupportsRuntimeInteraction() const override;
	CHAOSCLOTH_API virtual UClothingSimulationInteractor* CreateInteractor() override;

	CHAOSCLOTH_API virtual TArrayView<const TSubclassOf<UClothConfigBase>> GetClothConfigClasses() const override;
	CHAOSCLOTH_API const UEnum* GetWeightMapTargetEnum() const override;
};
