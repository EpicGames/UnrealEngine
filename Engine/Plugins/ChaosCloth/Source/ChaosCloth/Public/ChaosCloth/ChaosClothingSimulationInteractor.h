// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationInteractor.h"
#include "ChaosClothingSimulationInteractor.generated.h"

namespace Chaos
{
	class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") FClothingSimulation;  // TODO: 5.9 switch to IClothingSimulationInterface
	class FClothingSimulationCloth;
	class FClothingSimulationConfig;
}
class FClothingSimulationContextCommon;

namespace Chaos::Internal
{
	// Command signature for handling synced command buffer
	DECLARE_DELEGATE_TwoParams(FClothingInteractorConfigCommand, Chaos::FClothingSimulationConfig*, int32)
	DECLARE_DELEGATE_OneParam(FClothingInteractorCommand, Chaos::FClothingSimulationCloth*)
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	DECLARE_DELEGATE_TwoParams(FClothingSimulationInteractorCommand, Chaos::FClothingSimulation*, const FClothingSimulationContextCommon*)
PRAGMA_ENABLE_DEPRECATION_WARNINGS 
}

using FChaosClothingInteractorConfigCommand UE_DEPRECATED(5.7, "For internal use only. Use Chaos::Internal version instead.") = TDelegate<void(Chaos::FClothingSimulationConfig*, int32)>;
using FChaosClothingInteractorCommand UE_DEPRECATED(5.7, "For internal use only. Use Chaos::Internal version instead.") = TDelegate<void(Chaos::FClothingSimulationCloth*)>;
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
using FChaosClothingSimulationInteractorCommand UE_DEPRECATED(5.7, "For internal use only. Use Chaos::Internal version instead.") = TDelegate<void(Chaos::FClothingSimulation*, FClothingSimulationContextCommon*)>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UCLASS(BlueprintType, MinimalAPI)
class UChaosClothingInteractor : public UClothingInteractor
{
	GENERATED_BODY()

public:
	CHAOSCLOTH_API virtual void Sync(IClothingSimulationInterface* Simulation);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Edge Bending Area Stiffness"))
	CHAOSCLOTH_API void SetMaterialLinear(float EdgeStiffness = 1.f, float BendingStiffness = 1.f, float AreaStiffness = 1.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Edge Bending Area Stiffness"))
	CHAOSCLOTH_API void SetMaterial(FVector2D EdgeStiffness = FVector2D(1.f, 1.f), FVector2D BendingStiffness = FVector2D(1.f, 1.f), FVector2D AreaStiffness = FVector2D(1.f, 1.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Bending Element Buckling Stiffness Ratio"))
	CHAOSCLOTH_API void SetMaterialBuckling(FVector2D BucklingRatio = FVector2D(0.f, 0.f), FVector2D BucklingStiffness = FVector2D(1.f, 1.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Tether Stiffness Scale"))
	CHAOSCLOTH_API void SetLongRangeAttachmentLinear(float TetherStiffness = 1.f, float TetherScale = 1.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Tether Stiffness Scale"))
	CHAOSCLOTH_API void SetLongRangeAttachment(FVector2D TetherStiffness = FVector2D(1.f, 1.f), FVector2D TetherScale = FVector2D(1.f, 1.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Self Thickness Friction Coefficient"))
	CHAOSCLOTH_API void SetCollision(float CollisionThickness = 1.f, float FrictionCoefficient = 0.8f, bool bUseCCD = false, float SelfCollisionThickness = 2.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Backstop Enable Disable Use"))
	CHAOSCLOTH_API void SetBackstop(bool bEnabled = true);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Coefficient"))
	CHAOSCLOTH_API void SetDamping(float DampingCoefficient = 0.01f, float LocalDampingCoefficient = 0.f);

	// Deprecated. This function cannot set different Low and High values for the Drag and Lift weight maps. Use SetWind instead.
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Wind Drag Lift Coefficient"))
	CHAOSCLOTH_API void SetAerodynamics(float DragCoefficient = 0.07f, float LiftCoefficient = 0.035f, FVector WindVelocity = FVector(0.f, 0.f, 0.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Wind Drag Lift Coefficient Air Density Velocity"))
	CHAOSCLOTH_API void SetWind(FVector2D Drag = FVector2D(0.07f, 0.5f), FVector2D Lift = FVector2D(0.07f, 0.5f), float AirDensity = 1.225e-6f, FVector WindVelocity = FVector(0.f, 0.f, 0.f), FVector2D OuterDrag = FVector2D(0.07f, 0.5f), FVector2D OuterLift = FVector2D(0.07, 0.5f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Pressure"))
	CHAOSCLOTH_API void SetPressure(FVector2D Pressure = FVector2D(0.f, 1.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Scale Override"))
	CHAOSCLOTH_API void SetGravity(float GravityScale = 1.f, bool bIsGravityOverridden = false, FVector GravityOverride = FVector(0.f, 0.f, -981.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Stiffness"))
	CHAOSCLOTH_API void SetAnimDriveLinear(float AnimDriveStiffness = 0.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Stiffness Damping"))
	CHAOSCLOTH_API void SetAnimDrive(FVector2D AnimDriveStiffness = FVector2D(0.f, 1.f), FVector2D AnimDriveDamping = FVector2D(0.f, 1.f));

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Linear Angular Velocity Scale"))
	CHAOSCLOTH_API void SetVelocityScale(FVector LinearVelocityScale = FVector(0.75f, 0.75f, 0.75f), float AngularVelocityScale = 0.75f, float FictitiousAngularScale = 1.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos Max Linear Angular Velocity Acceleration Clamp"))
	CHAOSCLOTH_API void SetVelocityClamps(bool bEnableLinearVelocityClamping = false, FVector MaxLinearVelocity = FVector(1000., 1000., 1000.),
		bool bEnableLinearAccelerationClamping = false, FVector MaxLinearAcceleration = FVector(60000., 60000., 60000.),
		bool bEnableAngularVelocityClamping = false, float MaxAngularVelocity = 200.f,
		bool bEnableAngularAccelerationClamping = false, float MaxAngularAcceleration = 12000.f);

	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Chaos"))
	CHAOSCLOTH_API void ResetAndTeleport(bool bReset = false, bool bTeleport = false);

private:
	// Cloth command queue processed when we hit a sync
	TArray<Chaos::Internal::FClothingInteractorCommand> Commands;
	TArray<Chaos::Internal::FClothingInteractorConfigCommand> ConfigCommands;
};

UCLASS(BlueprintType, MinimalAPI)
class UChaosClothingSimulationInteractor : public UClothingSimulationInteractor
{
	GENERATED_BODY()
public:
	// UClothingSimulationInteractor interface
	CHAOSCLOTH_API virtual void Sync(IClothingSimulationInterface* InSimulation, const IClothingSimulationContext* InContext) override;

	CHAOSCLOTH_API virtual void PhysicsAssetUpdated() override;
	CHAOSCLOTH_API virtual void ClothConfigUpdated() override;
	CHAOSCLOTH_API virtual void SetAnimDriveSpringStiffness(float Stiffness) override;
	CHAOSCLOTH_API virtual void EnableGravityOverride(const FVector& Gravity) override;
	CHAOSCLOTH_API virtual void DisableGravityOverride() override;
	CHAOSCLOTH_API virtual void SetNumIterations(int32 NumIterations = 1) override;
	CHAOSCLOTH_API virtual void SetMaxNumIterations(int32 MaxNumIterations = 10) override;
	CHAOSCLOTH_API virtual void SetNumSubsteps(int32 NumSubsteps = 1) override;

protected:
	CHAOSCLOTH_API virtual UClothingInteractor* CreateClothingInteractor() override;

private:
	// Simulation command queue processed when we hit a sync
	TArray<Chaos::Internal::FClothingSimulationInteractorCommand> Commands;
	TArray<Chaos::Internal::FClothingInteractorConfigCommand> ConfigCommands;
};
