// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothingSimulationDefaultAccessoryMesh.h"
#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Containers/ContainersFwd.h"
#include "ClothVertBoneData.h"
#include "EngineDefines.h"

#define UE_API CHAOSCLOTH_API

struct FMeshToMeshVertData;
struct FClothVertBoneData;
struct FManagedArrayCollection;


namespace Chaos
{
	class FClothingSimulationSolver;
	class FClothingSimulationAccessoryMesh;

	// Mesh simulation node
	class FClothingSimulationMesh
	{
	public:
		UE_API explicit FClothingSimulationMesh(const FString& InDebugName);

		UE_API virtual ~FClothingSimulationMesh();

		FClothingSimulationMesh(const FClothingSimulationMesh&) = delete;
		FClothingSimulationMesh(FClothingSimulationMesh&&) = delete;
		FClothingSimulationMesh& operator=(const FClothingSimulationMesh&) = delete;
		FClothingSimulationMesh& operator=(FClothingSimulationMesh&&) = delete;

#if !UE_BUILD_SHIPPING
		const FString& GetDebugName() const { return DebugName; }
#else
		const FString& GetDebugName() const { return FText::GetEmpty().ToString(); }
#endif
		FName GetReferenceBoneName() const
		{
#if UE_ENABLE_DEBUG_DRAWING
			return ReferenceBoneName;
#else
			return NAME_None;
#endif
		}

		/* Return the number of LODs on this mesh. */
		virtual int32 GetNumLODs() const = 0;

		/* Return the cloth mesh LOD Index. The returned value can then be used to switch LODs on the simulation object. */
		virtual int32 GetLODIndex() const = 0;

		/* Return the cloth mesh LOD Index for a given owner LOD index. */
		virtual int32 GetLODIndexFromOwnerLODIndex(int32 OwnerLODIndex) const
		{
			// This default implementation assumes OwnerLODIndex == Cloth mesh LOD Index. 
			// Override this method if you have some other mapping.
			return IsValidLODIndex(OwnerLODIndex) ? OwnerLODIndex : INDEX_NONE;
		}

		/* Return the owner component LOD Index from the specified cloth mesh LOD index, or 0 if the owner LOD cannot be determined. The mapping between the cloth mesh LOD and the owner component LOD is not necessarily one to one. */
		virtual int32 GetOwnerLODIndex(int32 LODIndex) const = 0;

		/* Return whether the specified LOD index is valid. */
		virtual bool IsValidLODIndex(int32 LODIndex) const = 0;

		/* Return the number of points for the specified LOD, or 0 if the LOD is empty or invalid. */
		virtual int32 GetNumPoints(int32 LODIndex) const = 0;

		/* Return the number of pattern points (2d, unwelded) for the specified LOD, or 0 if patterns are not supported or the LOD is empty or invalid. */
		virtual int32 GetNumPatternPoints(int32 LODIndex) const = 0;

		/* Return the source mesh positions (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetPositions(int32 LODIndex) const = 0;

		/* Return the source mesh 2d pattern positions. */
		virtual TConstArrayView<FVector2f> GetPatternPositions(int32 LODIndex) const = 0;

		/* Return the source mesh normals (pre-skinning). */
		virtual TConstArrayView<FVector3f> GetNormals(int32 LODIndex) const = 0;

		/* Return the specified LOD's triangle indices for this mesh. */
		virtual TConstArrayView<uint32> GetIndices(int32 LODIndex) const = 0;

		/* Return the specified LOD's pattern (unwelded) triangle indices for this mesh, or empty array if patterns are not supported. */
		virtual TConstArrayView<uint32> GetPatternIndices(int32 LODIndex) const = 0;

		/* Return the specified LOD's map from pattern (unwelded) vertices to (welded) vertices, or empty array if patterns are not supported. */
		virtual TConstArrayView<uint32> GetPatternToWeldedIndices(int32 LODIndex) const = 0;

		/* Return all weight maps associated with this mesh returned in the same order as GetWeightMaps. */
		virtual TArray<FName> GetWeightMapNames(int32 LODIndex) const = 0;

		/* Return a map of all weight map names associated with this mesh to the index in the array returned by GetWeightMaps. */
		virtual TMap<FString, int32> GetWeightMapIndices(int32 LODIndex) const = 0;

		/* Return the specified LOD's weight map. */
		virtual TArray<TConstArrayView<FRealSingle>> GetWeightMaps(int32 LODIndex) const = 0;

		/* Return the specified LOD's vertex sets. */
		virtual TMap<FString, const TSet<int32>*> GetVertexSets(int32 LODIndex) const = 0;

		/* Return the specified LOD's face sets. */
		virtual TMap<FString, const TSet<int32>*> GetFaceSets(int32 LODIndex) const = 0;

		/* Return the specified LOD's face int maps. */
		virtual TMap<FString, TConstArrayView<int32>> GetFaceIntMaps(int32 LODIndex) const = 0;

		/* Return the tethers connections for the long range attachment into convenient parallel friendly batches. */
		virtual TArray<TConstArrayView<TTuple<int32, int32, float>>> GetTethers(int32 LODIndex, bool bUseGeodesicTethers) const = 0;

		/* Return the bone to treat as the root of the simulation space. */
		virtual int32 GetReferenceBoneIndex() const = 0;

		/* Return the transform of the bone treated as the root of the simulation space. */
		virtual FTransform GetReferenceBoneTransform() const = 0;

		/* Return the bone transforms as required when updating the collider pose. */
		virtual const TArray<FTransform>& GetBoneTransforms() const = 0;

		/* Return the transform of the bone treated as the root of the simulation space. */
		virtual const FTransform& GetComponentToWorldTransform() const = 0;

		/* Return the skinning matrices. */
		virtual const TArray<FMatrix44f>& GetRefToLocalMatrices() const = 0;

		/* Return the bone map used to remap the used bones index into the correct skinning matrix index. */
		virtual TConstArrayView<int32> GetBoneMap() const = 0;

		/* Return the bone data containing bone weights and influences. */
		virtual TConstArrayView<FClothVertBoneData> GetBoneData(int32 LODIndex) const = 0;

		/* Return the transition up data (PrevLODIndex < LODIndex), for matching shapes during LOD changes. */
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionUpSkinData(int32 LODIndex) const = 0;

		/* Return the transition down data (PrevLODIndex > LODIndex), for matching shapes during LOD changes. */
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionDownSkinData(int32 LODIndex) const = 0;

		/** Return this mesh uniform scale as the maximum of the three axis scale value. */
		UE_API virtual Softs::FSolverReal GetScale() const;

		/** Return the associated ManagedArrayCollection for this LOD, if it exists. */
		virtual TSharedPtr<const FManagedArrayCollection> GetManagedArrayCollection(int32 LODIndex) const
		{
			return TSharedPtr<const FManagedArrayCollection>();
		}

		/** Return the MorphTargetIndex for a given Morph Target Name, if it exists. */
		virtual int32 FindMorphTargetByName(int32 LODIndex, const FString& Name) const
		{
			return INDEX_NONE;
		}

		/** Get a list of all MorphTargets for a given LODIndex. (Index matches FindMorphTargetByName) */
		virtual TConstArrayView<FString> GetAllMorphTargetNames(int32 LODIndex) const
		{
			return TConstArrayView<FString>();
		}

		/** Get all Morph Target position deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Positions via MorphTargetIndices */
		virtual TConstArrayView<FVector3f> GetMorphTargetPositionDeltas(int32 LODIndex, int32 MorphTargetIndex) const
		{
			return TConstArrayView<FVector3f>();
		}

		/** Get all Morph Target tangent z (normal) deltas for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). Deltas index back to Normals via MorphTargetIndices */
		virtual TConstArrayView<FVector3f> GetMorphTargetTangentZDeltas(int32 LODIndex, int32 MorphTargetIndex) const
		{
			return TConstArrayView<FVector3f>();
		}

		/** Get all Morph Target indices for a given MorphTargetIndex (e.g., index returned by FindMorphTargetByName). These indices can map MorphTargetDeltas back to Positions. */
		virtual TConstArrayView<int32> GetMorphTargetIndices(int32 LODIndex, int32 MorphTargetIndex) const
		{
			return TConstArrayView<int32>();
		}

		virtual TConstArrayView<FName> GetAllAccessoryMeshNames(int32 LODIndex) const
		{
			return TConstArrayView<FName>();
		}

		/** Return the Accessory Mesh if it exists.*/
		virtual const FClothingSimulationAccessoryMesh* GetAccessoryMesh(int32 LODIndex, const FName& AccessoryMeshName) const
		{
			return nullptr;
		}

		FClothingSimulationDefaultAccessoryMesh GetDefaultAccessoryMesh(int32 LODIndex) const
		{
			return FClothingSimulationDefaultAccessoryMesh(*this, LODIndex);
		}

		/** Deform the specified positions to match the shape of the previous LOD. */
		UE_API bool WrapDeformLOD(
			int32 PrevLODIndex,
			int32 LODIndex,
			const Softs::FSolverVec3* Normals,
			const Softs::FSolverVec3* Positions,
			Softs::FSolverVec3* OutPositions) const;

		/** Deform the specified positions and transfer velocities to match the dynamics of the previous LOD. */
		UE_API bool WrapDeformLOD(
			int32 PrevLODIndex,
			int32 LODIndex,
			const Softs::FSolverVec3* Normals,
			const Softs::FPAndInvM* PositionAndInvMs,
			const Softs::FSolverVec3* Velocities,
			Softs::FPAndInvM* OutPositionAndInvMs0,
			Softs::FSolverVec3* OutPositions1,
			Softs::FSolverVec3* OutVelocities) const;

		/* Update the mesh for the next solver step, doing skinning and matching the shapes during LOD changes. */
		UE_API void Update(
			FClothingSimulationSolver* Solver,
			int32 PrevLODIndex,
			int32 LODIndex,
			int32 PrevOffset,
			int32 Offset,
			int32 ActiveMorphTargetIndex = INDEX_NONE,
			float ActiveMorphTargetWeight = 0.f);

		/** Apply a Morph Target to a given set of positions */
		UE_API void ApplyMorphTarget(
			int32 LODIndex,
			int32 ActiveMorphTargetIndex,
			float ActiveMorphTargetWeight,
			const TArrayView<Softs::FSolverVec3>& InOutPositions,
			const TArrayView<Softs::FSolverVec3>& InOutNormals) const;
		// ---- End of the Cloth interface ----

	private:

		bool WrapDeformLOD(
			int32 PrevLODIndex,
			int32 LODIndex,
			const TConstArrayView<Softs::FSolverVec3>& Positions,
			const TConstArrayView<Softs::FSolverVec3>& Normals,
			TArrayView<Softs::FSolverVec3>& OutPositions,
			TArrayView<Softs::FSolverVec3>& OutNormals) const;

#if !UE_BUILD_SHIPPING
		/** Debug name of the source component. */
		FString DebugName;
#endif

	protected:
#if UE_ENABLE_DEBUG_DRAWING
		FName ReferenceBoneName = NAME_None;
#endif
	};
} // namespace Chaos

#undef UE_API
