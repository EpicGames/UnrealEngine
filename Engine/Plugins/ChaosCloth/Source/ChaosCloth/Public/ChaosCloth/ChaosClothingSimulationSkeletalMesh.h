// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothingSimulationMesh.h"

class FClothingSimulationContextCommon;
class UClothingAssetCommon;

namespace Chaos
{
	class FClothingSimulationSkeletalMesh final : public FClothingSimulationMesh
	{
	public:
		FClothingSimulationSkeletalMesh(const UClothingAssetCommon* InAsset, const USkeletalMeshComponent* InSkeletalMeshComponent);
		virtual ~FClothingSimulationSkeletalMesh() override = default;

		FClothingSimulationSkeletalMesh(const FClothingSimulationSkeletalMesh&) = delete;
		FClothingSimulationSkeletalMesh(FClothingSimulationSkeletalMesh&&) = delete;
		FClothingSimulationSkeletalMesh& operator=(const FClothingSimulationSkeletalMesh&) = delete;
		FClothingSimulationSkeletalMesh& operator=(FClothingSimulationSkeletalMesh&&) = delete;

		const UClothingAssetCommon* GetAsset() const { return Asset; }
		const USkeletalMeshComponent* GetSkeletalMeshComponent() const { return SkeletalMeshComponent; }

		//~ Begin FClothingSimulationMesh Interface
		virtual int32 GetNumLODs() const override;
		virtual int32 GetLODIndex() const override;
		virtual int32 GetLODIndexFromOwnerLODIndex(int32 OwnerLODIndex) const override;
		virtual int32 GetOwnerLODIndex(int32 LODIndex) const override;
		virtual bool IsValidLODIndex(int32 LODIndex) const override;
		virtual int32 GetNumPoints(int32 LODIndex) const override;
		virtual int32 GetNumPatternPoints(int32 LODIndex) const override;
		virtual TConstArrayView<FVector3f> GetPositions(int32 LODIndex) const override;
		virtual TConstArrayView<FVector2f> GetPatternPositions(int32 LODIndex) const override;
		virtual TConstArrayView<FVector3f> GetNormals(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetIndices(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetPatternIndices(int32 LODIndex) const override;
		virtual TConstArrayView<uint32> GetPatternToWeldedIndices(int32 LODIndex) const override;
		virtual TArray<FName> GetWeightMapNames(int32 LODIndex) const override;
		virtual TMap<FString, int32> GetWeightMapIndices(int32 LODIndex) const override;
		virtual TArray<TConstArrayView<FRealSingle>> GetWeightMaps(int32 LODIndex) const override;
		virtual TMap<FString, const TSet<int32>*> GetVertexSets(int32 LODIndex) const override;
		virtual TMap<FString, const TSet<int32>*> GetFaceSets(int32 LODIndex) const override;
		virtual TMap<FString, TConstArrayView<int32>> GetFaceIntMaps(int32 LODIndex) const override;
		virtual TArray<TConstArrayView<TTuple<int32, int32, float>>> GetTethers(int32 LODIndex, bool bUseGeodesicTethers) const override;
		virtual int32 GetReferenceBoneIndex() const override;
		virtual FTransform GetReferenceBoneTransform() const override;
		virtual const TArray<FTransform>& GetBoneTransforms() const override;
		virtual const FTransform& GetComponentToWorldTransform() const override;
		virtual const TArray<FMatrix44f>& GetRefToLocalMatrices() const override;
		virtual TConstArrayView<int32> GetBoneMap() const override;
		virtual TConstArrayView<FClothVertBoneData> GetBoneData(int32 LODIndex) const override;
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionUpSkinData(int32 LODIndex) const override;
		virtual TConstArrayView<FMeshToMeshVertData> GetTransitionDownSkinData(int32 LODIndex) const override;
		//~ End FClothingSimulationMesh Interface

	private:
		const FClothingSimulationContextCommon* GetContext() const;

		const UClothingAssetCommon* Asset;
		const USkeletalMeshComponent* SkeletalMeshComponent;
	};
}
