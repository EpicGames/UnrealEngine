// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCloth/ChaosClothConstraints.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "Chaos/Transform.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/ImplicitObject.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "ChaosClothingSimulationSolver.generated.h"

class USkeletalMeshComponent;
class UClothingAssetCommon;
struct FClothingSimulationCacheData;

/**
 * Fictitious forces model.
 * Decide how fictitious forces (e.g. centrifugal force) are applied to the simulation.
 */
UENUM()
enum class EChaosClothingSimulationSolverFictitiousForcesModel : uint8
{
	// Don't apply fictitious forces.
	None,
	// Use partial scaled angular velocity, fictitious forces are calculated from the portion of the scaled character velocities. Legacy behavior, use the FullScale model instead.
	Legacy,
	// Use applied angular velocity (velocities after clamping), fictitious forces are calculated from the applied scale character velocities.
	AppliedScale,
	// Use full angular velocity, fictitious forces are calculated from the full scale character velocities.
	FullScale,
};

namespace Chaos
{
	class FTriangleMesh;

	class FClothingSimulationCloth;
	class FClothingSimulationCollider;
	class FClothingSimulationMesh;
	class FClothingSimulationConfig;
	class FClothComplexColliders;

	namespace Softs
	{
		class FCollectionPropertyConstFacade;
	}

	// Solver simulation node
	class FClothingSimulationSolver final : public FPhysicsSolverEvents
	{
	public:
		using EFictitiousForcesModel = EChaosClothingSimulationSolverFictitiousForcesModel;

		CHAOSCLOTH_API FClothingSimulationSolver(FClothingSimulationConfig* InConfig = nullptr, bool bUseLegacySolver = true);
		CHAOSCLOTH_API ~FClothingSimulationSolver();
		
		FClothingSimulationSolver(const FClothingSimulationSolver&) = delete;
		FClothingSimulationSolver(FClothingSimulationSolver&&) = delete;
		FClothingSimulationSolver& operator=(const FClothingSimulationSolver&) = delete;
		FClothingSimulationSolver& operator=(FClothingSimulationSolver&&) = delete;

		bool IsLegacySolver() const { return !!PBDEvolution; }

		// ---- Animatable property setters ----
		CHAOSCLOTH_API void SetLocalSpaceLocation(const FVec3& InLocalSpaceLocation, bool bReset = false);
		const FVec3& GetLocalSpaceLocation() const { return LocalSpaceLocation; }
		void SetLocalSpaceRotation(const FQuat& InLocalSpaceRotation) { LocalSpaceRotation = InLocalSpaceRotation; }
		const FRotation3& GetLocalSpaceRotation() const { return LocalSpaceRotation; }
		CHAOSCLOTH_API void SetLocalSpaceScale(FReal InLocalSpaceScale, bool bReset = false);
		FReal GetLocalSpaceScale() const { return LocalSpaceScale; }
		void SetVelocityScale(FReal InVelocityScale) { VelocityScale = InVelocityScale; }
		FReal GetVelocityScale() const { return VelocityScale; }

		// Disables all Cloths gravity override mechanism
		void EnableClothGravityOverride(bool bInIsClothGravityOverrideEnabled) { bIsClothGravityOverrideEnabled = bInIsClothGravityOverrideEnabled; }
		bool IsClothGravityOverrideEnabled() const { return bIsClothGravityOverrideEnabled; }
		void SetGravity(const TVec3<FRealSingle>& InGravity) { Gravity = InGravity; }
		const TVec3<FRealSingle>& GetGravity() const { return Gravity; }

		CHAOSCLOTH_API void SetWindVelocity(const TVec3<FRealSingle>& InWindVelocity, FRealSingle InLegacyWindAdaption = (FRealSingle)0.);
		const TVec3<FRealSingle>& GetWindVelocity() const { return WindVelocity; }

		CHAOSCLOTH_API int32 GetNumIterations() const;
		CHAOSCLOTH_API int32 GetMaxNumIterations() const;
		CHAOSCLOTH_API int32 GetNumSubsteps() const;

		void SetEnableSolver(bool InbEnableSolver) { bEnableSolver = InbEnableSolver; }
		bool GetEnableSolver() const { return bEnableSolver; }
		// ---- End of the animatable property setters ----

		// ---- Object management functions ----
		CHAOSCLOTH_API void SetCloths(TArray<FClothingSimulationCloth*>&& InCloths); 
		CHAOSCLOTH_API void AddCloth(FClothingSimulationCloth* InCloth);
		CHAOSCLOTH_API void RemoveCloth(FClothingSimulationCloth* InCloth);
		CHAOSCLOTH_API void RemoveCloths();

		CHAOSCLOTH_API void RefreshCloth(FClothingSimulationCloth* InCloth);
		CHAOSCLOTH_API void RefreshCloths();

		TConstArrayView<const FClothingSimulationCloth*> GetCloths() const { return Cloths; }

		/** Get the solver configuration. */
		FClothingSimulationConfig* GetConfig() const { return Config; }

		/**
		 * Set the solver configuration.
		 * Can use a cloth config if a single cloth is being simulated.
		 */
		CHAOSCLOTH_API void SetConfig(FClothingSimulationConfig* InConfig);

		CHAOSCLOTH_API void SetSolverLOD(int32 LODIndex);
		int32 GetSolverLOD() const { return SolverLOD; }

		/** Advance the simulation. */
		CHAOSCLOTH_API void Update(FSolverReal InDeltaTime);

		void ExtremeDeformationReset();

		/** Return the last delta time used for advancing the simulation. */
		FSolverReal GetDeltaTime() const { return DeltaTime; }

		/** Set the cached positions onto the particles */
		CHAOSCLOTH_API void UpdateFromCache(const FClothingSimulationCacheData& CacheData);

		// Return the actual of number of iterations used by the Evolution solver after the update (different from the number of iterations, depends on frame rate)
		CHAOSCLOTH_API int32 GetNumUsedIterations() const;
		int32 GetNumUsedSubsteps() const { return NumUsedSubsteps; }
		// Return the actual number of linear solver iterations used by the Evolution solver after the update (force-based solver only)
		CHAOSCLOTH_API int32 GetNumLinearSolverIterations(int32 ParticleRangeId) const;
		// Return the final error of the linear solver after the update (force-based solver only)
		CHAOSCLOTH_API FRealSingle GetLinearSolverError(int32 ParticleRangeId) const;

		CHAOSCLOTH_API FBoxSphereBounds CalculateBounds() const;
		// ---- End of the object management functions ----

		// ---- Cloth interface ----
		CHAOSCLOTH_API int32 AddParticles(int32 NumParticles, uint32 GroupId);
		CHAOSCLOTH_API void EnableParticles(int32 ParticleRangeId, bool bEnable);

		CHAOSCLOTH_API void ResetStartPose(int32 ParticleRangeId, int32 NumParticles);

		// Get the current solver time
		FSolverReal GetTime() const { return Time; }
		CHAOSCLOTH_API void SetParticleMassUniform(int32 ParticleRangeId, const FVector2f& UniformMass, const TConstArrayView<FRealSingle>& UniformMassMultipliers, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate);
		CHAOSCLOTH_API void SetParticleMassFromTotalMass(int32 ParticleRangeId, FRealSingle TotalMass, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate);
		CHAOSCLOTH_API void SetParticleMassFromDensity(int32 ParticleRangeId, const FVector2f& Density, const TConstArrayView<FRealSingle>& DensityMultipliers, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate);
		void SetParticleMassUniform(int32 ParticleRangeId, FRealSingle UniformMass, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate)
		{
			SetParticleMassUniform(ParticleRangeId, FVector2f(UniformMass), TConstArrayView<FRealSingle>(), MinPerParticleMass, Mesh, KinematicPredicate);
		}
		void SetParticleMassFromDensity(int32 ParticleRangeId, FRealSingle Density, FRealSingle MinPerParticleMass, const FTriangleMesh& Mesh, const TFunctionRef<bool(int32)>& KinematicPredicate)
		{
			SetParticleMassFromDensity(ParticleRangeId, FVector2f(Density), TConstArrayView<FRealSingle>(), MinPerParticleMass, Mesh, KinematicPredicate);
		}

		// Set the amount of velocity allowed to filter from the given change in reference space transform, including local simulation space.
		// NOTE: Force-based solver does not apply FictitiousAngularScale here. It's applied directly via the PropertyCollection.
		CHAOSCLOTH_API void SetReferenceVelocityScale(
			uint32 GroupId,
			const FRigidTransform3& OldReferenceSpaceTransform,
			const FRigidTransform3& ReferenceSpaceTransform,
			TVec3<FReal>& InOutAppliedReferenceVelocity, // Old reference velocity is passed in. New reference velocity is returned.
			TVec3<FReal>& InOutAppliedReferenceAngularVelocity, // Old reference velocity is passed in. New reference velocity is returned.
			const EChaosSoftsSimulationSpace VelocityScaleSpace, // the space the following linear velocity properties are in.
			const TVec3<FRealSingle>& LinearVelocityScale,
			const TVec3<FRealSingle>& MaxLinearVelocity,
			const TVec3<FRealSingle>& MaxLinearAcceleration,
			FRealSingle AngularVelocityScale,
			FRealSingle MaxAngularVelocity,
			FRealSingle MaxAngularAcceleration,
			FRealSingle FictitiousAngularScale,
			FRealSingle MaxVelocityScale,
			EFictitiousForcesModel FictitiousForcesModel);

		UE_DEPRECATED(5.8, "Use the version that takes a EFictitiousForcesModel argument instead.")
		void SetReferenceVelocityScale(
			uint32 GroupId,
			const FRigidTransform3& OldReferenceSpaceTransform,
			const FRigidTransform3& ReferenceSpaceTransform,
			TVec3<FReal>& InOutAppliedReferenceVelocity, // Old reference velocity is passed in. New reference velocity is returned.
			TVec3<FReal>& InOutAppliedReferenceAngularVelocity, // Old reference velocity is passed in. New reference velocity is returned.
			const EChaosSoftsSimulationSpace VelocityScaleSpace, // the space the following linear velocity properties are in.
			const TVec3<FRealSingle>& LinearVelocityScale,
			const TVec3<FRealSingle>& MaxLinearVelocity,
			const TVec3<FRealSingle>& MaxLinearAcceleration,
			FRealSingle AngularVelocityScale,
			FRealSingle MaxAngularVelocity,
			FRealSingle MaxAngularAcceleration,
			FRealSingle FictitiousAngularScale,
			FRealSingle MaxVelocityScale,
			bool bDisableFictitiousForces)
		{
			SetReferenceVelocityScale(
				GroupId,
				OldReferenceSpaceTransform,
				ReferenceSpaceTransform,
				InOutAppliedReferenceVelocity,
				InOutAppliedReferenceAngularVelocity,
				VelocityScaleSpace,
				LinearVelocityScale,
				MaxLinearVelocity,
				MaxLinearAcceleration,
				AngularVelocityScale,
				MaxAngularVelocity,
				MaxAngularAcceleration,
				FictitiousAngularScale,
				MaxVelocityScale,
				bDisableFictitiousForces ? EFictitiousForcesModel::None : EFictitiousForcesModel::Legacy);
		}

		UE_DEPRECATED(5.8, "Use the version that takes a EFictitiousForcesModel argument instead.")
		void SetReferenceVelocityScale(
			uint32 GroupId,
			const FRigidTransform3& OldReferenceSpaceTransform,
			const FRigidTransform3& ReferenceSpaceTransform,
			const TVec3<FRealSingle>& LinearVelocityScale,
			FRealSingle AngularVelocityScale,
			FRealSingle FictitiousAngularScale,
			FRealSingle MaxVelocityScale = 1.f,
			bool bDisableFictitiousForces = false)
		{
			// Acceleration clamps are disabled, so it doesn't matter what the old velocities were.
			TVec3<FReal> ReferenceVelocity(0.);
			TVec3<FReal> ReferenceAngularVelocity(0.);
			SetReferenceVelocityScale(
				GroupId,
				OldReferenceSpaceTransform,
				ReferenceSpaceTransform,
				ReferenceVelocity,
				ReferenceAngularVelocity,
				EChaosSoftsSimulationSpace::ReferenceBoneSpace,
				LinearVelocityScale,
				TVec3<FRealSingle>(TNumericLimits<FRealSingle>::Max()),
				TVec3<FRealSingle>(TNumericLimits<FRealSingle>::Max()),
				AngularVelocityScale,
				TNumericLimits<FRealSingle>::Max(),
				TNumericLimits<FRealSingle>::Max(),
				FictitiousAngularScale,
				MaxVelocityScale,
				bDisableFictitiousForces ? EFictitiousForcesModel::None : EFictitiousForcesModel::Legacy);
		}

		/** PBDSolver version */
		CHAOSCLOTH_API void SetProperties(
			uint32 GroupId,
			FRealSingle DampingCoefficient,
			FRealSingle MotionDampingCoefficient,
			FRealSingle CollisionThickness,
			FRealSingle FrictionCoefficient);

		/** Force based solver version */
		CHAOSCLOTH_API void SetProperties(int32 ParticleRangeId, const Softs::FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps) ;

		/** Begin PBD-solver only property methods (these properties are controlled directly via PropertyCollection in force-based solver) */
		// Set whether to use continuous collision detection.
		CHAOSCLOTH_API void SetUseCCD(uint32 GroupId, bool bUseCCD);

		// Set per group gravity, used to override solver's gravity. Must be called during cloth update.
		CHAOSCLOTH_API void SetGravity(uint32 GroupId, const TVec3<FRealSingle>& Gravity);

		// Set per group wind velocity, used to override solver's wind velocity. Must be called during cloth update.
		CHAOSCLOTH_API void SetWindVelocity(uint32 GroupId, const TVec3<FRealSingle>& InWindVelocity);

		// Set the geometry affected by the wind and pressure
		CHAOSCLOTH_API void SetWindAndPressureGeometry(
			uint32 GroupId,
			const FTriangleMesh& TriangleMesh,
			const Softs::FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

		CHAOSCLOTH_API void SetWindAndPressureGeometry(
			uint32 GroupId,
			const FTriangleMesh& TriangleMesh,
			const TConstArrayView<FRealSingle>& DragMultipliers,
			const TConstArrayView<FRealSingle>& OuterDragMultipliers,
			const TConstArrayView<FRealSingle>& LiftMultipliers,
			const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
			const TConstArrayView<FRealSingle>& PressureMultipliers);

		UE_DEPRECATED(5.5, "Use version with PropertyCollection (preferred) or OuterDrag and OuterLift")
		void SetWindAndPressureGeometry(
			uint32 GroupId,
			const FTriangleMesh& TriangleMesh,
			const TConstArrayView<FRealSingle>& DragMultipliers,
			const TConstArrayView<FRealSingle>& LiftMultipliers,
			const TConstArrayView<FRealSingle>& PressureMultipliers)
		{
			SetWindAndPressureGeometry(GroupId, TriangleMesh, DragMultipliers, DragMultipliers, LiftMultipliers, LiftMultipliers, PressureMultipliers);
		}

		// Set the wind and pressure properties.
		CHAOSCLOTH_API void SetWindAndPressureProperties(
			uint32 GroupId,
			const Softs::FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			bool bEnableAerodynamics);

		CHAOSCLOTH_API void SetWindAndPressureProperties(
			uint32 GroupId,
			const TVec2<FRealSingle>& Drag,
			const TVec2<FRealSingle>& OuterDrag,
			const TVec2<FRealSingle>& Lift,
			const TVec2<FRealSingle>& OuterLift,
			FRealSingle FluidDensity,
			const TVec2<FRealSingle>& Pressure);

		UE_DEPRECATED(5.5, "Use version with PropertyCollection (preferred) or OuterDrag and OuterLift")
		void SetWindAndPressureProperties(
			uint32 GroupId,
			const TVec2<FRealSingle>& Drag,
			const TVec2<FRealSingle>& Lift,
			FRealSingle FluidDensity = 1.225f,
			const TVec2<FRealSingle>& Pressure = TVec2<FRealSingle>::ZeroVector)
		{
			SetWindAndPressureProperties(GroupId, Drag, Drag, Lift, Lift, FluidDensity, Pressure);
		}

		// Return the wind velocity and pressure field associated with a given group id.
		CHAOSCLOTH_API const Softs::FVelocityAndPressureField& GetWindVelocityAndPressureField(uint32 GroupId) const;

		// Add external forces to the particles
		CHAOSCLOTH_API void AddExternalForces(uint32 GroupId, bool bUseLegacyWind);
		/** End PBD-only based property methods */

		const TArray<Softs::FSolverVec3>& GetOldAnimationPositions() const { return OldAnimationPositions; }
		const TArray<Softs::FSolverVec3>& GetAnimationPositions() const { return AnimationPositions; }
		const TArray<Softs::FSolverVec3>& GetInterpolatedAnimationPositions() const { return InterpolatedAnimationPositions; }
		const TArray<Softs::FSolverVec3>& GetOldAnimationNormals() const { return OldAnimationNormals; }
		const TArray<Softs::FSolverVec3>& GetAnimationNormals() const { return AnimationNormals; }
		const TArray<Softs::FSolverVec3>& GetInterpolatedAnimationNormals() const { return InterpolatedAnimationNormals; }
		const TArray<Softs::FSolverVec3>& GetNormals() const { return Normals; }
		const TArray<Softs::FSolverVec3>& GetAnimationVelocities() const { return AnimationVelocities; }

		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetOldAnimationPositionsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetOldAnimationPositionsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationPositionsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetAnimationPositionsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetInterpolatedAnimationPositionsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetInterpolatedAnimationPositionsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetOldAnimationNormalsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetOldAnimationNormalsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationNormalsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetAnimationNormalsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetInterpolatedAnimationNormalsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetInterpolatedAnimationNormalsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetNormalsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetNormalsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetAnimationVelocitiesView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetAnimationVelocitiesView(int32 ParticleRangeId);

		const Softs::FSolverVec3* GetOldAnimationPositions(int32 ParticleRangeId) const { return GetOldAnimationPositionsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetOldAnimationPositions(int32 ParticleRangeId) { return GetOldAnimationPositionsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetAnimationPositions(int32 ParticleRangeId) const { return GetAnimationPositionsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetAnimationPositions(int32 ParticleRangeId) { return GetAnimationPositionsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetInterpolatedAnimationPositions(int32 ParticleRangeId) const { return GetInterpolatedAnimationPositionsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetInterpolatedAnimationPositions(int32 ParticleRangeId) { return GetInterpolatedAnimationPositionsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetOldAnimationNormals(int32 ParticleRangeId) const { return GetOldAnimationNormalsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetOldAnimationNormals(int32 ParticleRangeId) { return GetOldAnimationNormalsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetAnimationNormals(int32 ParticleRangeId) const { return GetAnimationNormalsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetAnimationNormals(int32 ParticleRangeId) { return GetAnimationNormalsView(ParticleRangeId).GetData();
		}
		const Softs::FSolverVec3* GetInterpolatedAnimationNormals(int32 ParticleRangeId) const { return GetInterpolatedAnimationNormalsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetInterpolatedAnimationNormals(int32 ParticleRangeId) { return GetInterpolatedAnimationNormalsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetNormals(int32 ParticleRangeId) const { return GetNormalsView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetNormals(int32 ParticleRangeId) { return GetNormalsView(ParticleRangeId).GetData(); }
		const Softs::FSolverVec3* GetAnimationVelocities(int32 ParticleRangeId) const { return GetAnimationVelocitiesView(ParticleRangeId).GetData(); }
		Softs::FSolverVec3* GetAnimationVelocities(int32 ParticleRangeId) { return GetAnimationVelocitiesView(ParticleRangeId).GetData(); }

		CHAOSCLOTH_API const TArray<Softs::FPAndInvM>& GetParticlePandInvMs() const;
		CHAOSCLOTH_API const TArray<Softs::FSolverVec3>& GetParticleXs() const;
		CHAOSCLOTH_API const TArray<Softs::FSolverVec3>& GetParticleVs() const;
		CHAOSCLOTH_API const TArray<Softs::FSolverReal>& GetParticleInvMasses() const;

		CHAOSCLOTH_API TConstArrayView<Softs::FPAndInvM> GetParticlePandInvMsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FPAndInvM> GetParticlePandInvMsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetParticleXsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetParticleXsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetParticleVsView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetParticleVsView(int32 ParticleRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverReal> GetParticleInvMassesView(int32 ParticleRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverReal> GetParticleInvMassesView(int32 ParticleRangeId);

		const Softs::FPAndInvM* GetParticlePandInvMs(int32 ParticleRangeId) const 
		{
			return GetParticlePandInvMsView(ParticleRangeId).GetData();
		}
		Softs::FPAndInvM* GetParticlePandInvMs(int32 ParticleRangeId)
		{
			return GetParticlePandInvMsView(ParticleRangeId).GetData();
		}
		const Softs::FSolverVec3* GetParticleXs(int32 ParticleRangeId) const
		{
			return GetParticleXsView(ParticleRangeId).GetData();
		}
		Softs::FSolverVec3* GetParticleXs(int32 ParticleRangeId)
		{
			return GetParticleXsView(ParticleRangeId).GetData();
		}
		const Softs::FSolverVec3* GetParticleVs(int32 ParticleRangeId) const
		{
			return GetParticleVsView(ParticleRangeId).GetData();
		}
		Softs::FSolverVec3* GetParticleVs(int32 ParticleRangeId)
		{
			return GetParticleVsView(ParticleRangeId).GetData();
		}
		const Softs::FSolverReal* GetParticleInvMasses(int32 ParticleRangeId) const
		{
			return GetParticleInvMassesView(ParticleRangeId).GetData();
		}

		const FClothConstraints& GetClothConstraints(int32 ParticleRangeId) const { return *ClothsConstraints.FindChecked(ParticleRangeId); }
		FClothConstraints& GetClothConstraints(int32 ParticleRangeId) { return *ClothsConstraints.FindChecked(ParticleRangeId); }		

		CHAOSCLOTH_API bool UsesBuoyancy();
		CHAOSCLOTH_API void PreSimulateBuoyancyField_GameThread(float dt, const UWorld* World, const FBox BoundingBox);		

		CHAOSCLOTH_API uint32 GetNumParticles() const;
		CHAOSCLOTH_API int32 GetNumActiveParticles() const;
		CHAOSCLOTH_API int32 GetGlobalParticleOffset(int32 ParticleRangeId) const;

		const Softs::FSolverVec3& GetAppliedReferenceSpaceVelocity(uint32 GroupId) const { return AppliedReferenceSpaceVelocities[GroupId]; }
		const Softs::FSolverVec3& GetAppliedReferenceSpaceAngularVelocity(uint32 GroupId) const { return AppliedReferenceSpaceAngularVelocities[GroupId]; }
#if CHAOS_DEBUG_DRAW
		const Softs::FSolverVec3& GetOriginalReferenceSpaceVelocity(uint32 GroupId) const { return OriginalReferenceSpaceVelocities[GroupId]; }
		const Softs::FSolverVec3& GetOriginalReferenceSpaceAngularVelocity(uint32 GroupId) const { return OriginalReferenceSpaceAngularVelocities[GroupId]; }
#endif
		// ---- End of the Cloth interface ----

		// ---- Collider interface ----
		CHAOSCLOTH_API int32 AddCollisionParticles(int32 NumCollisionParticles, uint32 GroupId, int32 RecycledCollisionRangeId = 0);
		CHAOSCLOTH_API void EnableCollisionParticles(int32 CollisionRangeId, bool bEnable);
		// This method is not supported by LegacySolver and will return nullptr.
		const FClothComplexColliders* GetComplexColliders(int32 CollisionRangeId) const 
		{ 
			if (const TUniquePtr<FClothComplexColliders>* const Found = ComplexColliders.Find(CollisionRangeId))
			{
				return Found->Get();
			}
			return nullptr;
		}
		FClothComplexColliders* GetComplexColliders(int32 CollisionRangeId) 
		{
			if (TUniquePtr<FClothComplexColliders> const* Found = ComplexColliders.Find(CollisionRangeId))
			{
				return Found->Get();
			}
			return nullptr;
		}

		CHAOSCLOTH_API void ResetCollisionStartPose(int32 CollisionRangeId, int32 NumCollisionParticles);

		CHAOSCLOTH_API TConstArrayView<int32> GetCollisionBoneIndicesView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<int32> GetCollisionBoneIndicesView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRigidTransform3> GetCollisionBaseTransformsView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverRigidTransform3> GetCollisionBaseTransformsView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRigidTransform3> GetOldCollisionTransformsView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverRigidTransform3> GetOldCollisionTransformsView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRigidTransform3> GetCollisionTransformsView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverRigidTransform3> GetCollisionTransformsView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverVec3> GetCollisionParticleXsView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverVec3> GetCollisionParticleXsView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRotation3> GetCollisionParticleRsView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverRotation3> GetCollisionParticleRsView(int32 CollisionRangeId);
		CHAOSCLOTH_API TConstArrayView<FImplicitObjectPtr> GetCollisionGeometryView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TConstArrayView<bool> GetCollisionStatusView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TConstArrayView<Softs::FSolverRigidTransform3> GetLastSubframeCollisionTransformsCCDView(int32 CollisionRangeId) const;
		CHAOSCLOTH_API TArrayView<Softs::FSolverRigidTransform3> GetLastSubframeCollisionTransformsCCDView(int32 CollisionRangeId);

		CHAOSCLOTH_API void SetCollisionGeometry(int32 CollisionRangeId, int32 Index, FImplicitObjectPtr&& Geometry);

		CHAOSCLOTH_API const TArray<Softs::FSolverVec3>& GetCollisionContacts() const;
		CHAOSCLOTH_API const TArray<Softs::FSolverVec3>& GetCollisionNormals() const;
		CHAOSCLOTH_API const TArray<Softs::FSolverReal>& GetCollisionPhis() const;

		const int32* GetCollisionBoneIndices(int32 CollisionRangeId) const { return GetCollisionBoneIndicesView(CollisionRangeId).GetData(); }
		int32* GetCollisionBoneIndices(int32 CollisionRangeId) { return GetCollisionBoneIndicesView(CollisionRangeId).GetData(); }
		const Softs::FSolverRigidTransform3* GetCollisionBaseTransforms(int32 CollisionRangeId) const { return GetCollisionBaseTransformsView(CollisionRangeId).GetData(); }
		Softs::FSolverRigidTransform3* GetCollisionBaseTransforms(int32 CollisionRangeId) { return GetCollisionBaseTransformsView(CollisionRangeId).GetData(); }
		const Softs::FSolverRigidTransform3* GetOldCollisionTransforms(int32 CollisionRangeId) const { return GetOldCollisionTransformsView(CollisionRangeId).GetData(); }
		Softs::FSolverRigidTransform3* GetOldCollisionTransforms(int32 CollisionRangeId) { return GetOldCollisionTransformsView(CollisionRangeId).GetData(); }
		const Softs::FSolverRigidTransform3* GetCollisionTransforms(int32 CollisionRangeId) const { return GetCollisionTransformsView(CollisionRangeId).GetData(); }
		Softs::FSolverRigidTransform3* GetCollisionTransforms(int32 CollisionRangeId) { return GetCollisionTransformsView(CollisionRangeId).GetData(); }
		const Softs::FSolverVec3* GetCollisionParticleXs(int32 CollisionRangeId) const { return GetCollisionParticleXsView(CollisionRangeId).GetData(); }
		Softs::FSolverVec3* GetCollisionParticleXs(int32 CollisionRangeId) { return GetCollisionParticleXsView(CollisionRangeId).GetData(); }
		const Softs::FSolverRotation3* GetCollisionParticleRs(int32 CollisionRangeId) const { return GetCollisionParticleRsView(CollisionRangeId).GetData(); }
		Softs::FSolverRotation3* GetCollisionParticleRs(int32 CollisionRangeId) { return GetCollisionParticleRsView(CollisionRangeId).GetData(); }
		const FImplicitObjectPtr* GetCollisionGeometry(int32 CollisionRangeId) const { return GetCollisionGeometryView(CollisionRangeId).GetData(); }
		const bool* GetCollisionStatus(int32 CollisionRangeId) const { return GetCollisionStatusView(CollisionRangeId).GetData(); }
		const Softs::FSolverRigidTransform3* GetLastSubframeCollisionTransformsCCD(int32 CollisionRangeId) const 
		{
			return GetLastSubframeCollisionTransformsCCDView(CollisionRangeId).GetData();
		}
		Softs::FSolverRigidTransform3* GetLastSubframeCollisionTransformsCCD(int32 CollisionRangeId)
		{
			return GetLastSubframeCollisionTransformsCCDView(CollisionRangeId).GetData();
		}

		// ---- End of the Collider interface ----

		UE_DEPRECATED(5.4, "Use SetCollisionGeometry instead.")
		void SetCollisionGeometry(int32 Offset, int32 Index, TUniquePtr<Chaos::FImplicitObject>&& Geometry) {check(false);}

		UE_DEPRECATED(5.4, "Use GetCollisionGeometry instead.")
		const TUniquePtr<Chaos::FImplicitObject>* GetCollisionGeometries(int32 Offset) const {check(false); return nullptr;}

		// ---- Field interface ----
		FPerSolverFieldSystem& GetPerSolverField() { return PerSolverField; }
		const FPerSolverFieldSystem& GetPerSolverField() const { return PerSolverField; }
		// ---- End of the Field interface ----

		// Solver Properties
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(NumIterations, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxNumIterations, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(NumSubsteps, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DynamicSubstepDeltaTime, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(EnableNumSelfCollisionSubsteps, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(NumSelfCollisionSubsteps, int32);

	private:
		void Reset();

		/** Begin Force-only methods */
		void ParticleMassClampAndKinematicStateUpdate(Softs::FSolverParticlesRange& Particles, Softs::FSolverReal MinPerParticleMass, const TFunctionRef<bool(int32)>& KinematicPredicate);
		Softs::FSolverReal SetParticleMassPerArea(Softs::FSolverParticlesRange& Particles, const FTriangleMesh& Mesh);
		void ParticleMassUpdateDensity(Softs::FSolverParticlesRange& Particles, const FTriangleMesh& Mesh, const Softs::FSolverVec2& Density, const TConstArrayView<FRealSingle>& DensityMultipliers);
		/** End Force-only methods */

		/** Begin PBD-only methods */
		CHAOSCLOTH_API void ResetParticles();
		CHAOSCLOTH_API void ResetCollisionParticles(int32 InCollisionParticlesOffset = 0);
		CHAOSCLOTH_API void ParticleMassClampAndKinematicStateUpdate(int32 Offset, int32 Size, Softs::FSolverReal MinPerParticleMass, const TFunctionRef<bool(int32)>& KinematicPredicate);
		CHAOSCLOTH_API Softs::FSolverReal SetParticleMassPerArea(int32 Offset, int32 Size, const FTriangleMesh& Mesh);
		CHAOSCLOTH_API void ParticleMassUpdateDensity(const FTriangleMesh& Mesh, int32 Offset, int32 Size, const Softs::FSolverVec2& Density, const TConstArrayView<FRealSingle>& DensityMultipliers);
		/** End PBD - only methods */

		CHAOSCLOTH_API void ApplyPreSimulationTransforms();
		CHAOSCLOTH_API void PreSubstep(const Softs::FSolverReal InterpolationAlpha, bool bDetectSelfCollisions);

		// Update the solver field forces/velocities at the particles location
		CHAOSCLOTH_API void UpdateSolverField();

	private:

		// Exactly one of these should be non-null
		TUniquePtr<Softs::FEvolution> Evolution;
		TUniquePtr<Softs::FPBDEvolution> PBDEvolution;


		// Object arrays
		TArray<FClothingSimulationCloth*> Cloths;
		
		// Solver config
		int32 SolverLOD = 0; // Config may contain multiple LODs.
		FClothingSimulationConfig* Config = nullptr;
		TSharedPtr<FManagedArrayCollection> PropertyCollection;  // Used for backward compatibility only, otherwise the properties are owned by the Config

		// Simulation group attributes
		TArrayCollectionArray<Softs::FSolverRigidTransform3> PreSimulationTransforms;  // Allow a different frame of reference for each cloth groups
		TArrayCollectionArray<Softs::FSolverVec3> FictitiousAngularVelocities;  // Relative angular velocity of the reference bone. Depends on the fictitious angular scale factor when using PBDEvolution (is applied later for Evolution)
		TArrayCollectionArray<Softs::FSolverVec3> ReferenceSpaceLocations;  // Center of rotations for fictitious forces and local velocity damping in local coordinate to the simulation space location
		TArrayCollectionArray<Softs::FSolverVec3> AppliedReferenceSpaceVelocities; // Used for local velocity damping.
		TArrayCollectionArray<Softs::FSolverVec3> AppliedReferenceSpaceAngularVelocities; // Used for local velocity damping.
#if CHAOS_DEBUG_DRAW
		TArrayCollectionArray<Softs::FSolverVec3> OriginalReferenceSpaceVelocities; // Reference space velocity before any scaling or clamping. Used for debug draw.
		TArrayCollectionArray<Softs::FSolverVec3> OriginalReferenceSpaceAngularVelocities; // Reference space velocity before any scaling or clamping. Used for debug draw.
#endif


		// Particle attributes
		TArrayCollectionArray<Softs::FSolverVec3> Normals;
		TArrayCollectionArray<Softs::FSolverVec3> OldAnimationPositions;
		TArrayCollectionArray<Softs::FSolverVec3> AnimationPositions;
		TArrayCollectionArray<Softs::FSolverVec3> InterpolatedAnimationPositions;
		TArrayCollectionArray<Softs::FSolverVec3> OldAnimationNormals;
		TArrayCollectionArray<Softs::FSolverVec3> AnimationNormals;
		TArrayCollectionArray<Softs::FSolverVec3> InterpolatedAnimationNormals;
		TArrayCollectionArray<Softs::FSolverVec3> AnimationVelocities;

		// Collision particle attributes
		TArrayCollectionArray<int32> CollisionBoneIndices;
		TArrayCollectionArray<Softs::FSolverRigidTransform3> CollisionBaseTransforms;
		TArrayCollectionArray<Softs::FSolverRigidTransform3> OldCollisionTransforms;
		TArrayCollectionArray<Softs::FSolverRigidTransform3> CollisionTransforms;
		TArrayCollectionArray<Softs::FSolverRigidTransform3> LastSubframeCollisionTransformsCCD;
		TArrayCollectionArray<bool> Collided;

		// Extended collision data to support complex colliders (FEvolution only)
		TMap<int32, TUniquePtr<FClothComplexColliders>> ComplexColliders;


		// Cloth constraints
		TMap<int32, TUniquePtr<FClothConstraints>> ClothsConstraints;

		// Local space simulation
		FReal OldLocalSpaceScale = 1.; // Multiply this to simulation space coordinates to get world space coordinates.
		FReal LocalSpaceScale = 1.;
		FVec3 OldLocalSpaceLocation;  // This is used to translate between world space and simulation space,
		FVec3 LocalSpaceLocation;     // add this to simulation space coordinates to get world space coordinates, must keep FReal as underlying type for LWC
		FRotation3 LocalSpaceRotation; // NOTE: Currently the simulation does not apply LocalSpaceRotation.
		FReal VelocityScale;


		// Time stepping
		FSolverReal Time;
		FSolverReal DeltaTime;
		int32 NumUsedSubsteps;  // may change depending on dynamic substepping.

		// FEvolution-only
		TArray<FSolverVec3> CollisionContacts;
		TArray<FSolverVec3> CollisionNormals;
		TArray<FSolverReal> CollisionPhis;

		// Solver colliders offset (PBD Evolution-only)
		int32 CollisionParticlesOffset;  // Collision particle offset on the first solver/non cloth collider
		int32 CollisionParticlesSize;  // Number of solver only colliders

		// Solver parameters
		TVec3<FRealSingle> Gravity;
		TVec3<FRealSingle> WindVelocity;
		FRealSingle LegacyWindAdaption;
		bool bIsClothGravityOverrideEnabled;

		// Field system unique to the cloth solver
		FPerSolverFieldSystem PerSolverField;

		// Boolean to enable/disable solver in case caching is used
		bool bEnableSolver;
	};

} // namespace Chaos

#if !defined(CHAOS_CALCULATE_BOUNDS_ISPC_ENABLED_DEFAULT)
#define CHAOS_CALCULATE_BOUNDS_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(CHAOS_PRE_SIMULATION_TRANSFORMS_ISPC_ENABLED_DEFAULT)
#define CHAOS_PRE_SIMULATION_TRANSFORMS_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(CHAOS_PRE_SUBSTEP_INTERPOLATION_ISPC_ENABLED_DEFAULT)
#define CHAOS_PRE_SUBSTEP_INTERPOLATION_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_CalculateBounds_ISPC_Enabled = INTEL_ISPC && CHAOS_CALCULATE_BOUNDS_ISPC_ENABLED_DEFAULT;
static constexpr bool bChaos_PreSimulationTransforms_ISPC_Enabled = INTEL_ISPC && CHAOS_PRE_SIMULATION_TRANSFORMS_ISPC_ENABLED_DEFAULT;
static constexpr bool bChaos_PreSubstepInterpolation_ISPC_Enabled = INTEL_ISPC && CHAOS_PRE_SUBSTEP_INTERPOLATION_ISPC_ENABLED_DEFAULT;
#else
extern bool bChaos_PreSimulationTransforms_ISPC_Enabled;
extern bool bChaos_CalculateBounds_ISPC_Enabled;
extern bool bChaos_PreSubstepInterpolation_ISPC_Enabled;
#endif
