// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothEditor : ModuleRules
{
    public ChaosClothEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "ClothingSystemEditorInterface",
                "SlateCore",
                "Slate",
                "Persona",
                "ChaosCloth",
				"EditorFramework",
                "UnrealEd",
                "Engine",
                "DetailCustomizations",
                "CoreUObject",
                "InputCore",
            }
        );

        SetupModulePhysicsSupport(Target);
    }
}
