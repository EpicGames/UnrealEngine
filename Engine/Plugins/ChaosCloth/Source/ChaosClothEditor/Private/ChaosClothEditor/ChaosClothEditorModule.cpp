// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothEditor/ChaosClothEditorModule.h"
#include "ChaosClothEditor/ChaosClothEditorPrivate.h"

#include "ClothingSystemEditorInterfaceModule.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "ChaosClothWeightedValueCustomization.h"

IMPLEMENT_MODULE(FChaosClothEditorModule, ChaosClothEditor);
DEFINE_LOG_CATEGORY(LogChaosClothEditor);

void FChaosClothEditorModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(FClothingSystemEditorInterfaceModule::ExtenderFeatureName, &ChaosEditorExtender);

	// Register type customizations
	FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule)
	{
		// Register weight map property type.
		PropertyModule->RegisterCustomPropertyTypeLayout("ChaosClothWeightedValue", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosClothWeightedValueCustomization::MakeInstance));
	}
}

void FChaosClothEditorModule::ShutdownModule()
{
	IModularFeatures::Get().RegisterModularFeature(FClothingSystemEditorInterfaceModule::ExtenderFeatureName, &ChaosEditorExtender);

	// Unregister type customizations
	FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule)
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout("ChaosClothWeightedValue");
	}
}
