// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothWeightedValueCustomization.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ChaosClothEditorPrivate.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "FChaosClothWeightedValueCustomization"

TSharedRef<IPropertyTypeCustomization> FChaosClothWeightedValueCustomization::MakeInstance() 
{
	return MakeShareable(new FChaosClothWeightedValueCustomization);
}

FChaosClothWeightedValueCustomization::FChaosClothWeightedValueCustomization()
{
}

FChaosClothWeightedValueCustomization::~FChaosClothWeightedValueCustomization()
{
}

TSharedRef<SWidget> FChaosClothWeightedValueCustomization::MakeChildWidget(
	TSharedRef<IPropertyHandle>& StructurePropertyHandle,
	TSharedRef<IPropertyHandle>& PropertyHandle)
{
	const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();
	
	if (PropertyClass == FFloatProperty::StaticClass())
	{
		return MakeNumericWidget<float>(StructurePropertyHandle, PropertyHandle);
	}

	UE_LOGF(LogChaosClothEditor, Fatal, "Weighted values part of this customization must be floating point values in the unit interval.");
	return SNullWidget::NullWidget;
}

template<typename NumericType>
TSharedRef<SWidget> FChaosClothWeightedValueCustomization::MakeNumericWidget(
	TSharedRef<IPropertyHandle>& StructurePropertyHandle,
	TSharedRef<IPropertyHandle>& PropertyHandle)
{
	TOptional<NumericType> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
	NumericType SliderExponent, Delta;
	float ShiftMultiplier = 10.f;
	float CtrlMultiplier = 0.1f;
	bool SupportDynamicSliderMaxValue = false;
	bool SupportDynamicSliderMinValue = false;

	ExtractNumericMetadata(StructurePropertyHandle, MinValue, MaxValue, SliderMinValue, SliderMaxValue, SliderExponent, Delta, ShiftMultiplier, CtrlMultiplier, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

	TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;

	return SNew(SNumericEntryBox<NumericType>)
		.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, WeakHandlePtr)
		.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.Value(this, &FChaosClothWeightedValueCustomization::OnGetValue, WeakHandlePtr)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.OnValueCommitted(this, &FChaosClothWeightedValueCustomization::OnValueCommitted<NumericType>, WeakHandlePtr)
		.OnValueChanged(this, &FChaosClothWeightedValueCustomization::OnValueChanged<NumericType>, WeakHandlePtr)
		.OnBeginSliderMovement(this, &FChaosClothWeightedValueCustomization::OnBeginSliderMovement)
		.OnEndSliderMovement(this, &FChaosClothWeightedValueCustomization::OnEndSliderMovement<NumericType>)
		.LabelVAlign(VAlign_Center)
		// Only allow spin on handles with one object.  Otherwise it is not clear what value to spin
		.AllowSpin(PropertyHandle->GetNumOuterObjects() < 2)
		.ShiftMultiplier(ShiftMultiplier)
		.CtrlMultiplier(CtrlMultiplier)
		.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
		.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
		.OnDynamicSliderMaxValueChanged(this, &FChaosClothWeightedValueCustomization::OnDynamicSliderMaxValueChanged<NumericType>)
		.OnDynamicSliderMinValueChanged(this, &FChaosClothWeightedValueCustomization::OnDynamicSliderMinValueChanged<NumericType>)
		.MinValue(MinValue)
		.MaxValue(MaxValue)
		.MinSliderValue(SliderMinValue)
		.MaxSliderValue(SliderMaxValue)
		.SliderExponent(SliderExponent)
		.Delta(Delta)
		.Label()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::FromString(PropertyHandle->GetMetaData(TEXT("ChaosClothShortName"))))  // Case specific metadata, uses ChaosCloth prefix to avoid conflict with future engine extensions
		];
}

// The following code is just a plain copy of FMathStructCustomization which
// would need changes to be able to serve as a base class for this customization.
template<typename NumericType>
void FChaosClothWeightedValueCustomization::ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, TOptional<NumericType>& MinValue, TOptional<NumericType>& MaxValue, TOptional<NumericType>& SliderMinValue, TOptional<NumericType>& SliderMaxValue, NumericType& SliderExponent, NumericType& Delta, float& ShiftMultiplier, float& CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue)
{
	FProperty* Property = PropertyHandle->GetProperty();

	const FString& MetaUIMinString = Property->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = Property->GetMetaData(TEXT("UIMax"));
	const FString& SliderExponentString = Property->GetMetaData(TEXT("SliderExponent"));
	const FString& DeltaString = Property->GetMetaData(TEXT("Delta"));
	const FString& ShiftMultiplierString = Property->GetMetaData(TEXT("ShiftMultiplier"));
	const FString& CtrlMultiplierString = Property->GetMetaData(TEXT("CtrlMultiplier"));
	const FString& SupportDynamicSliderMaxValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
	const FString& SupportDynamicSliderMinValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
	const FString& ClampMinString = Property->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = Property->GetMetaData(TEXT("ClampMax"));

	// If no UIMin/Max was specified then use the clamp string
	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
	NumericType ClampMax = TNumericLimits<NumericType>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
	}

	NumericType UIMin = TNumericLimits<NumericType>::Lowest();
	NumericType UIMax = TNumericLimits<NumericType>::Max();
	TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
	TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);

	SliderExponent = NumericType(1);

	if (SliderExponentString.Len())
	{
		TTypeFromString<NumericType>::FromString(SliderExponent, *SliderExponentString);
	}

	Delta = NumericType(0);

	if (DeltaString.Len())
	{
		TTypeFromString<NumericType>::FromString(Delta, *DeltaString);
	}

	ShiftMultiplier = 10.f;
	if (ShiftMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(ShiftMultiplier, *ShiftMultiplierString);
	}

	CtrlMultiplier = 0.1f;
	if (CtrlMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(CtrlMultiplier, *CtrlMultiplierString);
	}

	const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
	const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

	MinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
	MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
	SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
	SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

	SupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
	SupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
}

template<typename NumericType>
TOptional<NumericType> FChaosClothWeightedValueCustomization::OnGetValue(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	NumericType NumericVal = 0;
	if (WeakHandlePtr.Pin()->GetValue(NumericVal) == FPropertyAccess::Success)
	{
		return TOptional<NumericType>(NumericVal);
	}

	// Value couldn't be accessed.  Return an unset value
	return TOptional<NumericType>();
}

template<typename NumericType>
void FChaosClothWeightedValueCustomization::OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags;
	SetValue(NewValue, Flags, WeakHandlePtr);
}


template<typename NumericType>
void FChaosClothWeightedValueCustomization::OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	if (bIsUsingSlider)
	{
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange;
		SetValue(NewValue, Flags, WeakHandlePtr);
	}
}

template<typename NumericType>
void FChaosClothWeightedValueCustomization::SetValue(NumericType NewValue, EPropertyValueSetFlags::Type Flags, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	if (bPreserveScaleRatio)
	{
		// Get the value for each object for the modified component
		TArray<FString> OldValues;
		if (WeakHandlePtr.Pin()->GetPerObjectValues(OldValues) == FPropertyAccess::Success)
		{
			// Loop through each object and scale based on the new ratio for each object individually
			for (int32 OutputIndex = 0; OutputIndex < OldValues.Num(); ++OutputIndex)
			{
				NumericType OldValue;
				TTypeFromString<NumericType>::FromString(OldValue, *OldValues[OutputIndex]);

				// Account for the previous scale being zero.  Just set to the new value in that case?
				NumericType Ratio = OldValue == 0 ? NewValue : NewValue / OldValue;
				if (Ratio == 0)
				{
					Ratio = NewValue;
				}

				// Loop through all the child handles (each component of the math struct, like X, Y, Z...etc)
				for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
				{
					// Ignore scaling our selves.
					TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
					if (ChildHandle != WeakHandlePtr.Pin())
					{
						// Get the value for each object.
						TArray<FString> ObjectChildValues;
						if (ChildHandle->GetPerObjectValues(ObjectChildValues) == FPropertyAccess::Success)
						{
							// Individually scale each object's components by the same ratio.
							for (int32 ChildOutputIndex = 0; ChildOutputIndex < ObjectChildValues.Num(); ++ChildOutputIndex)
							{
								NumericType ChildOldValue;
								TTypeFromString<NumericType>::FromString(ChildOldValue, *ObjectChildValues[ChildOutputIndex]);

								NumericType ChildNewValue = ChildOldValue * Ratio;
								ObjectChildValues[ChildOutputIndex] = TTypeToString<NumericType>::ToSanitizedString(ChildNewValue);
							}

							ChildHandle->SetPerObjectValues(ObjectChildValues);
						}
					}
				}
			}
		}
	}

	WeakHandlePtr.Pin()->SetValue(NewValue, Flags);
}

void FChaosClothWeightedValueCustomization::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	GEditor->BeginTransaction(LOCTEXT("SetVectorProperty", "Set Vector Property"));
}


template<typename NumericType>
void FChaosClothWeightedValueCustomization::OnEndSliderMovement(NumericType NewValue)
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}

template <typename NumericType>
void FChaosClothWeightedValueCustomization::OnDynamicSliderMaxValueChanged(NumericType NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher)
{
	for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
	{
		TSharedPtr<SNumericEntryBox<NumericType>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<NumericType>>(Widget.Pin());

		if (NumericBox.IsValid())
		{
			TSharedPtr<SSpinBox<NumericType>> SpinBox = StaticCastSharedPtr<SSpinBox<NumericType>>(NumericBox->GetSpinBox());

			if (SpinBox.IsValid())
			{
				if (SpinBox != InValueChangedSourceWidget)
				{
					if ((NewMaxSliderValue > SpinBox->GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
					{
						// Make sure the max slider value is not a getter otherwise we will break the link!
						verifySlow(!SpinBox->IsMaxSliderValueBound());
						SpinBox->SetMaxSliderValue(NewMaxSliderValue);
					}
				}
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMaxValueChanged.Broadcast((float)NewMaxSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfHigher);
	}
}

template <typename NumericType>
void FChaosClothWeightedValueCustomization::OnDynamicSliderMinValueChanged(NumericType NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower)
{
	for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
	{
		TSharedPtr<SNumericEntryBox<NumericType>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<NumericType>>(Widget.Pin());

		if (NumericBox.IsValid())
		{
			TSharedPtr<SSpinBox<NumericType>> SpinBox = StaticCastSharedPtr<SSpinBox<NumericType>>(NumericBox->GetSpinBox());

			if (SpinBox.IsValid())
			{
				if (SpinBox != InValueChangedSourceWidget)
				{
					if ((NewMinSliderValue < SpinBox->GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
					{
						// Make sure the min slider value is not a getter otherwise we will break the link!
						verifySlow(!SpinBox->IsMinSliderValueBound());
						SpinBox->SetMinSliderValue(NewMinSliderValue);
					}
				}
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMinValueChanged.Broadcast((float)NewMinSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfLower);
	}
}

#undef LOCTEXT_NAMESPACE
