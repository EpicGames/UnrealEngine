// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MathStructCustomizations.h"

template <typename OptionalType> struct TOptional;

/**
 * Customization for all property that also uses weight maps.
 * Works like a FMathStructCustomization with the difference that
 * the structure member short names are displayed in the header's row.
 */
class FChaosClothWeightedValueCustomization
	: public FMathStructCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FChaosClothWeightedValueCustomization();
	virtual ~FChaosClothWeightedValueCustomization() override;

protected:
	virtual TSharedRef<SWidget> MakeChildWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle) override;

private:
	// Have to duplicate all protected templates from FMathStructCustomization, since they are not correctly exported
	template <typename NumericType>
	static void ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, TOptional<NumericType>& MinValue, TOptional<NumericType>& MaxValue, TOptional<NumericType>& SliderMinValue, TOptional<NumericType>& SliderMaxValue, NumericType& SliderExponent, NumericType& Delta, float& ShiftMultiplier, float& CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue);

	template<typename NumericType>
	TSharedRef<SWidget> MakeNumericWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle);

	template<typename NumericType>
	TOptional<NumericType> OnGetValue(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;

	template<typename NumericType>
	void OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr);

	template<typename NumericType>
	void OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr);

	template<typename NumericType>
	void SetValue(NumericType NewValue, EPropertyValueSetFlags::Type Flags, TWeakPtr<IPropertyHandle> WeakHandlePtr);

	void OnBeginSliderMovement();

	template<typename NumericType>
	void OnEndSliderMovement(NumericType NewValue);

	template <typename NumericType>
	void OnDynamicSliderMaxValueChanged(NumericType NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher);

	template <typename NumericType>
	void OnDynamicSliderMinValueChanged(NumericType NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower);
};
