// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothEditor/ChaosSimulationEditorExtender.h"

#include "ChaosCloth/ChaosClothingSimulationFactory.h"
#include "ChaosCloth/ChaosClothingSimulation.h"
#include "ChaosCloth/ChaosClothVisualization.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "IPersonaPreviewScene.h"

#define LOCTEXT_NAMESPACE "ChaosSimulationEditorExtender"

namespace Chaos
{


DECLARE_DELEGATE_TwoParams(FClothVisualizationDebugDraw, const ::Chaos::FClothVisualizationNoGC&, const ::Chaos::FClothVisualizationNoGC::FDrawContext&);
DECLARE_DELEGATE_TwoParams(FClothVisualizationDebugDrawTexts, const ::Chaos::FClothVisualizationNoGC&, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext&);

struct FVisualizationOption
{
	// Actual option entries
	static const FVisualizationOption OptionData[];
	static const uint32 Count;

	// Chaos debug draw function
	FClothVisualizationDebugDraw DebugDrawFunction;
	FClothVisualizationDebugDrawTexts DebugDrawTextsFunction;

	FText DisplayName;         // Text for menu entries.
	FText ToolTip;             // Text for menu tooltips.
	bool bDisablesSimulation;  // Whether or not this option requires the simulation to be disabled.
	bool bHidesClothSections;  // Hides the cloth section to avoid zfighting with the debug geometry.

	// Console override
	IConsoleVariable* const ConsoleVariable;

	FVisualizationOption(const FClothVisualizationDebugDraw& InDebugDrawFunction, const TCHAR* ConsoleName, const FText& InDisplayName, const FText& InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false)
		: DebugDrawFunction(InDebugDrawFunction)
		, DebugDrawTextsFunction(nullptr)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, ConsoleVariable(IConsoleManager::Get().RegisterConsoleVariable(ConsoleName, false, *InToolTip.ToString(), 0))
	{}

	FVisualizationOption(const FClothVisualizationDebugDrawTexts& InDebugDrawTextsFunction, const TCHAR* ConsoleName, const FText& InDisplayName, const FText& InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false)
		: DebugDrawFunction(nullptr)
		, DebugDrawTextsFunction(InDebugDrawTextsFunction)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, ConsoleVariable(IConsoleManager::Get().RegisterConsoleVariable(ConsoleName, false, *InToolTip.ToString(), 0))
	{}

	~FVisualizationOption()
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleVariable);
	}

	bool IsConsoleVariableEnabled() const { return ConsoleVariable->AsVariableBool()->GetValueOnGameThread(); }
};

const FVisualizationOption FVisualizationOption::OptionData[] = 
{
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawPhysMeshShaded(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawPhysMeshShaded"),
		LOCTEXT("ChaosVisName_PhysMesh", "Physical Mesh (Flat Shaded)"),
		LOCTEXT("ChaosVisName_PhysMeshShaded_ToolTip", "Draws the current physical result as a doubled sided flat shaded mesh"),
		/*bDisablesSimulation =*/false,
		/*bHidesClothSections=*/true),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawPhysMeshWired(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawPhysMeshWired"),
		LOCTEXT("ChaosVisName_PhysMeshWire", "Physical Mesh (Wireframe)"),
		LOCTEXT("ChaosVisName_PhysMeshWired_ToolTip", "Draws the current physical mesh result in wireframe")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawAnimMeshWired(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawAnimMeshWired"),
		LOCTEXT("ChaosVisName_AnimMeshWire", "Animated Mesh (Wireframe)"),
		LOCTEXT("ChaosVisName_AnimMeshWired_ToolTip", "Draws the current animated mesh input in wireframe")),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawParticleIndices(Context);
		}),
		TEXT("p.ChaosClothEditor.DebugDrawParticleIndices"),
		LOCTEXT("ChaosVisName_ParticleIndices", "Particle Indices"),
		LOCTEXT("ChaosVisName_ParticleIndices_ToolTip", "Draws the particle indices as instantiated by the solver")),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawElementIndices(Context);
		}),
		TEXT("p.ChaosClothEditor.DebugDrawElementIndices"),
		LOCTEXT("ChaosVisName_ElementIndices", "Element Indices"),
		LOCTEXT("ChaosVisName_ElementIndices_ToolTip", "Draws the element's (triangle or other) indices as instantiated by the solver")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawPointNormals(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawPointNormals"),
		LOCTEXT("ChaosVisName_PointNormals", "Physical Mesh Normals"),
		LOCTEXT("ChaosVisName_PointNormals_ToolTip", "Draws the current point normals for the simulation mesh")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawPointVelocities(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawPointVelocities"),
		LOCTEXT("ChaosVisName_PointVelocities", "Point Velocities"),
		LOCTEXT("ChaosVisName_PointVelocities_ToolTip", "Draws the current point velocities for the simulation mesh")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawAnimNormals(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawAnimNormals"),
		LOCTEXT("ChaosVisName_AnimNormals", "Animated Mesh Normals"),
		LOCTEXT("ChaosVisName_AnimNormals_ToolTip", "Draws the current point normals for the animated mesh")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawCollision(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawCollision"),
		LOCTEXT("ChaosVisName_Collision", "Collisions"),
		LOCTEXT("ChaosVisName_Collision_ToolTip", "Draws the collision bodies the simulation is currently using")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawBackstops(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawBackstops"),
		LOCTEXT("ChaosVisName_Backstop", "Backstops"),
		LOCTEXT("ChaosVisName_Backstop_ToolTip", "Draws the backstop radius and position for each simulation particle")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawBackstopDistances(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawBackstopDistances"),
		LOCTEXT("ChaosVisName_BackstopDistance", "Backstop Distances"),
		LOCTEXT("ChaosVisName_BackstopDistance_ToolTip", "Draws the backstop distance offset for each simulation particle"),
		/*bDisablesSimulation =*/true),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawMaxDistances(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawMaxDistances"),
		LOCTEXT("ChaosVisName_MaxDistance", "Max Distances"),
		LOCTEXT("ChaosVisName_MaxDistance_ToolTip", "Draws the current max distances for the sim particles as a line along its normal"),
		/*bDisablesSimulation =*/true),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
			{
				Visualization.DrawMaxDistanceValues(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawMaxDistanceValues"),
		LOCTEXT("ChaosVisName_MaxDistanceValue", "Max Distances As Numbers"),
		LOCTEXT("ChaosVisName_MaxDistanceValue_ToolTip", "Draws the current max distances as numbers")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawAnimDrive(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawAnimDrive"),
		LOCTEXT("ChaosVisName_AnimDrive", "Anim Drive"),
		LOCTEXT("ChaosVisName_AnimDrive_Tooltip", "Draws the current skinned reference mesh for the simulation which anim drive will attempt to reach if enabled")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawEdgeConstraint(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawEdgeConstraint"),
		LOCTEXT("ChaosVisName_EdgeConstraint", "Edge Constraint"),
		LOCTEXT("ChaosVisName_EdgeConstraint_Tooltip", "Draws the edge spring constraints")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawBendingConstraint(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawBendingConstraint"),
		LOCTEXT("ChaosVisName_BendingConstraint", "Bending Constraint"),
		LOCTEXT("ChaosVisName_BendingConstraint_Tooltip", "Draws the bending spring constraints")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawLongRangeConstraint(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawLongRangeConstraint"),
		LOCTEXT("ChaosVisName_LongRangeConstraint", "Long Range Constraint"),
		LOCTEXT("ChaosVisName_LongRangeConstraint_Tooltip", "Draws the long range attachment constraint distances")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawWindAndPressureForces(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawWindAndPressureForces"),
		LOCTEXT("ChaosVisName_WindAndPressureForces", "Wind Aerodynamic And Pressure Forces"),
		LOCTEXT("ChaosVisName_WindAndPressure_Tooltip", "Draws the Wind drag and lift and pressure forces")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawWindVelocity(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawWindVelocity"),
		LOCTEXT("ChaosVisName_WindVelocity", "Wind Velocity"),
		LOCTEXT("ChaosVisName_WindVelocity_Tooltip", "Draws the Wind Velocity vector")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawLocalSpace(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawLocalSpace"),
		LOCTEXT("ChaosVisName_LocalSpace", "Local Space Reference Bone"),
		LOCTEXT("ChaosVisName_LocalSpace_Tooltip", "Draws the local space reference bone")),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
			{
				Visualization.DrawLocalSpaceBoneNames(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawLocalSpaceBoneNames"),
		LOCTEXT("ChaosVisName_LocalSpaceBoneNames", "Local Space Reference Bone Names"),
		LOCTEXT("ChaosVisName_LocalSpaceBoneNames_Tooltip", "Draws the local space reference bone names")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawVelocityScale(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawVelocityScale"),
		LOCTEXT("ChaosVisName_VelocityScale", "Velocity Scale"),
		LOCTEXT("ChaosVisName_VelocityScale_Tooltip", "Draws the velocity scale")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawSelfCollision(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawSelfCollision"),
		LOCTEXT("ChaosVisName_SelfCollision", "Self Collision"),
		LOCTEXT("ChaosVisName_SelfCollision_Tooltip", "Draws the self collision thickness/debugging information")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
			{
				Visualization.DrawSelfIntersection(Context);
			}),
		TEXT("p.ChaosClothEditor.DebugDrawSelfIntersection"),
		LOCTEXT("ChaosVisName_SelfIntersection", "Self Intersection"),
		LOCTEXT("ChaosVisName_SelfIntersection_Tooltip", "Draws the self intersection contour/region information"))
};
const uint32 FVisualizationOption::Count = sizeof(OptionData) / sizeof(FVisualizationOption);

namespace ClothEditorCVar
{
	namespace Private
	{
		// Add these as visualization options at some point?
		static TAutoConsoleVariable<int32> DebugDrawSingleVertex(TEXT("p.ChaosClothEditor.DebugDrawSingleVertex"), -1, TEXT("Filter Chaos Cloth debug draw to a single vertex. Default = -1 is no filtering."), ECVF_Cheat);
		static TAutoConsoleVariable<FString> DebugDrawFilterSet(TEXT("p.ChaosClothEditor.DebugDrawFilterSet"), "", TEXT("Filter Chaos Cloth debug draw to a named sim vertex set. Default ='' is no filtering."), ECVF_Cheat);
	}
}

FSimulationEditorExtender::FSimulationEditorExtender()
	: Flags(false, FVisualizationOption::Count)
{
}

FSimulationEditorExtender::~FSimulationEditorExtender() = default;

UClass* FSimulationEditorExtender::GetSupportedSimulationFactoryClass()
{
	return UChaosClothingSimulationFactory::StaticClass();
}

void FSimulationEditorExtender::ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, TSharedRef<IPersonaPreviewScene> PreviewScene)
{
	MenuBuilder.BeginSection(GetSectionName(), GetSectionHeadingText());
	{
		for (uint32 OptionIndex = 0; OptionIndex < FVisualizationOption::Count; ++OptionIndex)
		{
			// Handler for visualization entry being clicked
			const FExecuteAction ExecuteAction = FExecuteAction::CreateLambda([this, OptionIndex, PreviewScene]()
			{
				Flags[OptionIndex] = !Flags[OptionIndex];

				// If we need to toggle the disabled or visibility states, handle it
				if (UDebugSkelMeshComponent* const MeshComponent = PreviewScene->GetPreviewMeshComponent())
				{
					// Disable simulation
					const bool bShouldDisableSimulation = ShouldDisableSimulation();
					if (bShouldDisableSimulation && MeshComponent->bDisableClothSimulation != bShouldDisableSimulation)
					{
						MeshComponent->bDisableClothSimulation = !MeshComponent->bDisableClothSimulation;
					}
					// Hide cloth section
					if (FVisualizationOption::OptionData[OptionIndex].bHidesClothSections)
					{
						const bool bIsClothSectionsVisible = !Flags[OptionIndex];
						ShowClothSections(MeshComponent, bIsClothSectionsVisible);
					}
				}
			});

			// Checkstate function for visualization entries
			const FIsActionChecked IsActionChecked = FIsActionChecked::CreateLambda([this, OptionIndex]()
			{
				return Flags[OptionIndex];
			});

			const FUIAction Action(ExecuteAction, FCanExecuteAction(), IsActionChecked);

			MenuBuilder.AddMenuEntry(FVisualizationOption::OptionData[OptionIndex].DisplayName, FVisualizationOption::OptionData[OptionIndex].ToolTip, FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::ToggleButton);
		}
	}
	MenuBuilder.EndSection();
}

void FSimulationEditorExtender::DebugDrawSimulation(const IClothingSimulationInterface* Simulation, USkeletalMeshComponent* OwnerComponent, FPrimitiveDrawInterface* PDI)
{
	if (!ensure(Simulation) || (OwnerComponent && OwnerComponent->bDisableClothSimulation))  // Can't debug draw when the simulation is disabled
	{
		return;
	}

	if (const FClothVisualizationNoGC* const ClothVisualization = GetClothVisualization(Simulation))
	{
		FClothVisualizationNoGC::FDrawContext Context(PDI);
		Context.Filter.SingleVertex = ClothEditorCVar::Private::DebugDrawSingleVertex.GetValueOnAnyThread();
		Context.Filter.VertexSet = ClothEditorCVar::Private::DebugDrawFilterSet.GetValueOnAnyThread();
		for (int32 OptionIndex = 0; OptionIndex < FVisualizationOption::Count; ++OptionIndex)
		{
			if (FVisualizationOption::OptionData[OptionIndex].DebugDrawFunction.IsBound() &&
				(FVisualizationOption::OptionData[OptionIndex].IsConsoleVariableEnabled() || Flags[OptionIndex]))
			{
				FVisualizationOption::OptionData[OptionIndex].DebugDrawFunction.Execute(*ClothVisualization, Context);
			}
		}
	}
}

void FSimulationEditorExtender::DebugDrawSimulationTexts(const IClothingSimulationInterface* Simulation, USkeletalMeshComponent* OwnerComponent, FCanvas* Canvas, const FSceneView* SceneView)
{
	if (!ensure(Simulation) || (OwnerComponent && OwnerComponent->bDisableClothSimulation))  // Can't debug draw when the simulation is disabled
	{
		return;
	}

	if (const FClothVisualizationNoGC* const ClothVisualization = GetClothVisualization(Simulation))
	{
		FClothVisualizationNoGC::FDrawTextsContext Context(Canvas, SceneView);
		Context.Filter.SingleVertex = ClothEditorCVar::Private::DebugDrawSingleVertex.GetValueOnAnyThread();
		Context.Filter.VertexSet = ClothEditorCVar::Private::DebugDrawFilterSet.GetValueOnAnyThread();
		for (int32 OptionIndex = 0; OptionIndex < FVisualizationOption::Count; ++OptionIndex)
		{
			if (FVisualizationOption::OptionData[OptionIndex].DebugDrawTextsFunction.IsBound() &&
				(FVisualizationOption::OptionData[OptionIndex].IsConsoleVariableEnabled() || Flags[OptionIndex]))
			{
				FVisualizationOption::OptionData[OptionIndex].DebugDrawTextsFunction.Execute(*ClothVisualization, Context);
			}
		}
	}
}

FName FSimulationEditorExtender::GetSectionName() const
{
	return FName("ChaosSimulation_Visualizations");
}

FText FSimulationEditorExtender::GetSectionHeadingText() const
{
	return FText(LOCTEXT("VisualizationSection", "Skeletal Mesh Clothing Visualization"));
}

const FClothVisualizationNoGC* FSimulationEditorExtender::GetClothVisualization(const IClothingSimulationInterface* Simulation) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	return Simulation ? static_cast<const FClothingSimulation*>(Simulation)->GetClothVisualization() : nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FSimulationEditorExtender::ShouldDisableSimulation() const
{
	for (uint32 OptionIndex = 0; OptionIndex < FVisualizationOption::Count; ++OptionIndex)
	{
		if (Flags[OptionIndex])
		{
			const FVisualizationOption& Data = FVisualizationOption::OptionData[OptionIndex];

			if (Data.bDisablesSimulation)
			{
				return true;
			}
		}
	}
	return false;
}

void FSimulationEditorExtender::ShowClothSections(USkeletalMeshComponent* MeshComponent, bool bIsClothSectionsVisible) const
{
	if (FSkeletalMeshRenderData* const SkeletalMeshRenderData = MeshComponent->GetSkeletalMeshRenderData())
	{
		for (int32 LODIndex = 0; LODIndex < SkeletalMeshRenderData->LODRenderData.Num(); ++LODIndex)
		{
			FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData = SkeletalMeshRenderData->LODRenderData[LODIndex];

			for (int32 SectionIndex = 0; SectionIndex < SkeletalMeshLODRenderData.RenderSections.Num(); ++SectionIndex)
			{
				FSkelMeshRenderSection& SkelMeshRenderSection = SkeletalMeshLODRenderData.RenderSections[SectionIndex];

				if (SkelMeshRenderSection.HasClothingData())
				{
					MeshComponent->ShowMaterialSection(SkelMeshRenderSection.MaterialIndex, SectionIndex, bIsClothSectionsVisible, LODIndex);
				}
			}
		}
	}
}

}  // End namespace Chaos

#undef LOCTEXT_NAMESPACE
