// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "ChaosClothEditor/ChaosSimulationEditorExtender.h"

class FChaosClothEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	Chaos::FSimulationEditorExtender ChaosEditorExtender;
};
