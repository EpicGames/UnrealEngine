// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimulationEditorExtender.h"

namespace Chaos
{
	class FClothVisualizationNoGC;

	/** Chaos extension to the asset editor. */
	class FSimulationEditorExtender : public ISimulationEditorExtender
	{
	public:
		CHAOSCLOTHEDITOR_API FSimulationEditorExtender();
		CHAOSCLOTHEDITOR_API virtual ~FSimulationEditorExtender() override;

		// ISimulationEditorExtender Interface
		virtual UClass* GetSupportedSimulationFactoryClass() override;
		CHAOSCLOTHEDITOR_API virtual void ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, TSharedRef<IPersonaPreviewScene> PreviewScene) override;
		CHAOSCLOTHEDITOR_API virtual void DebugDrawSimulation(const IClothingSimulationInterface* Simulation, USkeletalMeshComponent* OwnerComponent, FPrimitiveDrawInterface* PDI) override;
		CHAOSCLOTHEDITOR_API virtual void DebugDrawSimulationTexts(const IClothingSimulationInterface* Simulation, USkeletalMeshComponent* OwnerComponent, FCanvas* Canvas, const FSceneView* SceneView) override;
		// End ISimulationEditorExtender Interface

	private:
		/** Return the name of the menu section. */
		virtual FName GetSectionName() const;

		/** Return the heading text of the menu section. */
		virtual FText GetSectionHeadingText() const;

		/** Return the visualization object for the specified simulation. */
		virtual const FClothVisualizationNoGC* GetClothVisualization(const IClothingSimulationInterface* Simulation) const;

		/** Return whether or not - given the current enabled options - the simulation should be disabled. */
		bool ShouldDisableSimulation() const;
		/** Show/hide all cloth sections for the specified mesh compoment. */
		void ShowClothSections(USkeletalMeshComponent* MeshComponent, bool bIsClothSectionsVisible) const;

	private:
		/** Flags used to store the checked status for the visualization options. */
		TBitArray<> Flags;
	};
}
